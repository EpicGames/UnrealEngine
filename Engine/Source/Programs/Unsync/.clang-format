---
Language:        Cpp
# BasedOnStyle:  Chromium
AccessModifierOffset: -4
AlignAfterOpenBracket: Align
AlignConsecutiveMacros: true
AlignConsecutiveAssignments: true
AlignConsecutiveBitFields: true
AlignConsecutiveDeclarations: true
AlignEscapedNewlines: Left
AlignOperands: Align
AlignTrailingComments: true
AllowAllArgumentsOnNextLine: false
AllowAllConstructorInitializersOnNextLine: false
AllowAllParametersOfDeclarationOnNextLine: false
AllowShortEnumsOnASingleLine: false
AllowShortBlocksOnASingleLine: Never
AllowShortCaseLabelsOnASingleLine: false
AllowShortFunctionsOnASingleLine: InlineOnly
AllowShortLambdasOnASingleLine: All
AllowShortIfStatementsOnASingleLine: Never
AllowShortLoopsOnASingleLine: false
AlwaysBreakAfterDefinitionReturnType: None
AlwaysBreakAfterReturnType: TopLevelDefinitions
AlwaysBreakBeforeMultilineStrings: true
AlwaysBreakTemplateDeclarations: Yes
BinPackArguments: false
BinPackParameters: false
BreakBeforeBraces: Custom
BraceWrapping:
  AfterCaseLabel:  true
  AfterClass:      true
  AfterControlStatement: Always
  AfterEnum:       false
  AfterFunction:   true
  AfterNamespace:  false
  AfterObjCDeclaration: false
  AfterStruct:     true
  AfterUnion:      true
  AfterExternBlock: true
  BeforeCatch:     true
  BeforeElse:      true
  BeforeLambdaBody: true
  BeforeWhile:     false
  IndentBraces:    false
  SplitEmptyFunction: true
  SplitEmptyRecord: true
  SplitEmptyNamespace: true
BreakBeforeBinaryOperators: None
BreakBeforeInheritanceComma: false
BreakInheritanceList: BeforeColon
BreakBeforeTernaryOperators: true
BreakConstructorInitializersBeforeComma: false
BreakConstructorInitializers: BeforeComma
BreakAfterJavaFieldAnnotations: false
BreakStringLiterals: true
ColumnLimit:     140
CommentPragmas:  '^ IWYU pragma:'
CompactNamespaces: true
ConstructorInitializerAllOnOneLineOrOnePerLine: true
ConstructorInitializerIndentWidth: 0
ContinuationIndentWidth: 4
Cpp11BracedListStyle: true
DeriveLineEnding: true
DerivePointerAlignment: false
DisableFormat:   false
ExperimentalAutoDetectBinPacking: false
FixNamespaceComments: true
ForEachMacros:
  - foreach
  - Q_FOREACH
  - BOOST_FOREACH
IncludeBlocks:   Preserve
IncludeCategories:
  - Regex:           '^<ext/.*\.h>'
    Priority:        2
    SortPriority:    0
  - Regex:           '^<.*\.h>'
    Priority:        1
    SortPriority:    0
  - Regex:           '^<.*'
    Priority:        2
    SortPriority:    0
  - Regex:           '.*'
    Priority:        3
    SortPriority:    0
IncludeIsMainRegex: '([-_](test|unittest))?$'
IncludeIsMainSourceRegex: ''
IndentCaseLabels: true
IndentCaseBlocks: true
IndentGotoLabels: false
IndentPPDirectives: AfterHash
IndentExternBlock: AfterExternBlock
IndentWidth:     4
IndentWrappedFunctionNames: false
InsertTrailingCommas: None
JavaScriptQuotes: Leave
JavaScriptWrapImports: true
KeepEmptyLinesAtTheStartOfBlocks: false
MacroBlockBegin: 'UE_TRACE_EVENT_BEGIN'
MacroBlockEnd:   'UE_TRACE_EVENT_END'
MaxEmptyLinesToKeep: 1
NamespaceIndentation: Inner
ObjCBinPackProtocolList: Never
ObjCBlockIndentWidth: 2
ObjCBreakBeforeNestedBlockParam: true
ObjCSpaceAfterProperty: false
ObjCSpaceBeforeProtocolList: true
PenaltyBreakAssignment: 2
PenaltyBreakBeforeFirstCallParameter: 1
PenaltyBreakComment: 300
PenaltyBreakFirstLessLess: 120
PenaltyBreakString: 1000
PenaltyBreakTemplateDeclaration: 10
PenaltyExcessCharacter: 1000000
PenaltyReturnTypeOnItsOwnLine: 200
PointerAlignment: Left
RawStringFormats:
  - Language:        Cpp
    Delimiters:
      - cc
      - CC
      - cpp
      - Cpp
      - CPP
      - 'c++'
      - 'C++'
    CanonicalDelimiter: ''
    BasedOnStyle:    google
  - Language:        TextProto
    Delimiters:
      - pb
      - PB
      - proto
      - PROTO
    EnclosingFunctions:
      - EqualsProto
      - EquivToProto
      - PARSE_PARTIAL_TEXT_PROTO
      - PARSE_TEST_PROTO
      - PARSE_TEXT_PROTO
      - ParseTextOrDie
      - ParseTextProtoOrDie
      - ParseTestProto
      - ParsePartialTestProto
    CanonicalDelimiter: ''
    BasedOnStyle:    google
ReflowComments:  true
SortIncludes:    true
SortUsingDeclarations: true
SpaceAfterCStyleCast: false
SpaceAfterLogicalNot: false
SpaceAfterTemplateKeyword: false
SpaceBeforeAssignmentOperators: true
SpaceBeforeCpp11BracedList: false
SpaceBeforeCtorInitializerColon: true
SpaceBeforeInheritanceColon: true
SpaceBeforeParens: ControlStatements
SpaceBeforeRangeBasedForLoopColon: true
SpaceInEmptyBlock: false
SpaceInEmptyParentheses: false
SpacesBeforeTrailingComments: 2
SpacesInAngles:  false
SpacesInConditionalStatement: false
SpacesInContainerLiterals: true
SpacesInCStyleCastParentheses: false
SpacesInParentheses: false
SpacesInSquareBrackets: false
SpaceBeforeSquareBrackets: false
BitFieldColonSpacing: Both
Standard:        Auto
StatementMacros:
  - Q_UNUSED
  - QT_REQUIRE_VERSION
TabWidth:        4
UseCRLF:         false
UseTab:          Always
WhitespaceSensitiveMacros:
  - STRINGIZE
  - PP_STRINGIZE
  - BOOST_PP_STRINGIZE
...
