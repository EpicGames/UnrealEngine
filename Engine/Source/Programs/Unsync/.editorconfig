# Visual Studio generated .editorconfig file with C++ settings.
root = true

indent_style = tab
indent_size = 4
trim_trailing_whitespace = true
insert_final_newline = true

[*.{cpp,h,inl}]

# Visual C++ Code Style settings

cpp_generate_documentation_comments = doxygen_slash_star

# Visual C++ Formatting settings

cpp_indent_braces = false
cpp_indent_multi_line_relative_to = innermost_parenthesis
cpp_indent_within_parentheses = indent
cpp_indent_preserve_within_parentheses = true
cpp_indent_case_contents = true
cpp_indent_case_labels = false
cpp_indent_case_contents_when_block = false
cpp_indent_lambda_braces_when_parameter = false
cpp_indent_goto_labels = one_left
cpp_indent_preprocessor = leftmost_column
cpp_indent_access_specifiers = false
cpp_indent_namespace_contents = true
cpp_indent_preserve_comments = false
cpp_new_line_before_open_brace_namespace = ignore
cpp_new_line_before_open_brace_type = ignore
cpp_new_line_before_open_brace_function = ignore
cpp_new_line_before_open_brace_block = ignore
cpp_new_line_before_open_brace_lambda = ignore
cpp_new_line_scope_braces_on_separate_lines = false
cpp_new_line_close_brace_same_line_empty_type = false
cpp_new_line_close_brace_same_line_empty_function = false
cpp_new_line_before_catch = true
cpp_new_line_before_else = true
cpp_new_line_before_while_in_do_while = false
cpp_space_before_function_open_parenthesis = remove
cpp_space_within_parameter_list_parentheses = false
cpp_space_between_empty_parameter_list_parentheses = false
cpp_space_after_keywords_in_control_flow_statements = true
cpp_space_within_control_flow_statement_parentheses = false
cpp_space_before_lambda_open_parenthesis = false
cpp_space_within_cast_parentheses = false
cpp_space_after_cast_close_parenthesis = false
cpp_space_within_expression_parentheses = false
cpp_space_before_block_open_brace = true
cpp_space_between_empty_braces = false
cpp_space_before_initializer_list_open_brace = false
cpp_space_within_initializer_list_braces = true
cpp_space_preserve_in_initializer_list = true
cpp_space_before_open_square_bracket = false
cpp_space_within_square_brackets = false
cpp_space_before_empty_square_brackets = false
cpp_space_between_empty_square_brackets = false
cpp_space_group_square_brackets = true
cpp_space_within_lambda_brackets = false
cpp_space_between_empty_lambda_brackets = false
cpp_space_before_comma = false
cpp_space_after_comma = true
cpp_space_remove_around_member_operators = true
cpp_space_before_inheritance_colon = true
cpp_space_before_constructor_colon = true
cpp_space_remove_before_semicolon = true
cpp_space_after_semicolon = true
cpp_space_remove_around_unary_operator = true
cpp_space_around_binary_operator = insert
cpp_space_around_assignment_operator = insert
cpp_space_pointer_reference_alignment = left
cpp_space_around_ternary_operator = insert
cpp_wrap_preserve_blocks = one_liners
