.git
.vscode
Build
build
