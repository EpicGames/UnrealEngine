cmake_minimum_required(VERSION 3.16)
set_property(GLOBAL PROPERTY USE_FOLDERS ON)

if(WIN32)
	set(VCPKG_TARGET_TRIPLET "x64-windows-static")
endif()

if(DEFINED ENV{VCPKG_ROOT})
	set(toolchain "$ENV{VCPKG_ROOT}/scripts/buildsystems/vcpkg.cmake")
	message(STATUS "Using cmake toolchain file: ${toolchain}")
	set(CMAKE_TOOLCHAIN_FILE ${toolchain} CACHE STRING "")
endif()

# Configure project

project(unsync)

option(UNSYNC_USE_DEBUG_HEAP "Include support for debug memory allocator" ON)

# Validate compiler

if (NOT WIN32 AND NOT ${CMAKE_CXX_COMPILER_ID} STREQUAL "GNU")
	message(FATAL_ERROR "Only GCC-11 or newer is currently supported.")
endif()

if (${CMAKE_CXX_COMPILER_ID} STREQUAL "GNU" AND ${CMAKE_CXX_COMPILER_VERSION} VERSION_LESS 11)
	message(FATAL_ERROR "Only GCC-11 or newer is currently supported")
endif()

# Add unsync code

set(src
	Private/UnsyncAuth.cpp
	Private/UnsyncAuth.h
	Private/UnsyncBuffer.cpp
	Private/UnsyncBuffer.h
	Private/UnsyncChunking.cpp
	Private/UnsyncChunking.h
	Private/UnsyncCmdDiff.cpp
	Private/UnsyncCmdDiff.h
	Private/UnsyncCmdHash.cpp
	Private/UnsyncCmdHash.h
	Private/UnsyncCmdInfo.cpp
	Private/UnsyncCmdInfo.h
	Private/UnsyncCmdLogin.cpp
	Private/UnsyncCmdLogin.h
	Private/UnsyncCmdMount.cpp
	Private/UnsyncCmdMount.h
	Private/UnsyncCmdPack.cpp
	Private/UnsyncCmdPack.h
	Private/UnsyncCmdPatch.cpp
	Private/UnsyncCmdPatch.h
	Private/UnsyncCmdPush.cpp
	Private/UnsyncCmdPush.h
	Private/UnsyncCmdQuery.cpp
	Private/UnsyncCmdQuery.h
	Private/UnsyncCmdSync.cpp
	Private/UnsyncCmdSync.h
	Private/UnsyncCmdVerify.cpp
	Private/UnsyncCmdVerify.h
	Private/UnsyncCommon.h
	Private/UnsyncCompression.cpp
	Private/UnsyncCompression.h
	Private/UnsyncCore.cpp
	Private/UnsyncCore.h
	Private/UnsyncDiff.cpp
	Private/UnsyncDiff.h
	Private/UnsyncError.h
	Private/UnsyncFile.cpp
	Private/UnsyncFile.h
	Private/UnsyncFilter.cpp
	Private/UnsyncFilter.h
	Private/UnsyncHash.cpp
	Private/UnsyncHash.h
	Private/UnsyncHashTable.h
	Private/UnsyncHorde.cpp
	Private/UnsyncHorde.h
	Private/UnsyncHttp.cpp
	Private/UnsyncHttp.h
	Private/UnsyncJupiter.cpp
	Private/UnsyncJupiter.h
	Private/UnsyncLog.cpp
	Private/UnsyncLog.h
	Private/UnsyncMain.cpp
	Private/UnsyncManifest.cpp
	Private/UnsyncManifest.h
	Private/UnsyncMemory.cpp
	Private/UnsyncMemory.h
	Private/UnsyncMiniCb.cpp
	Private/UnsyncMiniCb.h
	Private/UnsyncMount.cpp
	Private/UnsyncMount.h
	Private/UnsyncPack.cpp
	Private/UnsyncPack.h
	Private/UnsyncPool.h
	Private/UnsyncProgress.cpp
	Private/UnsyncProgress.h
	Private/UnsyncProtocol.cpp
	Private/UnsyncProtocol.h
	Private/UnsyncProxy.cpp
	Private/UnsyncProxy.h
	Private/UnsyncRemote.cpp
	Private/UnsyncRemote.h
	Private/UnsyncScan.h
	Private/UnsyncScavenger.cpp
	Private/UnsyncScavenger.h
	Private/UnsyncScheduler.cpp
	Private/UnsyncScheduler.h
	Private/UnsyncSerialization.cpp
	Private/UnsyncSerialization.h
	Private/UnsyncSocket.cpp
	Private/UnsyncSocket.h
	Private/UnsyncSource.cpp
	Private/UnsyncSource.h
	Private/UnsyncTarget.cpp
	Private/UnsyncTarget.h
	Private/UnsyncTest.cpp
	Private/UnsyncTest.h
	Private/UnsyncThread.cpp
	Private/UnsyncThread.h
	Private/UnsyncUtil.cpp
	Private/UnsyncUtil.h
	Private/UnsyncVarInt.h
	Private/UnsyncVersion.cpp
	Private/UnsyncVersion.h
)

# Add third party code

set(thirdparty
	ThirdParty/md5-sse2.cpp
	ThirdParty/md5-sse2.h
	ThirdParty/json11.cpp
	ThirdParty/json11.hpp
)

set(thirdparty ${thirdparty}
	ThirdParty/ig-debugheap/DebugHeap.h
	ThirdParty/ig-debugheap/DebugHeap.c
)

add_executable(unsync ${src} ${thirdparty} ${generated})

set_property(TARGET unsync PROPERTY CXX_STANDARD 20)
set_property(TARGET unsync PROPERTY MSVC_RUNTIME_LIBRARY "MultiThreaded$<$<CONFIG:Debug>:Debug>")

find_package(CLI11 CONFIG REQUIRED)
find_package(fmt CONFIG REQUIRED)
find_package(unofficial-http-parser CONFIG REQUIRED)
find_package(zstd CONFIG REQUIRED)


target_include_directories(unsync PRIVATE ${CMAKE_CURRENT_SOURCE_DIR}/thirdparty)

target_link_directories(unsync PRIVATE ${VCPKG_INSTALLED_DIR}/${VCPKG_TARGET_TRIPLET}/lib)

target_link_libraries(unsync PRIVATE
	CLI11::CLI11
	fmt::fmt-header-only # Header-only version as workaround for linker errors on Mac
	unofficial::http_parser::http_parser
	zstd::libzstd_static
)

find_path(BLAKE3_INCLUDE_DIR blake3.h REQUIRED)
target_include_directories(unsync PRIVATE ${BLAKE3_INCLUDE_DIR})
target_link_libraries(unsync PRIVATE blake3)

find_path(LIBRESSL_INCLUDE_DIR tls.h)
target_include_directories(unsync PRIVATE ${LIBRESSL_INCLUDE_DIR})
target_link_libraries(unsync PRIVATE tls ssl crypto)

if (UNSYNC_USE_DEBUG_HEAP)
	target_compile_definitions(unsync PRIVATE UNSYNC_USE_DEBUG_HEAP=1)
endif(UNSYNC_USE_DEBUG_HEAP)

if(${CMAKE_SYSTEM_NAME} MATCHES "Windows")
	target_compile_definitions(unsync PRIVATE
		UNSYNC_PLATFORM_WINDOWS=1
		UNSYNC_PLATFORM_UNIX=0
		WIN32_LEAN_AND_MEAN=1
		NOMINMAX=1
	)
else()
	target_compile_definitions(unsync PRIVATE 
		UNSYNC_PLATFORM_UNIX=1
		UNSYNC_PLATFORM_WINDOWS=0
	)
endif()

if(MSVC)

	add_definitions(-MP)
	add_definitions(-D_SILENCE_CXX17_OLD_ALLOCATOR_MEMBERS_DEPRECATION_WARNING=1)
	add_definitions(-D_CRT_SECURE_NO_WARNINGS=1)
	add_definitions(-D_WINSOCK_DEPRECATED_NO_WARNINGS=1)
	add_definitions(-D_SILENCE_CXX17_CODECVT_HEADER_DEPRECATION_WARNING=1)
	target_compile_options(unsync PRIVATE 
		-W4 -WX # level 4 warnings as errors
		-wd4100 # unreferenced formal parameter
	)

	# Less strict warnings in third party code
	foreach(file ${thirdparty})
		set_source_files_properties(${file} PROPERTIES COMPILE_FLAGS -W0)
	endforeach()

	set_property(DIRECTORY ${CMAKE_CURRENT_SOURCE_DIR} PROPERTY VS_STARTUP_PROJECT unsync)

endif()

source_group("Private" FILES ${src})
source_group("Generated" FILES ${generated})
source_group("ThirdParty" FILES ${thirdparty})
