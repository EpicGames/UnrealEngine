// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncAuth.h"
#include "UnsyncFile.h"
#include "UnsyncHttp.h"
#include "UnsyncLog.h"
#include "UnsyncProxy.h"

#include <fmt/format.h>
#include <ctime>
#include <json11.hpp>
#include <optional>

#include <openssl/err.h>
#include <openssl/evp.h>  // Base64 encoding
#include <openssl/rand.h>
#include <openssl/sha.h>

#if UNSYNC_PLATFORM_WINDOWS
#	define UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN 1
#	include <Windows.h>
#	include <Dpapi.h>
#	pragma comment(lib, "Crypt32.lib")
#else
#	define UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN 0
#endif

namespace unsync {

std::string
SecureRandomBytesAsHexString(uint32 NumBytes)
{
	static constexpr int NumStackBytes				 = 64;
	unsigned char		 StackStorage[NumStackBytes] = {};
	FBuffer				 DynamicStorage;

	unsigned char* RandomState = nullptr;

	if (NumBytes <= NumStackBytes)
	{
		RandomState = StackStorage;
	}
	else
	{
		DynamicStorage.Resize(NumBytes);
		RandomState = DynamicStorage.Data();
	}

	int RandResult = RAND_bytes(RandomState, NumBytes);
	if (RandResult != 1)
	{
		int ErrorCode = ERR_get_error();
		UNSYNC_FATAL(L"Failed to generate secure random number. Error code: %d", ErrorCode);
	}

	return BytesToHexString(RandomState, NumBytes);
}

FHash256
HashSha256Bytes(const uint8* Data, uint64 Size)
{
	FHash256 Result = {};
	static_assert(sizeof(Result.Data) == SHA256_DIGEST_LENGTH, "Unexpected SHA256 output buffer size");

	SHA256_CTX ShaCtx = {};

	UNSYNC_ASSERT(SHA256_Init(&ShaCtx) == 1);
	UNSYNC_ASSERT(SHA256_Update(&ShaCtx, Data, Size) == 1);
	UNSYNC_ASSERT(SHA256_Final(Result.Data, &ShaCtx) == 1);

	return Result;
}

std::string
EncodeBase64(const uint8* Data, uint64 Size)
{
	UNSYNC_ASSERT(Size <= std::numeric_limits<int32>::max());

	std::string Result;

	const uint64 ExpectedResultLength = ((Size + 2) / 3) * 4;

	Result.resize(ExpectedResultLength);

	int NumEncodedBytes = EVP_EncodeBlock((unsigned char*)Result.data(), (const unsigned char*)Data, (int)Size);
	UNSYNC_ASSERT(NumEncodedBytes == ExpectedResultLength)

	return Result;
}

bool
DecodeBase64(std::string_view Base64Data, FBuffer& Output)
{
	const uint64 ExpectedResultLength = 3 * Base64Data.length() / 4;
	UNSYNC_ASSERT(ExpectedResultLength <= std::numeric_limits<int32>::max());

	Output.Resize(ExpectedResultLength);  // Conservative size, since EVP_DecodeBlock fills padding with 0

	int NumDecodedBytes = EVP_DecodeBlock((unsigned char*)Output.Data(), (const unsigned char*)Base64Data.data(), (int)Base64Data.length());

	return NumDecodedBytes == ExpectedResultLength;
}

void
TransformBase64VanillaToUrlSafe(std::string& Data)
{
	std::replace(Data.begin(), Data.end(), '+', '-');
	std::replace(Data.begin(), Data.end(), '/', '_');

	while (Data.ends_with('='))
	{
		Data.pop_back();
	}
}

void
TransformBase64UrlSafeToVanilla(std::string& Data)
{
	std::replace(Data.begin(), Data.end(), '-', '+');
	std::replace(Data.begin(), Data.end(), '_', '/');

	while ((Data.length() % 4) != 0)
	{
		Data.push_back('=');
	}
}

std::string
GetPKCECodeChallenge(std::string_view CodeVerifier)
{
	FHash256	CodeVerifierHash = HashSha256Bytes((const uint8*)CodeVerifier.data(), CodeVerifier.size());
	std::string Result			 = EncodeBase64(CodeVerifierHash.Data, CodeVerifierHash.Size());

	TransformBase64VanillaToUrlSafe(Result);

	return Result;
}

static const char HttpCallbackResponseOk[] = R"(HTTP/1.1 200 OK

<!DOCTYPE html>
<html>
<body>
<center>
<h1 style="background-color:#75dd55">Success!</h1>
<p>Unsync is now authorized. You may close this page.</p>
</center>
</body>
</html>
)";

static const char HttpCallbackResponseError[] = R"(HTTP/1.1 400 Bad Request

<!DOCTYPE html>
<html>
<body>
<center>
<h1 style="background-color:#dd5555">Authorization failed!</h1>
<p>See unsync logs for details. You may close this page.</p>
</center>
</body>
</html>
)";

struct FHttpCallbackData
{
	std::string AuthCode;
	std::string State;
};

std::thread
StartHttpCallbackServer(FSocketHandle	   CallbackListenSocket,
						std::string_view   ExpectedPath,
						std::string_view   RandomState,
						FHttpCallbackData& HttpCallbackData)
{
	return std::thread([CallbackListenSocket, ExpectedPath, RandomState, &HttpCallbackData]() {
		FSocketHandle CallbackSocket = SocketAccept(CallbackListenSocket);

		static const size_t MaxRecvSize = 65536;
		char				RecvBuffer[MaxRecvSize];

		int32 ReceivedBytes = SocketRecvAny(CallbackSocket, RecvBuffer, MaxRecvSize);

		UNSYNC_VERBOSE2(L"HTTP Callback:\n%.*hs", ReceivedBytes, RecvBuffer);

		std::string_view RequestStr(RecvBuffer, ReceivedBytes);

		std::string ExpectedCallbackPrefix = fmt::format("GET /{}", ExpectedPath);

		if (RequestStr.starts_with(ExpectedCallbackPrefix))
		{
			// Trim request string, removing HTTP headers
			{
				size_t RequestEndPos = RequestStr.find("\n");
				if (RequestEndPos != std::string::npos)
				{
					RequestStr = RequestStr.substr(0, RequestEndPos);
				}
			}

			auto ExtractValue = [](std::string_view RequestStr, std::string_view Key) -> std::string_view {
				size_t			 Pos	= RequestStr.find(Key);
				std::string_view Result = {};
				if (Pos != std::string::npos)
				{
					Result = RequestStr.substr(Pos + Key.length());
					Result = Result.substr(0, Result.find_first_of("& \n"));
				}
				return Result;
			};

			HttpCallbackData.AuthCode = ExtractValue(RequestStr, "code=");
			HttpCallbackData.State	  = ExtractValue(RequestStr, "state=");

			if (HttpCallbackData.State == RandomState && !HttpCallbackData.AuthCode.empty())
			{
				SocketSend(CallbackSocket, HttpCallbackResponseOk, strlen(HttpCallbackResponseOk));
			}
			else
			{
				// TODO: could report more detailed error to the browser, but probably just the log file is sufficient
				SocketSend(CallbackSocket, HttpCallbackResponseError, strlen(HttpCallbackResponseError));
			}
		}
		else
		{
			const char ResponseNotFound[] = "HTTP/1.1 404 Not Found";
			SocketSend(CallbackSocket, ResponseNotFound, strlen(ResponseNotFound));
		}

		SocketClose(CallbackSocket);
	});
};

TResult<json11::Json>
DecodeJwtPayload(std::string JwtDataBase64Url)
{
	using namespace json11;

	size_t PayloadOffset = JwtDataBase64Url.find('.');
	if (PayloadOffset == std::string::npos)
	{
		return AppError(L"Failed to locate JWT payload section");
	}
	PayloadOffset += 1;	 // skip the delimiter

	size_t SignatureOffset = JwtDataBase64Url.find('.', PayloadOffset + 1);
	if (SignatureOffset == std::string::npos)
	{
		return AppError(L"Failed to locate JWT signature section");
	}
	SignatureOffset += 1;  // skip the delimiter

	size_t PayloadLength = SignatureOffset - PayloadOffset - 1;

	std::string JwtPayloadBase64 = JwtDataBase64Url.substr(PayloadOffset, PayloadLength);
	TransformBase64UrlSafeToVanilla(JwtPayloadBase64);

	FBuffer JasonData;
	bool	bDecoded = DecodeBase64(JwtPayloadBase64, JasonData);

	if (!bDecoded)
	{
		return AppError(L"Failed to decode Base64 JWT data");
	}

	JasonData.PushBack(0);

	std::string JsonErrorString;
	Json		JsonObject = Json::parse((const char*)JasonData.Data(), JsonErrorString);

	if (!JsonErrorString.empty())
	{
		return AppError(fmt::format("JSON error while parsing token: {}", JsonErrorString.c_str()));
	}

	return ResultOk(std::move(JsonObject));
}

TResult<FAuthToken>
AcquireAuthToken(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds, const FOpenIdConfig& OpenIdConfig)
{
	if (OpenIdConfig.AuthorizationEndpoint.empty())
	{
		return AppError(L"Authorization endpoint is required");
	}

	if (OpenIdConfig.TokenEndpoint.empty())
	{
		return AppError(L"Token endpoint is required");
	}

	if (AuthDesc.Callback.empty())
	{
		return AppError(L"Callback URI is required");
	}

	TResult<FRemoteDesc> CallbackServerDescResult = FRemoteDesc::FromUrl(AuthDesc.Callback);
	if (CallbackServerDescResult.IsError())
	{
		return AppError(L"Failed to parse callback URI");
	}

	const FRemoteDesc& CallbackServerDesc = CallbackServerDescResult.GetData();

	FAuthToken Result;

	TResult<FRemoteDesc> AuthRemoteDesc = FRemoteDesc::FromUrl(AuthDesc.AuthServer, unsync::EProtocolFlavor::Unknown, RecvTimeoutSeconds);
	if (AuthRemoteDesc.IsError())
	{
		return AppError(L"Failed to parse authentication server URI");
	}

	FHttpConnection AuthServerConnection = FHttpConnection::CreateDefaultHttps(*AuthRemoteDesc);

	const uint16 CallbackPortNumber = CallbackServerDesc.Host.Port;

	FSocketHandle CallbackListenSocket = SocketListenTcp("127.0.0.1", CallbackPortNumber);

	std::string RandomState	  = SecureRandomBytesAsHexString(16);
	std::string NonceStr	  = SecureRandomBytesAsHexString(16);
	std::string CodeVerifier  = SecureRandomBytesAsHexString(64);
	std::string CodeChallenge = GetPKCECodeChallenge(CodeVerifier);
	std::string CallbackUrl	  = AuthDesc.Callback;

	std::string AudienceParam;
	if (!AuthDesc.Audience.empty())
	{
		AudienceParam = fmt::format("audience={}&", AuthDesc.Audience);
	}

	std::string AuthorizeUrl = fmt::format(
		"https://{}{}?"
		"client_id={}&"
		"{}"  // optional audience parameter
		"response_type=code&"
		"scope=offline_access&"
		"code_challenge_method=S256&"
		"code_challenge={}&"
		"state={}&"
		"redirect_uri={}",
		AuthRemoteDesc->Host.Address,
		OpenIdConfig.AuthorizationEndpoint,
		AuthDesc.ClientId,
		AudienceParam,
		CodeChallenge,
		RandomState,
		CallbackUrl);

	FHttpCallbackData HttpCallbackData;

	std::thread ServerThread = StartHttpCallbackServer(CallbackListenSocket, CallbackServerDesc.RequestPath, RandomState, HttpCallbackData);

	UNSYNC_LOG(L"Opening authorization URL in default browser");
	OpenUrlInDefaultBrowser(AuthorizeUrl.c_str());

	UNSYNC_LOG(L"Waiting for HTTP callback on port %d...", int(CallbackPortNumber));

	ServerThread.join();
	SocketClose(CallbackListenSocket);

	if (RandomState != HttpCallbackData.State)
	{
		return AppError(L"Callback state value mismatch");
	}

	if (HttpCallbackData.AuthCode.empty())
	{
		return AppError(L"Did not receive authorization code callback");
	}

	std::string AccessToken;
	std::string RefreshToken;
	std::string IdToken;
	std::string TokenType;
	int64		ExpiresInSeconds = 0;

	// TODO: only try to acquire new token if close to expiry

	// Use authorization code to acquire tokens
	{
		std::string TokenPayload = fmt::format(
			"grant_type=authorization_code&"
			"client_id={}&"
			"code={}&"
			"code_verifier={}&"
			"redirect_uri={}",
			AuthDesc.ClientId,
			HttpCallbackData.AuthCode,
			CodeVerifier,
			CallbackUrl);

		FHttpRequest Request;
		Request.Url				   = OpenIdConfig.TokenEndpoint;
		Request.Method			   = EHttpMethod::POST;
		Request.PayloadContentType = EHttpContentType::Application_WWWFormUrlEncoded;
		Request.Payload			   = FBufferView{(const uint8*)TokenPayload.data(), (uint64)TokenPayload.size()};

		FHttpResponse Response = HttpRequest(AuthServerConnection, Request);

		if (Response.Success())
		{
			using namespace json11;
			std::string JsonString = std::string(Response.AsStringView());

			std::string JsonErrorString;
			Json		JsonObject = Json::parse(JsonString, JsonErrorString);

			if (!JsonErrorString.empty())
			{
				return AppError(fmt::format("JSON error while parsing token: {}", JsonErrorString.c_str()));
			}

			AccessToken		 = JsonObject["access_token"].string_value();
			RefreshToken	 = JsonObject["refresh_token"].string_value();
			IdToken			 = JsonObject["id_token"].string_value();
			TokenType		 = JsonObject["token_type"].string_value();
			ExpiresInSeconds = int64(JsonObject["expires_in"].number_value());

			TResult<json11::Json> DecodedAccessTokenResult = DecodeJwtPayload(AccessToken);
			if (DecodedAccessTokenResult.IsOk())
			{
				const json11::Json& AccessTokenJsonObject = DecodedAccessTokenResult.GetData();
				if (auto& Field = AccessTokenJsonObject["exp"]; Field.is_number())
				{
					Result.ExirationTime = int64(Field.number_value());
				}
			}
			else
			{
				return MoveError<FAuthToken>(DecodedAccessTokenResult);
			}

			Result.Raw = JsonString;
		}
		else
		{
			return HttpError(L"Could not acquire authorization code", Response.Code);
		}
	}

	if (AccessToken.empty())
	{
		return AppError(L"Did not receive new access token");
	}

	Result.Access  = std::move(AccessToken);
	Result.Refresh = std::move(RefreshToken);

	return ResultOk(std::move(Result));
}

TResult<FAuthUserInfo>
GetUserInfo(FHttpConnection& HttpConnection, const FAuthDesc& AuthDesc, const FOpenIdConfig& OpenIdConfig, const FAuthToken& AuthToken)
{
	if (OpenIdConfig.UserInfoEndpoint.empty())
	{
		return AppError(L"User info endpoint is unknown");
	}

	FHttpRequest Request;
	Request.Url			= OpenIdConfig.UserInfoEndpoint;
	Request.Method		= EHttpMethod::GET;
	Request.BearerToken = AuthToken.Access;

	FHttpResponse Response = HttpRequest(HttpConnection, Request);

	if (Response.Success())
	{
		using namespace json11;

		std::string JsonString = std::string(Response.AsStringView());

		std::string JsonErrorString;
		Json		JsonObject = Json::parse(JsonString, JsonErrorString);

		if (!JsonErrorString.empty())
		{
			return AppError(fmt::format("JSON error while parsing user info: {}", JsonErrorString.c_str()));
		}

		FAuthUserInfo Result;

		Result.Sub		  = JsonObject["sub"].string_value();
		Result.Name		  = JsonObject["name"].string_value();
		Result.Nickname	  = JsonObject["nickname"].string_value();
		Result.GivenName  = JsonObject["given_name"].string_value();
		Result.FamilyName = JsonObject["family_name"].string_value();
		Result.Email	  = JsonObject["email"].string_value();

		return ResultOk(Result);
	}
	else
	{
		return HttpError(L"Could not query user info from authorization server", Response.Code);
	}
}

TResult<FAuthToken>
RefreshAuthToken(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds, const FOpenIdConfig& OpenIdConfig, const FAuthToken& PreviousToken)
{
	if (OpenIdConfig.TokenEndpoint.empty())
	{
		return AppError(L"Token endpoint is unknown");
	}

	FAuthToken Result = PreviousToken;

	TResult<FRemoteDesc> AuthRemoteDesc = FRemoteDesc::FromUrl(AuthDesc.AuthServer, unsync::EProtocolFlavor::Unknown, RecvTimeoutSeconds);
	if (AuthRemoteDesc.IsError())
	{
		return AppError(L"Failed to parse authentication server URI");
	}

	FHttpConnection AuthServerConnection = FHttpConnection::CreateDefaultHttps(*AuthRemoteDesc);

	std::string AccessToken;
	std::string RefreshToken;
	std::string IdToken;
	std::string TokenType;
	int64		ExpiresInSeconds = 0;

	// Use refresh token to acquire new tokens
	{
		std::string TokenPayload = fmt::format(
			"grant_type=refresh_token&"
			"client_id={}&"
			"refresh_token={}",
			AuthDesc.ClientId,
			PreviousToken.Refresh);

		FHttpRequest Request;
		Request.Url				   = OpenIdConfig.TokenEndpoint;
		Request.Method			   = EHttpMethod::POST;
		Request.PayloadContentType = EHttpContentType::Application_WWWFormUrlEncoded;
		Request.Payload			   = FBufferView{(const uint8*)TokenPayload.data(), (uint64)TokenPayload.size()};

		FHttpResponse Response = HttpRequest(AuthServerConnection, Request);

		if (Response.Success())
		{
			using namespace json11;
			std::string JsonString = std::string(Response.AsStringView());

			std::string JsonErrorString;
			Json		JsonObject = Json::parse(JsonString, JsonErrorString);

			if (!JsonErrorString.empty())
			{
				return AppError(fmt::format("JSON error while parsing token: {}", JsonErrorString.c_str()));
			}

			AccessToken		 = JsonObject["access_token"].string_value();
			RefreshToken	 = JsonObject["refresh_token"].string_value();
			IdToken			 = JsonObject["id_token"].string_value();
			TokenType		 = JsonObject["token_type"].string_value();
			ExpiresInSeconds = int64(JsonObject["expires_in"].number_value());

			TResult<json11::Json> DecodedAccessTokenResult = DecodeJwtPayload(AccessToken);
			if (DecodedAccessTokenResult.IsOk())
			{
				const json11::Json& AccessTokenJsonObject = DecodedAccessTokenResult.GetData();
				if (auto& Field = AccessTokenJsonObject["exp"]; Field.is_number())
				{
					Result.ExirationTime = int64(Field.number_value());
				}
			}
			else
			{
				return MoveError<FAuthToken>(DecodedAccessTokenResult);
			}

			Result.Raw = JsonString;
		}
		else
		{
			return HttpError(L"Could not acquire authorization code", Response.Code);
		}
	}

	if (AccessToken.empty())
	{
		return AppError(L"Did not receive new access token");
	}

	Result.Access = AccessToken;

	if (!RefreshToken.empty())
	{
		Result.Refresh = RefreshToken;
	}

	return ResultOk(Result);
}

std::string
GenerateTokenId(const FAuthDesc& AuthDesc)
{
	// TODO: just stream fields directly through a hasher
	std::string HashInput;
	HashInput += AuthDesc.AuthServer + " ";
	HashInput += AuthDesc.ClientId + " ";
	HashInput += AuthDesc.Audience;
	//HashInput += AuthDesc.Callback; // don't need to consider the callback url
	FHash128 Hash = HashBlake3String<FHash128>(HashInput);
	return HashToHexString(Hash);
}

// Keeps last loaded token in memory
struct FAuthTokenCache
{
	std::mutex Mutex;

	struct FEntry
	{
		FPath			Path;
		FFileAttributes Attrib;
		FAuthToken		Token;
	};

	// Only keep the most recent token now, but could extend to N recent tokens in the future
	FEntry MostRecent;

	void Add(const FPath& Path, const FFileAttributes& Attrib, const FAuthToken& AuthToken)
	{
		std::lock_guard<std::mutex> LockGuard(Mutex);

		MostRecent.Path	  = Path;
		MostRecent.Attrib = Attrib;
		MostRecent.Token  = AuthToken;
	}

	std::optional<FAuthToken> Get(const FPath& Path, bool bCheckFileAttributes = false)
	{
		std::lock_guard<std::mutex> LockGuard(Mutex);

		if (MostRecent.Path == Path)
		{
			if (bCheckFileAttributes)
			{
				FFileAttributes Attrib = GetFileAttrib(Path);
				if (Attrib.Mtime != MostRecent.Attrib.Mtime || Attrib.Size != MostRecent.Attrib.Size)
				{
					return {};
				}
			}

			return std::optional<FAuthToken>(MostRecent.Token);
		}

		return {};
	}
};

static FAuthTokenCache GAuthTokenCache;

enum class EProtectedBufferFormat : uint64 {
	Invalid				  = 0,
	Win32CryptProtectData = 0x9E9AA2B319A7D98Full,
};

struct FProtectedBufferHeader
{
	EProtectedBufferFormat Format = EProtectedBufferFormat::Invalid;
	uint64				   Size	  = 0;
};

#if UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN
#	if UNSYNC_PLATFORM_WINDOWS
static TResult<FBuffer>
ProtectBuffer(const FBufferView& InPlainTextBuffer)
{
	DATA_BLOB Blob = {};
	Blob.cbData	   = (DWORD)InPlainTextBuffer.Size;
	Blob.pbData	   = (BYTE*)InPlainTextBuffer.Data;

	DATA_BLOB ProtectedBlob = {};

	BOOL ProtectOk = CryptProtectData(&Blob, nullptr, nullptr, nullptr, nullptr, 0, &ProtectedBlob);
	if (!ProtectOk)
	{
		DWORD ErrorCode = GetLastError();
		return SystemError(L"CryptProtectData failed", ErrorCode);
	}

	FBuffer Result;

	FProtectedBufferHeader Header;
	Header.Format = EProtectedBufferFormat::Win32CryptProtectData;
	Header.Size	  = ProtectedBlob.cbData;

	Result.Append(reinterpret_cast<const uint8*>(&Header), sizeof(Header));
	Result.Append(reinterpret_cast<const uint8*>(ProtectedBlob.pbData), ProtectedBlob.cbData);

	LocalFree(ProtectedBlob.pbData);

	return ResultOk(std::move(Result));
}

static TResult<FBuffer>
UnprotectBuffer(const FBufferView& InProtectedBuffer)
{
	FProtectedBufferHeader Header;

	if (InProtectedBuffer.Size < sizeof(Header))
	{
		return AppError(L"Protected buffer does not contain a valid header");
	}

	memcpy(&Header, InProtectedBuffer.Data, sizeof(Header));

	if (Header.Format != EProtectedBufferFormat::Win32CryptProtectData)
	{
		return AppError(L"Protected buffer format is not supported");
	}

	DATA_BLOB ProtectedBlob = {};
	ProtectedBlob.cbData	= (DWORD)Header.Size;
	ProtectedBlob.pbData	= (BYTE*)(InProtectedBuffer.Data + sizeof(Header));

	DATA_BLOB Blob = {};

	BOOL ProtectOk = CryptUnprotectData(&ProtectedBlob, nullptr, nullptr, nullptr, nullptr, 0, &Blob);
	if (!ProtectOk)
	{
		DWORD ErrorCode = GetLastError();
		return SystemError(L"CryptUnprotectData failed", ErrorCode);
	}

	FBuffer Result;
	Result.Append(reinterpret_cast<const uint8*>(Blob.pbData), Blob.cbData);

	LocalFree(Blob.pbData);

	return ResultOk(std::move(Result));
}
#	endif	// UNSYNC_PLATFORM_WINDOWS
#endif		// UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN

bool
SaveAuthToken(const FPath& Path, const FAuthToken& AuthToken)
{
#if UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN
	FBufferView		 AuthTokenView	 = {.Data = (const uint8*)AuthToken.Raw.data(), .Size = AuthToken.Raw.length()};
	TResult<FBuffer> ProtectedBuffer = ProtectBuffer(AuthTokenView);
	if (ProtectedBuffer.IsError())
	{
		LogError(ProtectedBuffer.GetError(), L"Failed to protect authentication token");
		return false;
	}
	const bool bWrittenOk = WriteBufferToFile(Path,
											  ProtectedBuffer.GetData(),
											  EFileMode::CreateWriteOnly | EFileMode::IgnoreDryRun);
#else	// UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN
	const bool bWrittenOk = WriteBufferToFile(Path,
											  (const uint8*)AuthToken.Raw.data(),
											  AuthToken.Raw.length(),
											  EFileMode::CreateWriteOnly | EFileMode::IgnoreDryRun);
#endif	// UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN

	if (!bWrittenOk)
	{
		return false;
	}

	{
		FPath ExtendedPath = MakeExtendedAbsolutePath(Path);
		std::error_code ErrorCode = {};
		std::filesystem::permissions(
				ExtendedPath,
				std::filesystem::perms::owner_write | std::filesystem::perms::owner_read,
				std::filesystem::perm_options::replace,
				ErrorCode);
		if (ErrorCode)
		{
			UNSYNC_ERROR(L"Failed to set cached token file permissions");
			return false;
		}
	}

	FFileAttributes Attrib = GetFileAttrib(Path);
	if (!Attrib.bValid)
	{
		return false;
	}

	GAuthTokenCache.Add(Path, Attrib, AuthToken);

	return true;
}

static bool
IsProtectedBuffer(const FBufferView& BufferView)
{
	EProtectedBufferFormat Magic = EProtectedBufferFormat::Invalid;
	if (BufferView.Size < sizeof(uint64))
	{
		return false;
	}

	memcpy(&Magic, BufferView.Data, sizeof(Magic));
	if (Magic == EProtectedBufferFormat::Win32CryptProtectData)
	{
		return true;
	}

	return false;
}

TResult<FAuthToken>
LoadAuthToken(const FPath& Path)
{
	if (std::optional<FAuthToken> CachedToken = GAuthTokenCache.Get(Path, /*bCheckFileAttributes*/ false))
	{
		return ResultOk(std::move(CachedToken.value()));
	}

	FBuffer FileBuffer = ReadFileToBuffer(Path);

	if (IsProtectedBuffer(FileBuffer))
	{
#if UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN
		TResult<FBuffer> UnprotectResult = UnprotectBuffer(FileBuffer);
		if (UnprotectResult.IsOk())
		{
			FileBuffer = std::move(UnprotectResult.GetData());
		}
		else
		{
			return MoveError<FAuthToken>(UnprotectResult);
		}
#else // UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN
		return AppError(L"Protected token format is not supported");
#endif	// UNSYNC_OBFUSCATE_CACHED_AUTH_TOKEN
	}

	if (FileBuffer.Size())
	{
		using namespace json11;

		FAuthToken AuthToken;
		AuthToken.Raw.append((const char*)FileBuffer.Data(), FileBuffer.Size());

		std::string JsonErrorString;
		Json		JsonObject = Json::parse(AuthToken.Raw, JsonErrorString);

		if (!JsonErrorString.empty())
		{
			return AppError(fmt::format("JSON error while parsing token: {}", JsonErrorString.c_str()));
		}

		AuthToken.Access  = JsonObject["access_token"].string_value();
		AuthToken.Refresh = JsonObject["refresh_token"].string_value();

		TResult<json11::Json> DecodedAccessTokenResult = DecodeJwtPayload(AuthToken.Access);
		if (DecodedAccessTokenResult.IsOk())
		{
			const json11::Json& AccessTokenJsonObject = DecodedAccessTokenResult.GetData();
			if (auto& Field = AccessTokenJsonObject["exp"]; Field.is_number())
			{
				AuthToken.ExirationTime = int64(Field.number_value());
			}
		}
		else
		{
			return MoveError<FAuthToken>(DecodedAccessTokenResult);
		}

		return ResultOk(std::move(AuthToken));
	}
	else
	{
		return AppError(L"Failed to load refresh token from file");
	}
}

TResult<FAuthToken>
RefreshOrAcquireToken(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds, const FOpenIdConfig& OpenIdConfig, const FAuthToken& PreviousToken)
{
	if (!PreviousToken.Refresh.empty())
	{
		UNSYNC_VERBOSE(L"Refreshing access token");
		TResult<FAuthToken> RefreshResult = RefreshAuthToken(AuthDesc, RecvTimeoutSeconds, OpenIdConfig, PreviousToken);
		if (RefreshResult.IsOk())
		{
			return RefreshResult;
		}
	}

	UNSYNC_VERBOSE(L"Requesting new access token");
	return AcquireAuthToken(AuthDesc, RecvTimeoutSeconds, OpenIdConfig);
}

TResult<FPath>
GetTokenCachePath(const FAuthDesc& AuthDesc)
{
	if (!AuthDesc.TokenPath.empty())
	{
		return ResultOk(AuthDesc.TokenPath);
	}

	FPath UserHomePath = GetUserHomeDirectory();
	if (UserHomePath.empty())
	{
		return AppError(L"Could not query user home directory path");
	}

	std::string TokenId = GenerateTokenId(AuthDesc);

	FPath UnsyncSettingsPath = UserHomePath / FPath(".unsync");
	FPath TokenCachePath	 = UnsyncSettingsPath / FPath(TokenId);

	return ResultOk(TokenCachePath);
}

void
LogAuthTokenExpiration(const FAuthToken& AuthToken)
{
	if (AuthToken.ExirationTime != 0)
	{
		int64 CurrentTime	   = GetSecondsFromUnixEpoch();
		int64 ExpiresInSeconds = AuthToken.ExirationTime - CurrentTime;
		if (ExpiresInSeconds > 0)
		{
			UNSYNC_VERBOSE(L"Authentication token will expire in %d sec", int(ExpiresInSeconds));
		}
		else
		{
			UNSYNC_VERBOSE(L"Authentication token has expired");
		}
	}
}

FAuthDesc
FAuthDesc::FromHelloResponse(const ProxyQuery::FHelloResponse& HelloResponse)
{
	FAuthDesc AuthDesc;
	AuthDesc.AuthServer = HelloResponse.AuthServerUri;
	AuthDesc.ClientId	= HelloResponse.AuthClientId;
	AuthDesc.Audience	= HelloResponse.AuthAudience;
	AuthDesc.Callback	= HelloResponse.CallbackUri;

	if (AuthDesc.Callback.empty())
	{
		AuthDesc.Callback = "http://localhost:8080";  // sensible default
	}

	return AuthDesc;
}

TResult<FAuthToken>
Authenticate(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds, int32 RefreshThreshold)
{
	// Authentication must be serialized (only one thread should ever open the browser for interactive login, etc.)
	static std::mutex			AuthMutex;
	std::lock_guard<std::mutex> LockGuard(AuthMutex);

	FAuthToken PreviousToken;

	TResult<FPath> TokenCachePathResult = GetTokenCachePath(AuthDesc);

	if (const FPath* TokenCachePath = TokenCachePathResult.TryData())
	{
		TResult<FAuthToken> LoadResult = LoadAuthToken(*TokenCachePath);
		if (FAuthToken* LoadedToken = LoadResult.TryData())
		{
			PreviousToken = std::move(*LoadedToken);
		}
	}

	static FHash128 LastLoggedTokenHash;
	FHash128		TokenHash = HashBlake3String<FHash128>(PreviousToken.Raw);
	bool			bShouldLog = false;
	if (LastLoggedTokenHash != TokenHash)
	{
		bShouldLog			= true;
		LastLoggedTokenHash = TokenHash;
	}

	if (bShouldLog && !PreviousToken.Raw.empty())
	{
		UNSYNC_VERBOSE(L"Loaded cached authentication token");
	}

	int64 CurrentTime	   = GetSecondsFromUnixEpoch();
	int64 ExpiresInSeconds = PreviousToken.ExirationTime - CurrentTime;
	if (ExpiresInSeconds > RefreshThreshold)
	{
		if (bShouldLog)
		{
			LogAuthTokenExpiration(PreviousToken);
		}

		return ResultOk(PreviousToken);
	}

	TResult<FOpenIdConfig> OpenIdConfigResult = GetOpenIdConfig(AuthDesc, RecvTimeoutSeconds);
	if (OpenIdConfigResult.IsError())
	{
		return MoveError<FAuthToken>(OpenIdConfigResult);
	}

	TResult<FAuthToken> FreshTokenResult = RefreshOrAcquireToken(AuthDesc, RecvTimeoutSeconds, * OpenIdConfigResult, PreviousToken);
	if (FreshTokenResult.IsError())
	{
		return FreshTokenResult;
	}

	if (const FPath* TokenCachePath = TokenCachePathResult.TryData())
	{
		CreateDirectories(TokenCachePath->parent_path());

		bool bSaved = SaveAuthToken(*TokenCachePath, FreshTokenResult.GetData());

		if (bSaved)
		{
			UNSYNC_VERBOSE2(L"Saved authentication token to file: %ls", TokenCachePath->wstring().c_str());
		}
	}

	if (FreshTokenResult.IsOk())
	{
		LogAuthTokenExpiration(FreshTokenResult.GetData());
	}

	return FreshTokenResult;
}

TResult<FAuthDesc>
GetRemoteAuthDesc(const FRemoteDesc& RemoteDesc)
{
	TResult<ProxyQuery::FHelloResponse> HelloResponseResult = ProxyQuery::Hello(RemoteDesc, nullptr /*AuthDesc: null for anonymous initial connection*/);
	if (HelloResponseResult.IsError())
	{
		UNSYNC_ERROR("Failed establish a handshake with server '%hs'", RemoteDesc.Host.Address.c_str());
		LogError(HelloResponseResult.GetError());
		return MoveError<FAuthDesc>(HelloResponseResult);
	}

	const FAuthDesc AuthDesc = FAuthDesc::FromHelloResponse(*HelloResponseResult);

	return ResultOk(AuthDesc);
}

TResult<FOpenIdConfig>
GetOpenIdConfig(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds)
{
	if (!AuthDesc.IsValid())
	{
		return AppError(L"Mandatory authentication parameters not provided");
	}

	TResult<FRemoteDesc> AuthServerDescResult = FRemoteDesc::FromUrl(AuthDesc.AuthServer, unsync::EProtocolFlavor::Unknown, RecvTimeoutSeconds);
	if (AuthServerDescResult.IsError())
	{
		return MoveError<FOpenIdConfig>(AuthServerDescResult);
	}

	const FRemoteDesc& AuthServerDesc = AuthServerDescResult.GetData();

	std::string ServerApiPrefix;
	if (!AuthServerDesc.RequestPath.empty())
	{
		ServerApiPrefix = fmt::format("/{}", AuthServerDesc.RequestPath);
	}

	FHttpConnection AuthServerConnection = FHttpConnection::CreateDefaultHttps(*AuthServerDescResult);
	std::string		ConfigEndpoint		 = fmt::format("{}/.well-known/openid-configuration", ServerApiPrefix);

	FOpenIdConfig OpenIdConfig;

	FHttpResponse ConfigResponse = HttpRequest(AuthServerConnection, EHttpMethod::GET, ConfigEndpoint);
	if (ConfigResponse.Success())
	{
		using namespace json11;
		std::string JsonString = std::string(ConfigResponse.AsStringView());

		std::string JsonErrorString;
		Json		JsonObject = Json::parse(JsonString, JsonErrorString);

		std::string EndpointPrefix = fmt::format("https://{}", AuthServerDescResult->Host.Address);

		if (JsonErrorString.empty())
		{
			auto ExtractEndpoint = [&EndpointPrefix, &JsonObject](const char* FieldName) -> std::string {
				if (auto& Field = JsonObject[FieldName]; Field.is_string())
				{
					const std::string& Value = Field.string_value();
					if (Value.starts_with(EndpointPrefix))
					{
						return Value.substr(EndpointPrefix.length());
					}
				}
				return {};
			};

			OpenIdConfig.AuthorizationEndpoint = ExtractEndpoint("authorization_endpoint");
			OpenIdConfig.TokenEndpoint		   = ExtractEndpoint("token_endpoint");
			OpenIdConfig.UserInfoEndpoint	   = ExtractEndpoint("userinfo_endpoint");
			OpenIdConfig.JwksUri			   = JsonObject["jwks_uri"].string_value();
		}
	}

	return ResultOk(OpenIdConfig);
}

int64
GetSecondsFromUnixEpoch()
{
	return int64(std::time(nullptr));
}

}  // namespace unsync
