// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncBuffer.h"
#include "UnsyncCommon.h"
#include "UnsyncError.h"
#include "UnsyncHash.h"

#include <string>

namespace unsync {

struct FHttpConnection;
struct FRemoteDesc;

namespace ProxyQuery {
	struct FHelloResponse;
}

struct FAuthToken
{
	// Raw data acquired from authentication endpoint
	std::string Raw;

	// Parsed data for convenience
	std::string Access;
	std::string Refresh;
	int64		ExirationTime = 0; // UNIX timestamp in seconds (0 if unknown)
};

// Data from /.well-known/openid-configuration
struct FOpenIdConfig
{
	std::string AuthorizationEndpoint;
	std::string TokenEndpoint;
	std::string UserInfoEndpoint;
	std::string JwksUri;
};

// Data from server configuration
struct FAuthDesc
{
	// Mandatory configuration
	std::string AuthServer;
	std::string ClientId;
	
	// Optional configuration
	std::string Audience;
	std::string Callback;	// OAuth flow redirection URL

	FPath TokenPath; // Optional explicit storage location for the auth token

	// Check that mandatory fields have values
	bool IsValid() const
	{
		return !AuthServer.empty() && !ClientId.empty();
	}

	static FAuthDesc FromHelloResponse(const ProxyQuery::FHelloResponse& HelloResponse);
};

// Query OIDC/OAuth2 configuration via HTTP
TResult<FOpenIdConfig> GetOpenIdConfig(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds);

// Perform the complete authentication flow:
// - Attempt use stored refresh token first
// - If refresh is not possible, use PKCE Authentication flow to get new tokens
// - Save refresh token in user directory for future use
// - Skips acquiring new token if remaining valid time is above RefreshThreshold (in seconds, 5 minutes by default)
TResult<FAuthToken> Authenticate(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds, int32 RefreshThreshold = 300);

// Auth utility functions

TResult<FAuthToken> AcquireAuthToken(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds, const FOpenIdConfig& OpenIdConfig);
TResult<FAuthToken> RefreshAuthToken(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds, const FOpenIdConfig& OpenIdConfig, const FAuthToken& PreviousToken);
TResult<FAuthToken> RefreshOrAcquireToken(const FAuthDesc& AuthDesc, uint32 RecvTimeoutSeconds, const FOpenIdConfig& OpenIdConfig, const FAuthToken& PreviousToken);

struct FAuthUserInfo
{
	std::string Sub;
	std::string Name;
	std::string Nickname;
	std::string GivenName;
	std::string FamilyName;
	std::string Email;
};
TResult<FAuthUserInfo> GetUserInfo(FHttpConnection& HttpConnection, const FAuthDesc& AuthDesc, const FOpenIdConfig& OpenIdConfig, const FAuthToken& AuthToken);

std::string GenerateTokenId(const FAuthDesc& RemoteDesc);

bool				SaveAuthToken(const FPath& Path, const FAuthToken& AuthToken);
TResult<FAuthToken> LoadAuthToken(const FPath& Path);

std::string SecureRandomBytesAsHexString(uint32 NumBytes);
FHash256	HashSha256Bytes(const uint8* Data, uint64 Size);
std::string EncodeBase64(const uint8* Data, uint64 Size);
bool		DecodeBase64(std::string_view Base64Data, FBuffer& Output);

void TransformBase64VanillaToUrlSafe(std::string& Base64Vanilla);
void TransformBase64UrlSafeToVanilla(std::string& Base64UrlSafe);

std::string GetPKCECodeChallenge(std::string_view CodeVerifier);

TResult<FAuthDesc> GetRemoteAuthDesc(const FRemoteDesc& RemoteDesc);

int64 GetSecondsFromUnixEpoch();

}  // namespace unsync
