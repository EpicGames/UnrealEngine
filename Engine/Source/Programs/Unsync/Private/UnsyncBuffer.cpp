// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncBuffer.h"
#include "UnsyncMemory.h"

namespace unsync {

FBuffer::FBuffer(uint64 InitialSize)
{
	Resize(InitialSize);
}

FBuffer::FBuffer(uint64 InitialSize, uint8 Value)
{
	Resize(InitialSize);
	memset(Buffer, Value, InitialSize);
}

FBuffer::FBuffer(std::initializer_list<uint8> Values)
{
	if (Values.size())
	{
		Resize(Values.size());
		memcpy(Buffer, Values.begin(), Values.size());
	}
}

FBuffer::FBuffer(FBuffer&& Rhs)
{
	std::swap(Buffer, Rhs.Buffer);
	std::swap(BufferSize, Rhs.BufferSize);
	std::swap(BufferCapacity, Rhs.BufferCapacity);
}

FBuffer&
FBuffer::operator=(FBuffer&& Rhs)
{
	if (&Rhs != this)
	{
		std::swap(Buffer, Rhs.Buffer);
		std::swap(BufferSize, Rhs.BufferSize);
		std::swap(BufferCapacity, Rhs.BufferCapacity);

		Rhs.Clear();
	}
	return *this;
}

FBuffer::~FBuffer()
{
	UnsyncFree(Buffer);
	UNSYNC_CLOBBER(Buffer);
}

void
FBuffer::Reserve(uint64 RequiredCapacity)
{
	uint64 AlignedRequiredCapacity = std::max<uint64>(16, RequiredCapacity);
	EnsureCapacity(AlignedRequiredCapacity);
}

void
FBuffer::Append(const FBuffer& Other)
{
	Append(Other.Data(), Other.Size());
}

void
FBuffer::Append(const FBufferView& Other)
{
	Append(Other.Data, Other.Size);
}

void
FBuffer::Append(const uint8* AppendData, uint64 AppendSize)
{
	uint64 RequiredCapacity		   = BufferSize + AppendSize;
	uint64 AlignedRequiredCapacity = std::max<uint64>(16, BufferCapacity);
	while (AlignedRequiredCapacity < RequiredCapacity)
	{
		AlignedRequiredCapacity *= 2;
	}

	EnsureCapacity(AlignedRequiredCapacity);

	memcpy(Buffer + BufferSize, AppendData, AppendSize);
	BufferSize += AppendSize;
}

void
FBuffer::Shrink()
{
	if (BufferSize < BufferCapacity)
	{
		uint8* NewBuffer = nullptr;

		if (BufferSize)
		{
			NewBuffer = (uint8*)UnsyncMalloc(BufferSize);
			UNSYNC_ASSERT(NewBuffer);
			memcpy(NewBuffer, Buffer, BufferSize);
		}

		UnsyncFree(Buffer);

		Buffer		   = NewBuffer;
		BufferCapacity = BufferSize;
	}
}

void
FBuffer::Resize(uint64 NewSize)
{
	if (NewSize > BufferSize)
	{
		EnsureCapacity(NewSize);
	}
	BufferSize = NewSize;
}

void
FBuffer::EnsureCapacity(uint64 RequiredCapacity)
{
	if (RequiredCapacity > BufferCapacity)
	{
		uint8* NewBuffer = (uint8*)UnsyncMalloc(RequiredCapacity);
		UNSYNC_ASSERT(NewBuffer);
		if (BufferSize)
		{
			memcpy(NewBuffer, Buffer, BufferSize);
			UnsyncFree(Buffer);
		}
		Buffer		   = NewBuffer;
		BufferCapacity = RequiredCapacity;
	}
}

FBufferPool::FBufferPool(uint64 InMaxBufferSize) : MaxBufferSize(InMaxBufferSize)
{
}

FBufferPool::~FBufferPool()
{
	std::lock_guard<std::mutex> LockGuard(Mutex);

	UNSYNC_ASSERT(AllocatedBuffers == 0);
	for (FBuffer* It : Pool)
	{
		delete It;
	}
}

FBuffer*
FBufferPool::Acquire()
{
	std::lock_guard<std::mutex> LockGuard(Mutex);

	if (Pool.empty())
	{
		Pool.push_back(new FBuffer(MaxBufferSize));
	}

	FBuffer* TResult = Pool.back();
	Pool.pop_back();

	++AllocatedBuffers;

	return TResult;
}

void
FBufferPool::Release(FBuffer* Buffer)
{
	std::lock_guard<std::mutex> LockGuard(Mutex);

	UNSYNC_ASSERT(Buffer->Size() == MaxBufferSize);
	Pool.push_back(Buffer);

	--AllocatedBuffers;
}

}  // namespace unsync
