// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <mutex>
#include "UnsyncLog.h"
#include "UnsyncUtil.h"

namespace unsync {

struct FBufferView
{
	const uint8* Data = nullptr;
	uint64		 Size = 0;
};

struct FMutBufferView
{
	uint8* Data = nullptr;
	uint64 Size = 0;

	operator FBufferView() const
	{ 
		FBufferView Result;
		Result.Data = Data;
		Result.Size = Size;
		return Result;
	}
};

class FBuffer
{
public:
	FBuffer() = default;
	explicit FBuffer(uint64 InitialSize);
	FBuffer(uint64 InitialSize, uint8 Value);
	FBuffer(std::initializer_list<uint8> Values);
	FBuffer(const FBuffer&) = delete;
	FBuffer& operator=(const FBuffer&) = delete;
	FBuffer(FBuffer&& Rhs);
	FBuffer& operator=(FBuffer&& Rhs);
	~FBuffer();

	void Reserve(uint64 RequiredCapacity);

	uint64 Capacity() const { return BufferCapacity; }
	uint64 Size() const { return BufferSize; }
	bool   Empty() const { return Size() == 0; }

	uint8*		 Data() { return Buffer; }
	const uint8* Data() const { return Buffer; }

	uint8*		 begin() { return Buffer; }					  // NOLINT
	uint8*		 end() { return Buffer + BufferSize; }		  // NOLINT
	const uint8* begin() const { return Buffer; }			  // NOLINT
	const uint8* end() const { return Buffer + BufferSize; }  // NOLINT

	void Append(const uint8* AppendData, uint64 AppendSize);
	void Append(const FBuffer& Other);
	void Append(const FBufferView& Other);
	void PushBack(uint8 Value) { Append(&Value, 1); }
	void Resize(uint64 NewSize);
	void Clear() { Resize(0); }
	void Shrink();

	uint8& operator[](size_t I)
	{
		UNSYNC_ASSERTF(I < BufferSize, L"Buffer index is out of bounds. Index=%llu, BufferSize=%llu.", llu(I), llu(BufferSize));
		return Buffer[I];
	}

	const uint8& operator[](size_t I) const
	{
		UNSYNC_ASSERTF(I < BufferSize, L"Buffer index is out of bounds. Index=%llu, BufferSize=%llu", llu(I), llu(BufferSize));
		return Buffer[I];
	}

	FBufferView	   View() const { return FBufferView{.Data = Data(), .Size = Size()}; }
	FMutBufferView MutView() { return FMutBufferView{.Data = Data(), .Size = Size()}; }

	FBufferView View(uint64 ViewOffset, uint64 ViewSize) const
	{
		UNSYNC_ASSERT(ViewOffset + ViewSize <= Size());
		return FBufferView{.Data = Data() + ViewOffset, .Size = ViewSize};
	}

	FMutBufferView MutView(uint64 ViewOffset, uint64 ViewSize)
	{
		UNSYNC_ASSERT(ViewOffset + ViewSize <= Size());
		return FMutBufferView{.Data = Data() + ViewOffset, .Size = ViewSize};
	}

	operator FBufferView() const { return View(); }
	operator FMutBufferView() { return MutView(); }

private:
	void EnsureCapacity(uint64 RequiredCapacity);

	uint8* Buffer		  = nullptr;
	uint64 BufferSize	  = 0;
	uint64 BufferCapacity = 0;
};

template<typename T>
TArrayView<T>
ReinterpretView(FBufferView BufferView)
{
	const uint64 Count = BufferView.Size / sizeof(T);
	return MakeView(reinterpret_cast<const T*>(BufferView.Data), Count);
}

struct FBufferPool
{
	UNSYNC_DISALLOW_COPY_ASSIGN(FBufferPool);

	FBufferPool(uint64 InMaxBufferSize);
	~FBufferPool();

	FBuffer* Acquire();
	void	 Release(FBuffer* Buffer);

	uint64				  MaxBufferSize;
	uint64				  AllocatedBuffers = 0;
	std::mutex			  Mutex;
	std::vector<FBuffer*> Pool;
};

}  // namespace unsync
