// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncChunking.h"
#include "UnsyncFile.h"
#include "UnsyncProtocol.h"
#include "UnsyncScan.h"
#include "UnsyncScheduler.h"

UNSYNC_THIRD_PARTY_INCLUDES_START
#include <md5-sse2.h>
UNSYNC_THIRD_PARTY_INCLUDES_END

namespace unsync {

inline uint32
ComputeMinVariableBlockSize(uint32 BlockSize)
{
	return std::min(MAX_BLOCK_SIZE, std::max(BlockSize, 4096u) / 2);  // changing this invalidates cached blocks
}

inline uint32
ComputeMaxVariableBlockSize(uint32 BlockSize)
{
	return std::min(MAX_BLOCK_SIZE, std::max(BlockSize, 4096u) * 4);  // changing this invalidates cached blocks
}

inline uint32
ComputeWindowHashThreshold(uint32 TargetSize)
{
	const uint32 MinSize = ComputeMinVariableBlockSize(TargetSize);
	UNSYNC_ASSERT(TargetSize > MinSize);
	return uint32((1ull << 32) / (TargetSize-MinSize));
}

void UNSYNC_ATTRIB_NOINLINE
MemcpyNoInline(void* Dst, const void* Src, size_t Size)
{
	memcpy(Dst, Src, Size);
}

template<typename WeakHasherT>
FComputeBlocksResult
ComputeBlocksVariableStreamingT(FIOReader& Reader, const FComputeBlocksParams& Params)
{
	const uint32 BlockSize = Params.BlockSize;

	const uint64 InputSize = Reader.GetSize();

	const uint32 MinimumBlockSize = ComputeMinVariableBlockSize(BlockSize);
	const uint32 MaximumBlockSize = ComputeMaxVariableBlockSize(BlockSize);

	const uint64 BytesPerTask = std::min<uint64>(256_MB, std::max<uint64>(InputSize, 1ull));  // TODO: handle task boundary overlap
	const uint64 NumTasks	  = DivUp(InputSize, BytesPerTask);

	const uint64 TargetMacroBlockSize	 = Params.bNeedMacroBlocks ? Params.MacroBlockTargetSize : 0;
	const uint64 MinimumMacroBlockSize	 = std::max<uint64>(MinimumBlockSize, TargetMacroBlockSize / 8);
	const uint64 MaximumMacroBlockSize	 = Params.bNeedMacroBlocks ? Params.MacroBlockMaxSize : 0;
	const uint32 BlocksPerMacroBlock	 = CheckedNarrow(DivUp(TargetMacroBlockSize - MinimumMacroBlockSize, BlockSize));
	const uint32 MacroBlockHashThreshold = BlocksPerMacroBlock ? (0xFFFFFFFF / BlocksPerMacroBlock) : 0;

	FBlockSourceInfo SourceInfo;
	SourceInfo.TotalSize = InputSize;

	struct FHotState
	{
		uint8*		BlockWindowBegin = nullptr;	 // oldest element
		uint8*		BlockWindowEnd	 = nullptr;	 // newest element
		WeakHasherT WeakHasher;
	};

	struct FTask
	{
		uint64			   Offset = 0;
		FGenericBlockArray Blocks;
		FGenericBlockArray MacroBlocks;

		FBuffer BlockBuffer;

		uint64	  NumBlockBytes = 0;

		FHotState HotState;
	};

	std::vector<FTask> Tasks;
	Tasks.resize(NumTasks);

	FTaskGroup			TaskGroup = GScheduler->CreateTaskGroup();

	for (uint64 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
	{
		const uint64 ThisTaskOffset = BytesPerTask * TaskIndex;
		const uint64 ThisTaskSize	= CalcChunkSize(TaskIndex, BytesPerTask, InputSize);

		{
			FTask& Task = Tasks[TaskIndex];
			Task.Offset = ThisTaskOffset;
			Task.BlockBuffer.Resize(MaximumBlockSize);

			Task.HotState.BlockWindowBegin = Task.BlockBuffer.Data();
			Task.HotState.BlockWindowEnd   = Task.BlockBuffer.Data();
		}

		const uint32 ChunkWindowHashThreshold = ComputeWindowHashThreshold(Params.BlockSize);

		auto ScanTask = [&Tasks,
						 &Params,
						 &SourceInfo,
						 &Reader,
						 MinimumBlockSize,
						 MaximumBlockSize,
						 TaskIndex,
						 ThisTaskSize,
						 ChunkWindowHashThreshold,
						 TargetMacroBlockSize,
						 MinimumMacroBlockSize,
						 MaximumMacroBlockSize,
						 MacroBlockHashThreshold]()
		{
			FTask& Task = Tasks[TaskIndex];

			const uint64 ReadBatchSize = std::max<uint64>(1_MB, MaximumBlockSize);
			const uint64 NumReadBatches = DivUp(ThisTaskSize, ReadBatchSize);

			std::unique_ptr<FAsyncReader> AsyncReader = Reader.CreateAsyncReader(8);

			FBlake3Hasher MacroBlockHasher;

			FGenericBlock CurrentMacroBlock;
			CurrentMacroBlock.HashStrong.Type = MACRO_BLOCK_HASH_TYPE;
			CurrentMacroBlock.Offset		  = Task.Offset;

			IOCallback ReadCallback = [&Task,
									   &CurrentMacroBlock,
									   &Params,
									   &SourceInfo,
									   &MacroBlockHasher,
									   TargetMacroBlockSize,
									   MinimumMacroBlockSize,
									   MaximumMacroBlockSize,
									   MacroBlockHashThreshold,
									   MinimumBlockSize,
									   MaximumBlockSize,
									   ChunkWindowHashThreshold,
									   NumReadBatches](FIOBuffer Buffer, uint64 SourceOffset, uint64 ReadSize, uint64 UserData) 
			{
				uint8* BlockBufferData = Task.BlockBuffer.Data();

				const uint8* DataBegin = Buffer.GetData();
				const uint8* DataEnd   = DataBegin + ReadSize;

				const uint8* Cursor = DataBegin;

				const bool bLastBatch = UserData + 1 == NumReadBatches;

				FHotState State = Task.HotState;

				while (Cursor != DataEnd)
				{
					while (State.WeakHasher.Count < MinimumBlockSize && Cursor != DataEnd)
					{
						State.WeakHasher.Add(*Cursor);

						(*State.BlockWindowEnd++) = *Cursor;

						++Cursor;
					}

					const uint32 WindowHash = State.WeakHasher.Get();

					const bool	 bLastBlock	   = Cursor == DataEnd && bLastBatch;
					const uint64 ThisBlockSize = uint64(State.BlockWindowEnd - BlockBufferData);

					if (ThisBlockSize >= MaximumBlockSize || WindowHash < ChunkWindowHashThreshold || bLastBlock)
					{
						FGenericBlock Block;
						Block.Offset	 = Task.Offset + Task.NumBlockBytes;
						Block.Size		 = CheckedNarrow(ThisBlockSize);
						Block.HashWeak	 = WindowHash;
						Block.HashStrong = ComputeHash(BlockBufferData, ThisBlockSize, Params.Algorithm.StrongHashAlgorithmId);

						if (TargetMacroBlockSize)
						{
							MacroBlockHasher.Update(BlockBufferData, ThisBlockSize);
							CurrentMacroBlock.Size += Block.Size;

							uint32 HashStrong32 = 0;
							memcpy(&HashStrong32, Block.HashStrong.Data, 4);

							if ((CurrentMacroBlock.Size >= MinimumMacroBlockSize && HashStrong32 < MacroBlockHashThreshold) ||
								(CurrentMacroBlock.Size + Block.Size > MaximumMacroBlockSize) || bLastBlock)
							{
								// Commit the macro block
								const FHash256 MacroBlockHash	  = MacroBlockHasher.Finalize();
								CurrentMacroBlock.HashStrong	  = FGenericHash::FromBlake3_256(MacroBlockHash);
								CurrentMacroBlock.HashStrong.Type = MACRO_BLOCK_HASH_TYPE;

								Task.MacroBlocks.push_back(CurrentMacroBlock);

								// Reset macro block state
								MacroBlockHasher.Reset();
								CurrentMacroBlock.Offset += CurrentMacroBlock.Size;
								CurrentMacroBlock.Size = 0;
							}
						}

						if (Params.OnBlockGenerated)
						{
							FBufferView BlockView;
							BlockView.Data = BlockBufferData;
							BlockView.Size = Block.Size;
							Params.OnBlockGenerated(Block, SourceInfo, BlockView);
						}

						if (!Task.Blocks.empty())
						{
							UNSYNC_ASSERT(Task.Blocks.back().Offset + Task.Blocks.back().Size == Block.Offset);
						}

						Task.NumBlockBytes += Block.Size;

						Task.Blocks.push_back(Block);

						State.WeakHasher.Reset();

						State.BlockWindowBegin = BlockBufferData;
						State.BlockWindowEnd   = BlockBufferData;

						continue;
					}

					State.WeakHasher.Sub(*(State.BlockWindowBegin++));
				}

				//memcpy(&Task.HotState, &HotState, sizeof(HotState));
				MemcpyNoInline(&Task.HotState, &State, sizeof(State)); // NOTE: no-inline memcpy generates MUCH faster chunking loop code

			};

			for (uint64 BatchIndex = 0; BatchIndex < NumReadBatches; ++BatchIndex)
			{
				const uint64 ThisReadSize = CalcChunkSize(BatchIndex, ReadBatchSize, ThisTaskSize);
				AsyncReader->EnqueueRead(Task.Offset + BatchIndex * ReadBatchSize, ThisReadSize, BatchIndex, ReadCallback);
			}

		};

		TaskGroup.run(ScanTask);
	}

	TaskGroup.wait();

	// Merge blocks for all the tasks

	FComputeBlocksResult Result;

	for (uint64 I = 0; I < NumTasks; ++I)
	{
		const FTask& Task = Tasks[I];
		for (uint64 J = 0; J < Task.Blocks.size(); ++J)
		{
			Result.Blocks.push_back(Task.Blocks[J]);
		}
	}

	if (Params.bNeedMacroBlocks)
	{
		for (uint64 I = 0; I < NumTasks; ++I)
		{
			const FTask& Task = Tasks[I];
			for (uint64 J = 0; J < Task.MacroBlocks.size(); ++J)
			{
				Result.MacroBlocks.push_back(Task.MacroBlocks[J]);
			}
		}
	}

	uint64 UniqueBlockTotalSize = 0;
	uint64 UniqueBlockMinSize	= ~0ull;
	uint64 UniqueBlockMaxSize	= 0ull;

	uint64 NumTinyBlocks   = 0;
	uint64 NumSmallBlocks  = 0;
	uint64 NumMediumBlocks = 0;
	uint64 NumLargeBlocks  = 0;

	uint64 NumTotalBlocks = 0;

	THashSet<FGenericHash> UniqueBlockSet;
	FGenericBlockArray	   UniqueBlocks;
	for (const FGenericBlock& It : Result.Blocks)
	{
		auto InsertResult = UniqueBlockSet.insert(It.HashStrong);
		if (InsertResult.second)
		{
			if (It.Offset + It.Size < InputSize || Result.Blocks.size() == 1)
			{
				UniqueBlockMinSize = std::min<uint64>(UniqueBlockMinSize, It.Size);
			}

			UniqueBlockMaxSize = std::max<uint64>(UniqueBlockMaxSize, It.Size);
			UniqueBlockTotalSize += It.Size;
			UniqueBlocks.push_back(It);
		}

		if (It.Size < MaximumBlockSize / 8)
		{
			NumTinyBlocks++;
		}
		else if (It.Size <= MaximumBlockSize / 4)
		{
			NumSmallBlocks++;
		}
		else if (It.Size <= MaximumBlockSize / 2)
		{
			NumMediumBlocks++;
		}
		else
		{
			NumLargeBlocks++;
		}

		++NumTotalBlocks;
	}

	double AverageBlockSize = InputSize ? double(UniqueBlockTotalSize / UniqueBlocks.size()) : 0;

	UNSYNC_VERBOSE2(
		L"Blocks (tiny/small/medium/large): %llu / %llu / %llu / %llu, average unique size: %llu bytes, unique count: %llu, total count: "
		L"%llu",
		NumTinyBlocks,
		NumSmallBlocks,
		NumMediumBlocks,
		NumLargeBlocks,
		(uint64)AverageBlockSize,
		(uint64)UniqueBlocks.size(),
		NumTotalBlocks);

	UNSYNC_ASSERT(NumTotalBlocks == Result.Blocks.size());

	return Result;
}

template<typename WeakHasherT>
FComputeBlocksResult
ComputeBlocksVariableT(FIOReader& Reader, const FComputeBlocksParams& Params)
{
	const uint32 BlockSize = Params.BlockSize;

	const uint64 InputSize = Reader.GetSize();

	const uint32 MinimumBlockSize = ComputeMinVariableBlockSize(BlockSize);
	const uint32 MaximumBlockSize = ComputeMaxVariableBlockSize(BlockSize);

	const uint64 BytesPerTask = std::min<uint64>(256_MB, std::max<uint64>(InputSize, 1ull));  // TODO: handle task boundary overlap
	const uint64 NumTasks	  = DivUp(InputSize, BytesPerTask);

	const uint64 TargetMacroBlockSize	 = Params.bNeedMacroBlocks ? Params.MacroBlockTargetSize : 0;
	const uint64 MinimumMacroBlockSize	 = std::max<uint64>(MinimumBlockSize, TargetMacroBlockSize / 8);
	const uint64 MaximumMacroBlockSize	 = Params.bNeedMacroBlocks ? Params.MacroBlockMaxSize : 0;
	const uint32 BlocksPerMacroBlock	 = CheckedNarrow(DivUp(TargetMacroBlockSize - MinimumMacroBlockSize, BlockSize));
	const uint32 MacroBlockHashThreshold = BlocksPerMacroBlock ? (0xFFFFFFFF / BlocksPerMacroBlock) : 0;

	FBlockSourceInfo SourceInfo;
	SourceInfo.TotalSize = InputSize;

	struct FTask
	{
		FTask(EStrongHashAlgorithmID Algorithm)
			: StrongHasher(Algorithm) 
		{
		}

		uint64			   Offset = 0;
		FGenericBlockArray Blocks;
		FGenericBlockArray MacroBlocks;

		FStrongHasher StrongHasher;
		FBlake3Hasher MacroHasher;
	};

	std::vector<FTask> Tasks;
	Tasks.reserve(NumTasks);
	for (uint64 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
	{
		Tasks.emplace_back(Params.Algorithm.StrongHashAlgorithmId);
	}

	FSchedulerSemaphore IoSemaphore(*GScheduler, 16);
	FTaskGroup			TaskGroup = GScheduler->CreateTaskGroup();

	FBufferPool BufferPool(BytesPerTask);

	for (uint64 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
	{
		IoSemaphore.Acquire();	// TODO: remove

		const uint64 ThisTaskOffset = BytesPerTask * TaskIndex;
		const uint64 ThisTaskSize	= CalcChunkSize(TaskIndex, BytesPerTask, InputSize);

		Tasks[TaskIndex].Offset = ThisTaskOffset;

		FBuffer* ScanTaskBuffer = BufferPool.Acquire();
		UNSYNC_ASSERT(ScanTaskBuffer->Size() >= ThisTaskSize);

		uint64 ReadBytesForTask = BlockingReadLarge(Reader, ThisTaskOffset, ThisTaskSize, ScanTaskBuffer->Data(), ThisTaskSize);

		if (ReadBytesForTask != ThisTaskSize)
		{
			UNSYNC_FATAL(L"Expected to read %lld bytes from input, but %lld was actually read.", ThisTaskSize, ReadBytesForTask);
		}

		const uint32 ChunkWindowHashThreshold = ComputeWindowHashThreshold(Params.BlockSize);

		auto ScanTask = [&Tasks,
						 &IoSemaphore,
						 &BufferPool,
						 &Params,
						 &SourceInfo,
						 MinimumBlockSize,
						 MaximumBlockSize,
						 ScanTaskBuffer,
						 TaskIndex,
						 ThisTaskSize,
						 ChunkWindowHashThreshold,
						 TargetMacroBlockSize,
						 MinimumMacroBlockSize,
						 MaximumMacroBlockSize,
						 MacroBlockHashThreshold]()
		{
			FTask& Task = Tasks[TaskIndex];

			const uint8* DataBegin = ScanTaskBuffer->Data();
			const uint8* DataEnd   = DataBegin + ThisTaskSize;

			FGenericBlock CurrentMacroBlock;
			CurrentMacroBlock.HashStrong.Type = MACRO_BLOCK_HASH_TYPE;
			CurrentMacroBlock.Offset		  = Task.Offset;

			const uint8* LastBlockEnd = DataBegin;
			auto ScanFn = [MacroBlockHashThreshold,
						   MaximumMacroBlockSize,
						   MaximumBlockSize,
						   MinimumMacroBlockSize,
						   ChunkWindowHashThreshold,
						   DataEnd,
						   &LastBlockEnd,
						   &Task,
						   DataBegin,
						   &Params,
						   TargetMacroBlockSize,
						   &CurrentMacroBlock,
						   &SourceInfo](const uint8* WindowBegin, const uint8* WindowEnd, uint32 WindowHash) UNSYNC_ATTRIB_FORCEINLINE
			{
				const bool	 bLastBlock	   = WindowEnd == DataEnd;
				const uint64 ThisBlockSize = uint64(WindowEnd - LastBlockEnd);

				if (ThisBlockSize >= MaximumBlockSize || WindowHash < ChunkWindowHashThreshold || bLastBlock)
				{
					FGenericBlock Block;
					Block.Offset	 = Task.Offset + uint64(LastBlockEnd - DataBegin);
					Block.Size		 = CheckedNarrow(ThisBlockSize);
					Block.HashWeak	 = WindowHash;
					Block.HashStrong = ComputeHash(LastBlockEnd, ThisBlockSize, Params.Algorithm.StrongHashAlgorithmId);

					if (TargetMacroBlockSize)
					{
						Task.MacroHasher.Update(LastBlockEnd, ThisBlockSize);
						CurrentMacroBlock.Size += Block.Size;

						uint32 HashStrong32 = 0;
						memcpy(&HashStrong32, Block.HashStrong.Data, 4);

						if ((CurrentMacroBlock.Size >= MinimumMacroBlockSize && HashStrong32 < MacroBlockHashThreshold) ||
							(CurrentMacroBlock.Size + Block.Size > MaximumMacroBlockSize) || bLastBlock)
						{
							// Commit the macro block
							const FHash256 MacroBlockHash	  = Task.MacroHasher.Finalize();
							CurrentMacroBlock.HashStrong	  = FGenericHash::FromBlake3_256(MacroBlockHash);
							CurrentMacroBlock.HashStrong.Type = MACRO_BLOCK_HASH_TYPE;

							Task.MacroBlocks.push_back(CurrentMacroBlock);

							// Reset macro block state
							Task.MacroHasher.Reset();
							CurrentMacroBlock.Offset += CurrentMacroBlock.Size;
							CurrentMacroBlock.Size = 0;
						}
					}

					if (Params.OnBlockGenerated)
					{
						FBufferView BlockView;
						BlockView.Data = LastBlockEnd;
						BlockView.Size = Block.Size;
						Params.OnBlockGenerated(Block, SourceInfo, BlockView);
					}

					if (!Task.Blocks.empty())
					{
						UNSYNC_ASSERT(Task.Blocks.back().Offset + Task.Blocks.back().Size == Block.Offset);
					}

					Task.Blocks.push_back(Block);

					LastBlockEnd = WindowEnd;

					return true;
				}
				else
				{
					return false;
				}
			};

			HashScan<WeakHasherT>(DataBegin, ThisTaskSize, MinimumBlockSize, ScanFn);

			BufferPool.Release(ScanTaskBuffer);
			IoSemaphore.Release();
		};

		TaskGroup.run(ScanTask);
	}

	TaskGroup.wait();

	// Merge blocks for all the tasks

	FComputeBlocksResult Result;

	for (uint64 I = 0; I < NumTasks; ++I)
	{
		const FTask& Task = Tasks[I];
		for (uint64 J = 0; J < Task.Blocks.size(); ++J)
		{
			Result.Blocks.push_back(Task.Blocks[J]);
		}
	}

	if (Params.bNeedMacroBlocks)
	{
		for (uint64 I = 0; I < NumTasks; ++I)
		{
			const FTask& Task = Tasks[I];
			for (uint64 J = 0; J < Task.MacroBlocks.size(); ++J)
			{
				Result.MacroBlocks.push_back(Task.MacroBlocks[J]);
			}
		}
	}

	uint64 UniqueBlockTotalSize = 0;
	uint64 UniqueBlockMinSize	= ~0ull;
	uint64 UniqueBlockMaxSize	= 0ull;

	uint64 NumTinyBlocks   = 0;
	uint64 NumSmallBlocks  = 0;
	uint64 NumMediumBlocks = 0;
	uint64 NumLargeBlocks  = 0;

	uint64 NumTotalBlocks = 0;

	THashSet<FGenericHash> UniqueBlockSet;
	FGenericBlockArray	   UniqueBlocks;
	for (const FGenericBlock& It : Result.Blocks)
	{
		auto InsertResult = UniqueBlockSet.insert(It.HashStrong);
		if (InsertResult.second)
		{
			if (It.Offset + It.Size < InputSize || Result.Blocks.size() == 1)
			{
				UniqueBlockMinSize = std::min<uint64>(UniqueBlockMinSize, It.Size);
			}

			UniqueBlockMaxSize = std::max<uint64>(UniqueBlockMaxSize, It.Size);
			UniqueBlockTotalSize += It.Size;
			UniqueBlocks.push_back(It);
		}

		if (It.Size < MaximumBlockSize / 8)
		{
			NumTinyBlocks++;
		}
		else if (It.Size <= MaximumBlockSize / 4)
		{
			NumSmallBlocks++;
		}
		else if (It.Size <= MaximumBlockSize / 2)
		{
			NumMediumBlocks++;
		}
		else
		{
			NumLargeBlocks++;
		}

		++NumTotalBlocks;
	}

	double AverageBlockSize = InputSize ? double(UniqueBlockTotalSize / UniqueBlocks.size()) : 0;

	UNSYNC_VERBOSE2(
		L"Blocks (tiny/small/medium/large): %llu / %llu / %llu / %llu, average unique size: %llu bytes, unique count: %llu, total count: "
		L"%llu",
		NumTinyBlocks,
		NumSmallBlocks,
		NumMediumBlocks,
		NumLargeBlocks,
		(uint64)AverageBlockSize,
		(uint64)UniqueBlocks.size(),
		NumTotalBlocks);

	UNSYNC_ASSERT(NumTotalBlocks == Result.Blocks.size());

	return Result;
}

FComputeBlocksResult
ComputeBlocksVariable(FIOReader& Reader, const FComputeBlocksParams& Params)
{
	if (Params.bAllowStreaming)
	{
		switch (Params.Algorithm.WeakHashAlgorithmId)
		{
			case EWeakHashAlgorithmID::Naive:
				return ComputeBlocksVariableStreamingT<FRollingChecksum>(Reader, Params);
			case EWeakHashAlgorithmID::BuzHash:
				return ComputeBlocksVariableStreamingT<FBuzHash>(Reader, Params);
			default:
				UNSYNC_FATAL(L"Unsupported weak hash algorithm mode");
				return {};
		}
	}
	else
	{
		switch (Params.Algorithm.WeakHashAlgorithmId)
		{
			case EWeakHashAlgorithmID::Naive:
				return ComputeBlocksVariableT<FRollingChecksum>(Reader, Params);
			case EWeakHashAlgorithmID::BuzHash:
				return ComputeBlocksVariableT<FBuzHash>(Reader, Params);
			default:
				UNSYNC_FATAL(L"Unsupported weak hash algorithm mode");
				return {};
		}
	}
}

FGenericBlockArray
ComputeBlocks(FIOReader& Reader, uint32 BlockSize, FAlgorithmOptions Algorithm)
{
	FComputeBlocksParams Params;
	Params.Algorithm			= Algorithm;
	Params.BlockSize			= BlockSize;
	FComputeBlocksResult Result = ComputeBlocks(Reader, Params);
	return std::move(Result.Blocks);
}

FGenericBlockArray
ComputeBlocks(const uint8* Data, uint64 Size, uint32 BlockSize, FAlgorithmOptions Algorithm)
{
	FComputeBlocksParams Params;
	Params.Algorithm			= Algorithm;
	Params.BlockSize			= BlockSize;
	FComputeBlocksResult Result = ComputeBlocks(Data, Size, Params);
	return std::move(Result.Blocks);
}

FGenericBlockArray
ComputeBlocksVariable(FIOReader& Reader, uint32 BlockSize, EWeakHashAlgorithmID WeakHasher, EStrongHashAlgorithmID StrongHasher)
{
	FComputeBlocksParams Params;
	Params.Algorithm.WeakHashAlgorithmId   = WeakHasher;
	Params.Algorithm.StrongHashAlgorithmId = StrongHasher;
	Params.BlockSize					   = BlockSize;
	FComputeBlocksResult Result			   = ComputeBlocks(Reader, Params);
	return std::move(Result.Blocks);
}

template<typename WeakHasher>
FComputeBlocksResult
ComputeBlocksFixedT(FIOReader& Reader, const FComputeBlocksParams& Params)
{
	UNSYNC_LOG_INDENT;

	auto TimeBegin = TimePointNow();

	const uint32 BlockSize = Params.BlockSize;
	const uint64 NumBlocks = DivUp(Reader.GetSize(), BlockSize);

	FGenericBlockArray Blocks(NumBlocks);
	for (uint64 I = 0; I < NumBlocks; ++I)
	{
		uint64 ChunkSize = CalcChunkSize(I, BlockSize, Reader.GetSize());
		Blocks[I].Offset = I * BlockSize;
		Blocks[I].Size	 = CheckedNarrow(ChunkSize);
	}

	uint64 ReadSize = std::max<uint64>(BlockSize, 8_MB);
	if (Params.bNeedMacroBlocks)
	{
		UNSYNC_FATAL(L"Macro block generation is not implemented for fixed block mode");
		ReadSize = std::max<uint64>(ReadSize, Params.MacroBlockTargetSize);
	}
	UNSYNC_ASSERT(ReadSize % BlockSize == 0);

	const uint64		NumReads		   = DivUp(Reader.GetSize(), ReadSize);
	std::atomic<uint64> NumReadsCompleted  = {};
	std::atomic<uint64> NumBlocksCompleted = {};

	{
		FSchedulerSemaphore IoSemaphore(*GScheduler, 16);
		FTaskGroup			TaskGroup = GScheduler->CreateTaskGroup(&IoSemaphore);
		
		std::unique_ptr<FAsyncReader> AsyncReader = Reader.CreateAsyncReader();

		for (uint64 I = 0; I < NumReads; ++I)
		{
			uint64 ThisReadSize = CalcChunkSize(I, ReadSize, Reader.GetSize());
			uint64 Offset		= I * ReadSize;

			auto ReadCallback = [&NumReadsCompleted, &TaskGroup, &NumBlocksCompleted, &Blocks, &Params, BlockSize](FIOBuffer CmdBuffer,
																												   uint64	 CmdOffset,
																												   uint64	 CmdReadSize,
																												   uint64	 CmdUserData)
			{
				UNSYNC_ASSERT(CmdReadSize);

				TaskGroup.run(
					[&NumReadsCompleted,
					 &NumBlocksCompleted,
					 &Blocks,
					 &Params,
					 BlockSize,
					 CmdBuffer	= MakeShared(std::move(CmdBuffer)),
					 BufferSize = CmdReadSize,
					 Offset		= CmdOffset]()
					{
						UNSYNC_ASSERT(CmdBuffer->GetSize() == BufferSize);

						uint8* Buffer = CmdBuffer->GetData();

						UNSYNC_ASSERT(Offset % BlockSize == 0);
						UNSYNC_ASSERT(BufferSize);
						UNSYNC_ASSERT(Buffer);

						uint64 FirstBlock	  = Offset / BlockSize;
						uint64 NumLocalBlocks = DivUp(BufferSize, BlockSize);
						for (uint64 I = 0; I < NumLocalBlocks; ++I)
						{
							FGenericBlock& Block = Blocks[FirstBlock + I];

							UNSYNC_ASSERT(Block.HashWeak == 0);
							UNSYNC_ASSERT(Block.HashStrong == FGenericHash{});

							Block.HashStrong = ComputeHash(Buffer + I * BlockSize, Block.Size, Params.Algorithm.StrongHashAlgorithmId);

							WeakHasher HashWeak;
							HashWeak.Update(Buffer + I * BlockSize, Block.Size);
							Block.HashWeak = HashWeak.Get();

							++NumBlocksCompleted;
						}

						++NumReadsCompleted;
					});
			};

			AsyncReader->EnqueueRead(Offset, ThisReadSize, 0, ReadCallback);
		}
		AsyncReader->Flush();
		TaskGroup.wait();
	}

	UNSYNC_ASSERT(NumReadsCompleted == NumReads);
	UNSYNC_ASSERT(NumBlocksCompleted == NumBlocks);

	md5_context Hasher;
	md5_init(&Hasher);
	for (uint64 I = 0; I < NumBlocks; ++I)
	{
		if (Blocks[I].HashStrong == FGenericHash{})
		{
			UNSYNC_ERROR(L"Found invalid hash in block %llu", I);
		}
		UNSYNC_ASSERT(Blocks[I].HashStrong != FGenericHash{});
		md5_update(&Hasher, Blocks[I].HashStrong.Data, sizeof(Blocks[I].HashStrong));
	}
	uint8 Hash[16] = {};
	md5_finish(&Hasher, Hash);
	std::string HashStr = BytesToHexString(Hash, sizeof(Hash));
	UNSYNC_VERBOSE2(L"Hash: %hs", HashStr.c_str());

	uint64 TocSize = sizeof(FBlock128) * NumBlocks;
	UNSYNC_VERBOSE2(L"Manifest size: %lld bytes (%.2f MB), blocks: %d", (long long)TocSize, SizeMb(TocSize), uint32(NumBlocks));

	double Duration = DurationSec(TimeBegin, TimePointNow());
	UNSYNC_VERBOSE2(L"Done in %.3f sec (%.3f MB / sec)", Duration, SizeMb((double(Reader.GetSize()) / Duration)));

	THashSet<uint32> UniqueHashes;
	for (const auto& It : Blocks)
	{
		UniqueHashes.insert(It.HashWeak);
	}

	FComputeBlocksResult Result;

	std::swap(Result.Blocks, Blocks);

	return Result;
}

FComputeBlocksResult
ComputeBlocksFixed(FIOReader& Reader, const FComputeBlocksParams& Params)
{
	switch (Params.Algorithm.WeakHashAlgorithmId)
	{
		case EWeakHashAlgorithmID::Naive:
			return ComputeBlocksFixedT<FRollingChecksum>(Reader, Params);
		case EWeakHashAlgorithmID::BuzHash:
			return ComputeBlocksFixedT<FBuzHash>(Reader, Params);
		default:
			UNSYNC_FATAL(L"Unsupported weak hash algorithm mode");
			return {};
	}
}

FComputeBlocksResult
ComputeBlocks(FIOReader& Reader, const FComputeBlocksParams& Params)
{
	switch (Params.Algorithm.ChunkingAlgorithmId)
	{
		case EChunkingAlgorithmID::FixedBlocks:
			return ComputeBlocksFixed(Reader, Params);
		case EChunkingAlgorithmID::VariableBlocks:
			return ComputeBlocksVariable(Reader, Params);
		default:
			UNSYNC_FATAL(L"Unsupported chunking mode");
			return {};
	}
}

FComputeBlocksResult
ComputeBlocks(const uint8* Data, uint64 Size, const FComputeBlocksParams& Params)
{
	FMemReader DataReader(Data, Size);
	return ComputeBlocks(DataReader, Params);
}

static FBuffer GenerateTestData(uint64 Size, uint32 Seed = 1234)
{
	FBuffer Buffer(Size, 0);
	uint32	Rng = Seed;
	for (uint64 I = 0; I < Buffer.Size(); ++I)
	{
		Buffer[I] = Xorshift32(Rng) & 0xFF;
	}
	return Buffer;
}

void
TestChunking()
{
	UNSYNC_LOG(L"TestChunking()");
	UNSYNC_LOG_INDENT;

	UNSYNC_LOG(L"Generating data");

	{
		const uint32 Threshold = ComputeWindowHashThreshold(uint32(64_KB));
		const uint32 ExpectedValue = 0x20000;
		if (Threshold != ExpectedValue)
		{
			UNSYNC_ERROR("Expected window hash threshold for 64 KB target block size: 0x%08x, actual value: 0x%08x", ExpectedValue, Threshold);
		}
	}

	FBuffer Buffer = GenerateTestData(128_MB);

	UNSYNC_LOG(L"Testing expected chunk boundaries");

	{
		UNSYNC_LOG_INDENT;

		FComputeBlocksParams Params;
		Params.bNeedMacroBlocks				   = false;
		Params.BlockSize					   = uint32(64_KB);
		Params.Algorithm.WeakHashAlgorithmId   = EWeakHashAlgorithmID::BuzHash;
		Params.Algorithm.StrongHashAlgorithmId = EStrongHashAlgorithmID::Blake3_160;

		FMemReader			 Reader(Buffer.Data(), 1_MB);
		FComputeBlocksResult Blocks = ComputeBlocksVariable(Reader, Params);

		uint64 NumBlocks = Blocks.Blocks.size();
		uint64 AvgSize	 = Buffer.Size() / NumBlocks;

		static constexpr uint32 NumExpectedBlocks = 18;

		// clang-format off
		const uint64 ExpectedOffsets[NumExpectedBlocks] = {
			0, 34577, 128471, 195115, 238047, 297334, 358754, 396031,
			462359, 508658, 601550, 702021, 754650, 790285, 854987, 887998,
			956848, 1042406
		};
		// clang-format on

		UNSYNC_LOG(L"Generated blocks: %llu, average size: %llu KB", llu(NumBlocks), llu(AvgSize) / 1024);

		if (Blocks.Blocks.size() != NumExpectedBlocks)
		{
			UNSYNC_ERROR("Expected blocks: %llu, actual number: %llu", llu(NumExpectedBlocks), llu(Blocks.Blocks.size()));
		}

		for (uint32 ChunkIndex = 0; ChunkIndex < std::min<size_t>(NumExpectedBlocks, Blocks.Blocks.size()); ++ChunkIndex)
		{
			const FGenericBlock& Block = Blocks.Blocks[ChunkIndex];
			UNSYNC_LOG(L" - [%2d] offset: %llu, size: %llu, weak_hash: 0x%08x",
					   ChunkIndex,
					   llu(Block.Offset),
					   llu(Block.Size),
					   Block.HashWeak);

			if (ExpectedOffsets[ChunkIndex] != Block.Offset)
			{
				UNSYNC_ERROR("Expected block at offset: %llu, actual offset: %llu", ExpectedOffsets[ChunkIndex], Block.Offset);
			}
		}
	}

	static constexpr uint32 NumConfigs		 = 9;
	const uint32			TestChunkSizesKB[NumConfigs] = {8, 16, 32, 64, 96, 128, 160, 192, 256};
	const uint32			ExpectedNumChunks[NumConfigs] = {16442, 8146, 4089, 2019, 1362, 1012, 811, 681, 503};

	UNSYNC_LOG(L"Testing average chunk size");

	for (uint32 ConfigIndex = 0; ConfigIndex < NumConfigs; ++ConfigIndex)
	{
		UNSYNC_LOG_INDENT;

		const uint32 ChunkSizeKB = TestChunkSizesKB[ConfigIndex];
		const uint32 ExpectedCount = ExpectedNumChunks[ConfigIndex];

		FComputeBlocksParams Params;
		Params.bNeedMacroBlocks				   = false;
		Params.BlockSize					   = uint32(ChunkSizeKB * 1024);
		Params.Algorithm.WeakHashAlgorithmId   = EWeakHashAlgorithmID::BuzHash;
		Params.Algorithm.StrongHashAlgorithmId = EStrongHashAlgorithmID::Blake3_160;

		const uint32 Threshold = ComputeWindowHashThreshold(Params.BlockSize);
		UNSYNC_LOG(L"ComputeBlocksVariableT<FBuzHash>, %d KB target, window hash threshold: 0x%08x", ChunkSizeKB, Threshold);

		FMemReader			 Reader(Buffer);
		FComputeBlocksResult Blocks = ComputeBlocksVariable(Reader, Params);

		uint64 NumBlocks = Blocks.Blocks.size();
		uint64 AvgSize	 = Buffer.Size() / NumBlocks;
		
		int64 AbsDiff = std::abs(int64(Params.BlockSize) - int64(AvgSize));
		double AbsDiffPct = 100.0 * double(AbsDiff) / double(Params.BlockSize);

		// Compute median block size
		std::sort(Blocks.Blocks.begin(),
				  Blocks.Blocks.end(),
				  [](const FGenericBlock& A, const FGenericBlock& B) { return A.Size < B.Size; });

		const uint64 MedianSize = Blocks.Blocks[Blocks.Blocks.size() / 2].Size;

		UNSYNC_LOG(L"Generated blocks: %llu, average size: %llu KB, median: %llu KB, average error %.2f %%",
				   llu(NumBlocks),
				   llu(AvgSize) / 1024,
				   llu(MedianSize) / 1024,
				   AbsDiffPct);

		if (AbsDiffPct > 5.0)
		{
			UNSYNC_ERROR(L"Average block size is significantly different from target");
		}

		if (ExpectedCount != NumBlocks)
		{
			UNSYNC_ERROR(L"Expected to generate blocks: %llu, actual: %llu", llu(ExpectedCount), llu(NumBlocks));
		}
	}
}

}  // namespace unsync
