// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"

namespace unsync {

struct FComputeBlocksResult
{
	FGenericBlockArray Blocks;
	FGenericBlockArray MacroBlocks;
};

FComputeBlocksResult ComputeBlocks(FIOReader& Reader, const FComputeBlocksParams& Params);
FComputeBlocksResult ComputeBlocks(const uint8* Data, uint64 Size, const FComputeBlocksParams& Params);
FComputeBlocksResult ComputeBlocksVariable(FIOReader& Reader, const FComputeBlocksParams& Params);

FGenericBlockArray ComputeBlocks(FIOReader& Reader, uint32 BlockSize, FAlgorithmOptions Algorithm);
FGenericBlockArray ComputeBlocks(const uint8* Data, uint64 Size, uint32 BlockSize, FAlgorithmOptions Algorithm);
FGenericBlockArray ComputeBlocksVariable(FIOReader&				Reader,
										 uint32					BlockSize,
										 EWeakHashAlgorithmID	WeakHasher,
										 EStrongHashAlgorithmID StrongHasher);

}  // namespace unsync
