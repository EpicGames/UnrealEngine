// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdDiff.h"
#include "UnsyncFile.h"
#include "UnsyncTarget.h"
#include "UnsyncDiff.h"

namespace unsync {

int32  // TODO: return a TResult
CmdDiff(const FCmdDiffOptions& Options)
{
	if (!Options.Output.empty())
	{
		UNSYNC_LOG(L"Generating patch for '%ls' -> '%ls'", Options.Base.wstring().c_str(), Options.Source.wstring().c_str());
		UNSYNC_LOG(L"Output file '%ls'", Options.Output.wstring().c_str());
	}
	else
	{
		UNSYNC_LOG(L"Comparing base '%ls' and source '%ls'", Options.Base.wstring().c_str(), Options.Source.wstring().c_str());
		UNSYNC_LOG(L"Dry run mode (no output path given).");
		GDryRun = true;
	}

	FBuffer BaseFile = ReadFileToBuffer(Options.Base);
	if (BaseFile.Empty())
	{
		UNSYNC_ERROR(L"Failed to read file '%ls'", Options.Base.wstring().c_str());
		return 1;
	}

	FBuffer SourceFile = ReadFileToBuffer(Options.Source);
	if (SourceFile.Empty())
	{
		UNSYNC_ERROR(L"Failed to open file '%ls'", Options.Source.wstring().c_str());
		return 1;
	}

	UNSYNC_LOG(L"Generating patch");
	FBuffer PatchData = GeneratePatch(BaseFile.Data(),
									  BaseFile.Size(),
									  SourceFile.Data(),
									  SourceFile.Size(),
									  Options.BlockSize,
									  Options.WeakHasher,
									  Options.StrongHasher,
									  Options.CompressionLevel);

	if (PatchData.Empty())
	{
		UNSYNC_LOG(L"Input files are identical. No patch required.");
	}
	else
	{
		UNSYNC_LOG(L"Patch size: %.2f MB", SizeMb(PatchData.Size()));

#if 1  // test patch application process
		{
			FBuffer TargetData = BuildTargetWithPatch(PatchData.Data(), PatchData.Size(), BaseFile.Data(), BaseFile.Size());

			if (TargetData.Size() != SourceFile.Size() || memcmp(TargetData.Data(), SourceFile.Data(), SourceFile.Size()))
			{
				UNSYNC_LOG(L"Patched file does not match source");
				return 1;
			}
		}
#endif	// test patch application process

		if (!GDryRun && !Options.Output.empty())
		{
			UNSYNC_LOG(L"Writing output file to '%ls'", Options.Output.wstring().c_str());
			return WriteBufferToFile(Options.Output, PatchData.Data(), PatchData.Size()) ? 0 : 1;
		}
	}

	return 0;
}

}  // namespace unsync
