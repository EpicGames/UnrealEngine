// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"

namespace unsync {

struct FCmdDiffOptions
{
	FPath				   Source;
	FPath				   Base;
	FPath				   Output;
	int32				   CompressionLevel = 3;
	uint32				   BlockSize		= 0;
	EWeakHashAlgorithmID   WeakHasher		= EWeakHashAlgorithmID::BuzHash;
	EStrongHashAlgorithmID StrongHasher		= EStrongHashAlgorithmID::Blake3_128;
};

int32 CmdDiff(const FCmdDiffOptions& Options);

}  // namespace unsync
