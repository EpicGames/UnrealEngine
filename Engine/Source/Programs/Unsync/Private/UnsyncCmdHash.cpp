// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdHash.h"
#include "UnsyncCore.h"
#include "UnsyncFile.h"
#include "UnsyncSerialization.h"
#include "UnsyncPack.h"
#include "UnsyncChunking.h"

#include <memory>

namespace unsync {

int32  // TODO: return a TResult
CmdHash(const FCmdHashOptions& Options)
{
	if (unsync::IsDirectory(Options.Input))
	{
		UNSYNC_LOG(L"Generating manifest for directory '%ls'", Options.Input.wstring().c_str());

		FPath InputRoot	   = Options.Input;
		FPath ManifestRoot = InputRoot / ".unsync";
		FPath DirectoryManifestPath;

		if (Options.Output.empty())
		{
			DirectoryManifestPath = ManifestRoot / "manifest.bin";

			bool bDirectoryExists = (PathExists(ManifestRoot) && IsDirectory(ManifestRoot)) || (GDryRun || CreateDirectories(ManifestRoot));

			if (!bDirectoryExists && !GDryRun)
			{
				UNSYNC_ERROR(L"Failed to initialize manifest directory '%ls'", ManifestRoot.wstring().c_str());
				return 1;
			}
		}
		else
		{
			DirectoryManifestPath = Options.Output;
		}

		FComputeBlocksParams ComputeBlocksParams;

		ComputeBlocksParams.Algorithm = Options.Algorithm;
		ComputeBlocksParams.BlockSize = Options.BlockSize;
		// TODO: macro block generation is only implemented for variable chunk mode
		ComputeBlocksParams.bNeedMacroBlocks = ComputeBlocksParams.Algorithm.ChunkingAlgorithmId == EChunkingAlgorithmID::VariableBlocks;

		FPath PackOutputRoot = ManifestRoot / "pack";

		std::unique_ptr<FPackWriteContext> PackWriter;
		THashSet<FGenericHash>			   PackedBlocks;
		std::mutex						   PackedBlocksMutex;

		if (!GDryRun && Options.bPackFiles)
		{

			if (!PathExists(PackOutputRoot))
			{
				bool bPackOutputExists = EnsureDirectoryExists(PackOutputRoot);
				if (!bPackOutputExists)
				{
					UNSYNC_ERROR(L"Failed to create pack output directory '%ls'", PackOutputRoot.wstring().c_str());
					return 1;
				}
			}

			PackWriter = std::unique_ptr<FPackWriteContext>(new FPackWriteContext(PackOutputRoot));

			UNSYNC_LOG(L"Pack output: '%ls'", PackOutputRoot.wstring().c_str());

			auto OnBlockGenerated = [&PackWriter, &Options, &PackedBlocks, &PackedBlocksMutex](const FGenericBlock&	   Block,
																							   const FBlockSourceInfo& Source,
																							   FBufferView			   Data)
			{
				if (Source.TotalSize > Options.MaxFileSizeToPack)
				{
					return;
				}

				// Only store unique blocks in the pack
				{
					std::unique_lock<std::mutex> LockScope(PackedBlocksMutex);
					if (!PackedBlocks.insert(Block.HashStrong).second)
					{
						return;
					}
				}

				// TODO(pack): compressed packs will require special handling by the server
				PackWriter->AddRawBlock(Block, Data);
			};

			ComputeBlocksParams.OnBlockGenerated = OnBlockGenerated;
		}

		FDirectoryManifest DirectoryManifest;
		if (Options.bForce)
		{
			DirectoryManifest = CreateDirectoryManifest(Options.Input, ComputeBlocksParams);
		}
		else if (Options.bIncremental)
		{
			UNSYNC_LOG(L"Performing incremental directory manifest generation");
			DirectoryManifest = CreateDirectoryManifestIncremental(Options.Input, ComputeBlocksParams);
		}
		else
		{
			LoadOrCreateDirectoryManifest(DirectoryManifest, Options.Input, ComputeBlocksParams);
		}

		if (PackWriter)
		{
			PackWriter->FinishPack();
			PackWriter->GetUniqueGeneratedPackIds(DirectoryManifest.PackReferences);
		}

		if (!GDryRun)
		{
			UNSYNC_LOG(L"Saving directory manifest '%ls'", DirectoryManifestPath.wstring().c_str());

			SaveDirectoryManifest(DirectoryManifest, DirectoryManifestPath);
		}
	}
	else
	{
		UNSYNC_LOG(L"Generating manfiest for file '%ls'", Options.Input.wstring().c_str());

		FNativeFile OverlappedFile(Options.Input);
		if (OverlappedFile.IsValid())
		{
			UNSYNC_LOG(L"Computing blocks for '%ls' (%.2f MB)", Options.Input.wstring().c_str(), SizeMb(OverlappedFile.GetSize()));
			FComputeBlocksParams ComputeBlocksParams;
			ComputeBlocksParams.Algorithm	 = Options.Algorithm;
			ComputeBlocksParams.BlockSize	 = Options.BlockSize;

			FComputeBlocksResult ComputedBlocks = ComputeBlocks(OverlappedFile, ComputeBlocksParams);

			const FGenericBlockArray& GenericBlocks = ComputedBlocks.Blocks;

			FPath OutputFilename = Options.Output;
			if (OutputFilename.empty())
			{
				OutputFilename = Options.Input.wstring() + std::wstring(L".unsync");
			}

			if (!GDryRun)
			{
				UNSYNC_LOG(L"Saving blocks to '%ls'", OutputFilename.wstring().c_str());
				std::vector<FBlock128> Blocks128;
				Blocks128.reserve(GenericBlocks.size());  // #wip-widehash
				for (const auto& It : GenericBlocks)
				{
					FBlock128 Block;
					Block.HashStrong = It.HashStrong.ToHash128();
					Block.HashWeak	 = It.HashWeak;
					Block.Offset	 = It.Offset;
					Block.Size		 = It.Size;
					Blocks128.push_back(Block);
				}
				return SaveBlocks(Blocks128, Options.BlockSize, OutputFilename) ? 0 : 1;
			}
		}
		else
		{
			UNSYNC_ERROR(L"Failed to open file '%ls'", Options.Input.wstring().c_str());
			return 1;
		}
	}

	return 0;
}

}  // namespace unsync

