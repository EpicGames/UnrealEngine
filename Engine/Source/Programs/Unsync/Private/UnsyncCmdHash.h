// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"

namespace unsync {

struct FCmdHashOptions
{
	FPath			  Input;
	FPath			  Output;
	bool			  bForce			= false;
	bool			  bIncremental		= false;
	bool			  bCompressPacks	= false;
	bool			  bPackFiles		= false;
	uint64			  MaxFileSizeToPack = ~0ull;
	uint32			  BlockSize			= uint32(64_KB);
	FAlgorithmOptions Algorithm;
};

int32 CmdHash(const FCmdHashOptions& Options);

}  // namespace unsync
