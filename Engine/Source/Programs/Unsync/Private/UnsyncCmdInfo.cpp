// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdInfo.h"
#include "UnsyncFile.h"
#include "UnsyncManifest.h"
#include "UnsyncFilter.h"
#include "UnsyncSerialization.h"

#include <fmt/format.h>
#if __has_include(<fmt/xchar.h>)
#	include <fmt/xchar.h>
#endif

namespace unsync {

static void
FilterManifest(const FSyncFilter& SyncFilter, FDirectoryManifest& Manifest)
{
	auto It = Manifest.Files.begin();
	while (It != Manifest.Files.end())
	{
		if (SyncFilter.ShouldSync(It->first))
		{
			++It;
		}
		else
		{
			It = Manifest.Files.erase(It);
		}
	}
}

static THashMap<FGenericHash, FGenericBlock>
BuildBlockMap(const FDirectoryManifest& Manifest, bool bNeedMacroBlocks)
{
	THashMap<FGenericHash, FGenericBlock> Result;
	for (const auto& It : Manifest.Files)
	{
		const FFileManifest& File = It.second;
		if (bNeedMacroBlocks)
		{
			for (const FGenericBlock& Block : File.MacroBlocks)
			{
				Result[Block.HashStrong] = Block;
			}
		}
		else
		{
			for (const FGenericBlock& Block : File.Blocks)
			{
				Result[Block.HashStrong] = Block;
			}
		}
	}
	return Result;
}


void
LogManifestDiff(ELogLevel LogLevel, const FDirectoryManifest& ManifestA, const FDirectoryManifest& ManifestB)
{
	THashMap<FGenericHash, FGenericBlock> BlocksA = BuildBlockMap(ManifestA, false);
	THashMap<FGenericHash, FGenericBlock> BlocksB = BuildBlockMap(ManifestB, false);

	THashMap<FGenericHash, FGenericBlock> MacroBlocksA = BuildBlockMap(ManifestA, true);
	THashMap<FGenericHash, FGenericBlock> MacroBlocksB = BuildBlockMap(ManifestB, true);

	uint32 NumCommonBlocks		= 0;
	uint64 TotalCommonBlockSize = 0;
	uint64 TotalSizeA			= 0;
	uint64 TotalSizeB			= 0;

	uint64 PatchSizeFromAtoB = 0;

	for (const auto& ItA : BlocksA)
	{
		TotalSizeA += ItA.second.Size;
		auto ItB = BlocksB.find(ItA.first);
		if (ItB != BlocksB.end())
		{
			NumCommonBlocks++;
			TotalCommonBlockSize += ItA.second.Size;
		}
	}

	for (const auto& ItB : BlocksB)
	{
		TotalSizeB += ItB.second.Size;
		if (BlocksA.find(ItB.first) == BlocksA.end())
		{
			PatchSizeFromAtoB += ItB.second.Size;
		}
	}

	uint32 NumCommonMacroBlocks		 = 0;
	uint64 TotalCommonMacroBlockSize = 0;
	for (const auto& ItA : MacroBlocksA)
	{
		auto ItB = MacroBlocksB.find(ItA.first);
		if (ItB != MacroBlocksB.end())
		{
			NumCommonMacroBlocks++;
			TotalCommonMacroBlockSize += ItA.second.Size;
		}
	}

	LogPrintf(LogLevel, L"Common macro blocks: %d, %.3f MB\n", NumCommonMacroBlocks, SizeMb(TotalCommonMacroBlockSize));

	LogPrintf(LogLevel,
			  L"Common blocks: %d, %.3f MB (%.2f%% of A, %.2f%% of B)\n",
			  NumCommonBlocks,
			  SizeMb(TotalCommonBlockSize),
			  100.0 * double(TotalCommonBlockSize) / double(TotalSizeA),
			  100.0 * double(TotalCommonBlockSize) / double(TotalSizeB));

	LogPrintf(LogLevel, L"Patch size: %.3f MB\n", SizeMb(PatchSizeFromAtoB));
}

static void LogDecodedManifestJson(const FDirectoryManifest& Manifest)
{
	std::wstring Output;

	Output += L"{\n"; // main object

	const std::string_view StrongHash = ToString(Manifest.Algorithm.StrongHashAlgorithmId);
	const std::string_view WeakHash	  = ToString(Manifest.Algorithm.WeakHashAlgorithmId);
	const std::string_view Chunking	  = ToString(Manifest.Algorithm.ChunkingAlgorithmId);

	FormatJsonKeyValueStr(Output, L"type", L"unsync_manifest", L",\n");
	FormatJsonKeyValueStr(Output, L"hash_strong", ConvertUtf8ToWide(StrongHash), L",\n");
	FormatJsonKeyValueStr(Output, L"hash_weak", ConvertUtf8ToWide(WeakHash), L",\n");
	FormatJsonKeyValueStr(Output, L"chunking", ConvertUtf8ToWide(Chunking), L",\n");

	Output += fmt::format(LR"("files": [)");
	Output += L"\n";

	uint64 FileIndex = 0;
	for (const auto& FileIt : Manifest.Files)
	{
		const std::wstring& Name = FileIt.first;
		const FFileManifest FileManifest = FileIt.second;

		if (FileIndex != 0)
		{
			Output += L",\n";
		}

		Output += L"{";
		FormatJsonKeyValueStr(Output, L"name", Name, L",");
		FormatJsonKeyValueBool(Output, L"read_only", FileManifest.bReadOnly, L", ");
		FormatJsonKeyValueBool(Output, L"executable", FileManifest.bIsExecutable, L", ");
		FormatJsonKeyValueUInt(Output, L"mtime", FileManifest.Mtime, L", ");
		FormatJsonKeyValueUInt(Output, L"size", FileManifest.Size, L", ");
		FormatJsonKeyValueUInt(Output, L"block_size", FileManifest.BlockSize, L", ");
		FormatJsonKeyValueUInt(Output, L"num_blocks", FileManifest.Blocks.size(), L", ");
		FormatJsonKeyValueUInt(Output, L"num_macro_blocks", FileManifest.MacroBlocks.size(), L",");
		Output += L"\"blocks\": ";
		FormatJsonBlockArray(Output, FileManifest.Blocks);
		Output += L",";
		Output += L"\"macro_blocks\": ";
		FormatJsonBlockArray(Output, FileManifest.MacroBlocks);
		Output += L"}";
		++FileIndex;
	}

	Output += L"]\n"; // files
	Output += L"}\n";  // main object

	LogPrintf(ELogLevel::MachineReadable, Output.c_str());
}

int32
CmdInfo(const FCmdInfoOptions& Options)
{
	FPath DirectoryManifestPathA = IsDirectory(Options.InputA) ? (Options.InputA / ".unsync" / "manifest.bin") : Options.InputA;
	FPath DirectoryManifestPathB = IsDirectory(Options.InputB) ? (Options.InputB / ".unsync" / "manifest.bin") : Options.InputB;

	FDirectoryManifest ManifestA;

	bool bManifestAValid = LoadDirectoryManifest(ManifestA, Options.InputA, DirectoryManifestPathA);

	if (!bManifestAValid)
	{
		return 1;
	}

	LogPrintf(ELogLevel::Info, L"Manifest A: %ls\n", DirectoryManifestPathA.wstring().c_str());

	if (Options.SyncFilter)
	{
		FilterManifest(*Options.SyncFilter, ManifestA);
	}

	{
		UNSYNC_LOG_INDENT;
		LogManifestInfo(ELogLevel::Info, ManifestA);
	}

	if (Options.bListFiles)
	{
		UNSYNC_LOG_INDENT;
		LogManifestFiles(ELogLevel::Info, ManifestA);
	}

	if (Options.bDecode)
	{
		LogDecodedManifestJson(ManifestA);
		return 0;
	}


	if (Options.InputB.empty())
	{
		return 0;
	}

	LogPrintf(ELogLevel::Info, L"\n");

	FDirectoryManifest ManifestB;

	bool bManifestBValid = LoadDirectoryManifest(ManifestB, Options.InputB, DirectoryManifestPathB);

	if (!bManifestBValid)
	{
		return 1;
	}

	LogPrintf(ELogLevel::Info, L"Manifest B: %ls\n", DirectoryManifestPathB.wstring().c_str());

	if (Options.SyncFilter)
	{
		FilterManifest(*Options.SyncFilter, ManifestB);
	}

	{
		UNSYNC_LOG_INDENT;
		LogManifestInfo(ELogLevel::Info, ManifestB);
	}
	if (Options.bListFiles)
	{
		UNSYNC_LOG_INDENT;
		LogManifestFiles(ELogLevel::Info, ManifestB);
	}

	LogPrintf(ELogLevel::Info, L"\n");
	LogPrintf(ELogLevel::Info, L"Difference:\n");

	{
		UNSYNC_LOG_INDENT;
		LogManifestDiff(ELogLevel::Info, ManifestA, ManifestB);
	}

	return 0;
}

}  // namespace unsync
