// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"

namespace unsync {

struct FSyncFilter;

struct FCmdInfoOptions
{
	FPath			   InputA;
	FPath			   InputB;
	bool			   bListFiles = false;
	const FSyncFilter* SyncFilter = nullptr;
	bool			   bDecode = false;
};

int32 CmdInfo(const FCmdInfoOptions& Options);

}  // namespace unsync
