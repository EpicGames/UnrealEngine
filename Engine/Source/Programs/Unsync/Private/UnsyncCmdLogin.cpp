// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdLogin.h"
#include "UnsyncAuth.h"
#include "UnsyncHttp.h"
#include "UnsyncLog.h"
#include "UnsyncProxy.h"

namespace unsync {

std::string_view
GetJwtPayload(std::string_view JwtData)
{
	size_t PayloadOffset = JwtData.find('.');
	if (PayloadOffset == std::string::npos)
	{
		return {};
	}
	PayloadOffset += 1;	 // skip the delimiter

	size_t SignatureOffset = JwtData.find('.', PayloadOffset + 1);
	if (SignatureOffset == std::string::npos)
	{
		return {};
	}
	SignatureOffset += 1;  // skip the delimiter

	size_t PayloadLength = SignatureOffset - PayloadOffset - 1;

	return JwtData.substr(PayloadOffset, PayloadLength);
}

int32
CmdLogin(const FCmdLoginOptions& Options)
{
	const int32 RefreshThreshold = Options.bForceRefresh ? INT_MAX : 5 * 60;

	UNSYNC_VERBOSE(L"Connecting to '%hs'", Options.Remote.Host.Address.c_str());
	TResult<ProxyQuery::FHelloResponse> HelloResponseResult = ProxyQuery::Hello(Options.Remote, nullptr /*AuthDesc: anonymous initial connection*/);
	if (HelloResponseResult.IsError())
	{
		UNSYNC_ERROR("Failed establish a handshake with server '%hs'", Options.Remote.Host.Address.c_str());
		LogError(HelloResponseResult.GetError());
		return -1;
	}

	const FAuthDesc AuthDesc = FAuthDesc::FromHelloResponse(*HelloResponseResult);

	TResult<FAuthToken> AuthTokenResult = Authenticate(AuthDesc, Options.Remote.RecvTimeoutSeconds, RefreshThreshold);

	if (AuthTokenResult.IsOk())
	{
		if (!Options.bQuick)
		{
			FHttpConnection Connection = FHttpConnection::CreateDefaultHttps(Options.Remote);

			FHttpRequest Request;

			if (Options.Remote.Protocol == EProtocolFlavor::Unsync)
			{
				Request.Url = "/api/v1/login";
			}
			else if (Options.Remote.Protocol == EProtocolFlavor::Horde)
			{
				Request.Url = "/api/v1/projects";
			}
			else
			{
				Request.Url = "/";
			}

			Request.Method		   = EHttpMethod::GET;
			Request.BearerToken	   = AuthTokenResult->Access;
			FHttpResponse Response = HttpRequest(Connection, Request);

			if (Response.Success())
			{
				UNSYNC_VERBOSE("Login successful");
			}
			else
			{
				LogError(HttpError(Response.Code), L"Failed to authenticate");
				return -1;
			}
		}

		if (Options.bPrint)
		{
			if (Options.bDecode)
			{
				std::string JwtPayload = std::string(GetJwtPayload(AuthTokenResult->Access));
				TransformBase64UrlSafeToVanilla(JwtPayload);

				FBuffer DecodedTokenData;
				if (!DecodeBase64(JwtPayload, DecodedTokenData))
				{
					UNSYNC_ERROR("Failed to Base64-decode access token");
					return -1;
				}
				DecodedTokenData.PushBack(0);

				LogPrintf(ELogLevel::MachineReadable, L"%hs\n", (const char*)DecodedTokenData.Data());
			}
			else
			{
				if (Options.bPrintHttpHeader)
				{
					std::string HttpHeader = "Authorization: Bearer " + AuthTokenResult->Access;
					LogPrintf(ELogLevel::MachineReadable, L"%hs\n", HttpHeader.c_str());
				}
				else
				{
					LogPrintf(ELogLevel::MachineReadable, L"%hs\n", AuthTokenResult->Raw.c_str());
				}
				
			}
		}

		return 0;
	}
	else
	{
		LogError(AuthTokenResult.GetError(), L"Failed to authenticate");
		return -1;
	}
}

}  // namespace unsync
