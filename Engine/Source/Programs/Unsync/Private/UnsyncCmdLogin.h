// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncRemote.h"

namespace unsync {

struct FCmdLoginOptions
{
	FRemoteDesc Remote;
	bool		bInteractive	 = false;  // TODO
	bool		bPrint			 = false;
	bool		bPrintHttpHeader = false;
	bool		bDecode			 = false;
	bool		bForceRefresh	 = false;
	bool		bQuick			 = false;
};

int32 CmdLogin(const FCmdLoginOptions& Options);

}  // namespace unsync
