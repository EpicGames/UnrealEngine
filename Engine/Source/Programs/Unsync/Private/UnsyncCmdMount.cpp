// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdMount.h"
#include "UnsyncMount.h"

namespace unsync {

int32
CmdMount(const FCmdMountOptions& Options)
{
	FMountedDirectory MountedDir;

	bool bOk = MountedDir.Mount(Options.Path);

	return bOk ? 0 : 1;
}

}  // namespace unsync
