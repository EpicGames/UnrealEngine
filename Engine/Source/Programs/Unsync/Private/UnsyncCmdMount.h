// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"

namespace unsync {

struct FCmdMountOptions
{
	FPath Path;
};

int32 CmdMount(const FCmdMountOptions& Options);

}
