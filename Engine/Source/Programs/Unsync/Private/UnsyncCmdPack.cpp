// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdPack.h"
#include "UnsyncCompression.h"
#include "UnsyncFile.h"
#include "UnsyncHashTable.h"
#include "UnsyncSerialization.h"
#include "UnsyncThread.h"
#include "UnsyncError.h"
#include "UnsyncProgress.h"
#include "UnsyncScheduler.h"
#include "UnsyncPack.h"
#include "UnsyncChunking.h"

#include <stdio.h>
#include <stdlib.h>
#include <atomic>

namespace unsync {

template<typename CallbackT>
static void
ForLines(std::string_view String, CallbackT Callback)
{
	while (!String.empty())
	{
		size_t LineEndPos = String.find('\n');
		if (LineEndPos == std::string::npos)
		{
			LineEndPos = String.length();
		}

		std::string_view LineView = String.substr(0, LineEndPos);

		if (LineView.ends_with('\r'))
		{
			LineView = LineView.substr(0, LineView.length() - 1);
		}

		Callback(LineView);

		String = String.substr(LineEndPos + 1);
	}
}

static void
BuildP4HaveSet(const FPath& Root, std::string_view P4HaveDataUtf8, FDirectoryManifest::FFileMap& Result)
{
	auto Callback = [&Result, &Root](std::string_view LineView)
	{
		if (LineView.starts_with("---"))  // p4 diagnostic data
		{
			return;
		}

		size_t HashPos = LineView.find('#');
		if (HashPos == std::string::npos)
		{
			return;
		}

		size_t SplitPos = LineView.find(" - ", HashPos);
		if (SplitPos == std::string::npos)
		{
			return;
		}

		std::string_view DepotPathUtf8 = LineView.substr(0, SplitPos);
		std::string_view LocalPathUtf8 = LineView.substr(SplitPos + 3);

		FPath LocalPath(ConvertUtf8ToWide(LocalPathUtf8));

		FPath RelativePath = GetRelativePath(LocalPath, Root);
		if (RelativePath.empty())
		{
			return;
		}

		FFileManifest FileManifest;
		FileManifest.CurrentPath			 = std::move(LocalPath);
		FileManifest.RevisionControlIdentity = std::move(DepotPathUtf8);

		std::wstring RelativePathStr = RelativePath.wstring();
		Result.insert(std::make_pair(std::move(RelativePathStr), FileManifest));
	};

	ForLines(P4HaveDataUtf8, Callback);
}

struct FPackDatabase
{
	struct FEntry
	{
		FPackIndexEntry IndexEntry = {};
		uint32			PackIndex  = ~0u;
	};

	// non-thread-safe
	void Load(const FPath& PackRoot)
	{
		// TODO: cache index files locally if they're remote

		const FPath ExpectedExtension = FPath(".unsync_index");
		for (const std::filesystem::directory_entry& Dir : RecursiveDirectoryScan(PackRoot))
		{
			if (!Dir.is_regular_file())
			{
				continue;
			}

			const FPath&	IndexFilePath = Dir.path();
			FPathStringView IndexFilePathView(IndexFilePath.native());
			if (!IndexFilePathView.ends_with(ExpectedExtension.native()))
			{
				continue;
			}

			FPath PackFilePath = FPath(IndexFilePathView).replace_extension(".unsync_pack");
			FFileAttributes PackAttrib = GetFileAttrib(PackFilePath);
			if (!PackAttrib.bValid)
			{
				// TODO: also check file size, etc.
				continue;
			}

			uint32 PackIndex = CheckedNarrow(PackFilenames.size());
			PackFilenames.push_back(PackFilePath);
			PackFileCache.push_back(nullptr);

			FBuffer IndexEntries = ReadFileToBuffer(IndexFilePath);	 // TODO: add header & hash at the end

			for (const FPackIndexEntry& IndexEntry : ReinterpretView<FPackIndexEntry>(IndexEntries))
			{
				UNSYNC_ASSERT(IndexEntry.PackBlockOffset + IndexEntry.PackBlockSize < GMaxPackFileSize);

				FEntry DatabaseEntry;
				DatabaseEntry.IndexEntry	   = IndexEntry;
				DatabaseEntry.PackIndex		   = PackIndex;

				if (BlockMap.insert(std::make_pair(IndexEntry.BlockHash, DatabaseEntry)).second)
				{
					TotalCompressedSize += DatabaseEntry.IndexEntry.PackBlockSize;
				}
			}
		}
	}

	// thread-safe
	std::shared_ptr<FNativeFile> GetPackFile(uint32 PackFileIndex) const
	{
		std::lock_guard<std::mutex> LockGuard(FileCacheMutex);

		if (!PackFileCache[PackFileIndex])
		{
			FNativeFile* NewFile		 = new FNativeFile(PackFilenames[PackFileIndex], EFileMode::ReadOnlyUnbuffered);
			PackFileCache[PackFileIndex] = std::shared_ptr<FNativeFile>(NewFile);
		}

		return PackFileCache[PackFileIndex];
	}

	std::vector<FPath>						  PackFilenames;
	THashMap<FHash128, FEntry>				  BlockMap;
	uint64									  TotalCompressedSize = 0;

	mutable std::vector<std::shared_ptr<FNativeFile>> PackFileCache;
	mutable std::mutex FileCacheMutex;
};

static int32
RunSubprocess(const char* Command, const FPath& WorkingDirectory, std::string& StdOutBuffer)
{
#if UNSYNC_PLATFORM_WINDOWS

	char TempBuffer[65536];

	wchar_t* PrevWD = _wgetcwd(nullptr, 0);

	if (!PrevWD)
	{
		UNSYNC_ERROR(L"Failed to get current working directory")
		return -1;
	}

	int32 ErrorCode = _wchdir(WorkingDirectory.native().c_str());

	FILE* Pipe = _popen(Command, "rt");
	if (Pipe)
	{
		for (;;)
		{
			size_t ReadSize = fread(TempBuffer, 1, sizeof(TempBuffer), Pipe);
			if (ReadSize != 0)
			{
				StdOutBuffer.append(TempBuffer, ReadSize);
			}

			if (feof(Pipe))
			{
				break;
			}
		}

		ErrorCode = _pclose(Pipe);
	}
	else
	{
		ErrorCode = errno;
	}

	_wchdir(PrevWD);
	free(PrevWD);

	return ErrorCode;

#else  // UNSYNC_PLATFORM_WINDOWS

	UNSYNC_FATAL(L"RunSubprocess() is not implemented");
	return -1;

#endif	// UNSYNC_PLATFORM_WINDOWS
}

int32
CmdPack(const FCmdPackOptions& Options)
{
	const FFileAttributes RootAttrib = GetFileAttrib(Options.RootPath);

	const FPath InputRoot	 = Options.RootPath;
	const FPath ManifestRoot = InputRoot / ".unsync";
	const FPath StoreRoot	 = Options.StorePath.empty() ? ManifestRoot : Options.StorePath;
	const FPath PackRoot	 = StoreRoot / "pack";
	const FPath SnapshotRoot = StoreRoot / "snapshot";
	const FPath TagRoot		 = StoreRoot / "tag";

	UNSYNC_LOG(L"Generating package for directory '%ls'", InputRoot.wstring().c_str());
	UNSYNC_LOG_INDENT;

	if (!RootAttrib.bValid)
	{
		UNSYNC_ERROR(L"Input directory '%ls' does not exist", InputRoot.wstring().c_str());
		return -1;
	}

	if (!RootAttrib.bDirectory)
	{
		UNSYNC_ERROR(L"Input '%ls' is not a directory", InputRoot.wstring().c_str());
		return -1;
	}

	std::string P4HaveBuffer;

	if (Options.bRunP4Have)
	{
		UNSYNC_LOG(L"Runinng `p4 have`");
		int32 ReturnCode = RunSubprocess("p4 have ...", InputRoot, P4HaveBuffer);
		if (ReturnCode != 0)
		{
			UNSYNC_ERROR("Error reported while running `p4 have`: %d", ReturnCode);
			return -1;
		}
	}
	else if (!Options.P4HavePath.empty())
	{
		UNSYNC_LOG(L"Loading p4 manifest file '%ls'", Options.P4HavePath.wstring().c_str());

		FNativeFile P4HaveFile(Options.P4HavePath, EFileMode::ReadOnly);
		if (!P4HaveFile.IsValid())
		{
			UNSYNC_ERROR(L"Could not open p4 manifest file '%ls'", Options.P4HavePath.wstring().c_str());
			return -1;
		}

		P4HaveBuffer.resize(P4HaveFile.GetSize());
		uint64 ReadBytes = P4HaveFile.Read(P4HaveBuffer.data(), 0, P4HaveFile.GetSize());
		if (ReadBytes != P4HaveFile.GetSize())
		{
			UNSYNC_ERROR(L"Could not read the entire p4 manifest from '%ls'", Options.P4HavePath.wstring().c_str());
			return -1;
		}
	}

	// TODO: allow explicit output path
	{
		if (!EnsureDirectoryExists(ManifestRoot))
		{
			UNSYNC_ERROR(L"Failed to create manifest output directory '%ls'", ManifestRoot.wstring().c_str());
			return -1;
		}

		if (!EnsureDirectoryExists(PackRoot))
		{
			UNSYNC_ERROR(L"Failed to create pack output directory '%ls'", PackRoot.wstring().c_str());
			return -1;
		}

		if (!EnsureDirectoryExists(SnapshotRoot))
		{
			UNSYNC_ERROR(L"Failed to create snapshot output directory '%ls'", SnapshotRoot.wstring().c_str());
			return -1;
		}

		if (!EnsureDirectoryExists(TagRoot))
		{
			UNSYNC_ERROR(L"Failed to create tag output directory '%ls'", TagRoot.wstring().c_str());
			return -1;
		}
	}

	THashSet<FHash128> SeenBlockHashSet;  // TODO: use FPackDatabase, perhaps in some osrt of lightweight mode

	UNSYNC_LOG(L"Loading block database");
	{
		UNSYNC_LOG_INDENT;
		FPath ExpectedExtension = FPath(".unsync_index");

		FPackIndexDatabase TempPackIndexDatabase;

		for (const std::filesystem::directory_entry& Dir : RecursiveDirectoryScan(PackRoot))
		{
			if (!Dir.is_regular_file())
			{
				continue;
			}

			const FPath&	FilePath = Dir.path();
			FPathStringView FilePathView(FilePath.native());
			if (!FilePathView.ends_with(ExpectedExtension.native()))
			{
				continue;
			}

			FNativeFile IndexFile(FilePath, EFileMode::ReadOnly);
			if (!IndexFile.IsValid())
			{
				UNSYNC_WARNING(L"Failed to open pack index file '%ls'", FilePath.wstring().c_str());
				continue;
			}

			FIOReaderStream IndexFileStream(IndexFile);
			if (!LoadPackIndexDatabase(TempPackIndexDatabase, IndexFileStream))
			{
				UNSYNC_WARNING(L"Failed to read pack index database from file '%ls'", FilePath.wstring().c_str());
				continue;
			}

			for (const FPackIndexEntry& Entry : TempPackIndexDatabase.Entries)
			{
				SeenBlockHashSet.insert(Entry.BlockHash);
			}
		}
	}

	const FPath DirectoryManifestPath = ManifestRoot / "manifest.bin";

	FDirectoryManifest DirectoryManifest;
	FDirectoryManifest OldDirectoryManifest;

	if (PathExists(DirectoryManifestPath))
	{
		UNSYNC_LOG(L"Loading previous manifest ");
		UNSYNC_LOG_INDENT;

		if (LoadDirectoryManifest(OldDirectoryManifest, InputRoot, DirectoryManifestPath))
		{
			if (OldDirectoryManifest.bHasFileRevisionControl)
			{
				UNSYNC_LOG(L"Loaded existing manifest with revision control data");
			}
			else
			{
				UNSYNC_LOG(L"Loaded existing manifest without revision control data");
			}
		}
	}

	if (!P4HaveBuffer.empty())
	{
		UNSYNC_LOG(L"Processing revision control data");

		DirectoryManifest.Algorithm = Options.Algorithm;

		BuildP4HaveSet(InputRoot, P4HaveBuffer, DirectoryManifest.Files);
		DirectoryManifest.bHasFileRevisionControl = true;

		UNSYNC_LOG(L"Loaded entries from p4 manifest: %llu", llu(DirectoryManifest.Files.size()));

		UNSYNC_LOG(L"Updating file metadata");
		auto UpdateFileMetadata = [&OldDirectoryManifest](std::pair<const std::wstring, FFileManifest>& It)
		{
			if (OldDirectoryManifest.bHasFileRevisionControl)
			{
				auto OldFileIt = OldDirectoryManifest.Files.find(It.first);
				if (OldFileIt != OldDirectoryManifest.Files.end())
				{
					if (It.second.RevisionControlIdentity == OldFileIt->second.RevisionControlIdentity)
					{
						It.second.Mtime		    = OldFileIt->second.Mtime;
						It.second.Size		    = OldFileIt->second.Size;
						It.second.bReadOnly     = OldFileIt->second.bReadOnly;
						It.second.bIsExecutable = OldFileIt->second.bIsExecutable;
					}
				}
			}

			if (!It.second.IsValid())
			{
				FFileAttributes Attrib = GetFileAttrib(It.second.CurrentPath);

				It.second.Mtime		    = Attrib.Mtime;
				It.second.Size		    = Attrib.Size;
				It.second.bReadOnly     = true;	 // treat all p4 files as read-only in the manifest
				It.second.bIsExecutable = Attrib.bIsExecutable;
			}
		};
		ParallelForEach(DirectoryManifest.Files, UpdateFileMetadata);
	}
	else
	{
		// create a lightweight manifest, without blocks
		FComputeBlocksParams LightweightManifestParams;
		LightweightManifestParams.Algorithm	  = Options.Algorithm;
		LightweightManifestParams.bNeedBlocks = false;
		LightweightManifestParams.BlockSize	  = 0;

		DirectoryManifest = CreateDirectoryManifest(InputRoot, LightweightManifestParams);
	}

	UNSYNC_LOG(L"Found files: %llu", llu(DirectoryManifest.Files.size()));

	std::atomic<uint64> ProcessedRawBytes;
	std::atomic<uint64> CompressedBytes;

	FPackWriteContext PackWriter(PackRoot);

	FThreadLogConfig LogConfig;

	std::mutex Mutex;

	FOnBlockGenerated OnBlockGenerated =
		[&Mutex, &PackWriter, &SeenBlockHashSet, &ProcessedRawBytes, &CompressedBytes, &LogConfig](const FGenericBlock&	   Block,
																								   const FBlockSourceInfo& SourceInfo,
																								   FBufferView			   Data)
	{
		UNSYNC_UNUSED(SourceInfo);

		FThreadLogConfig::FScope LogConfigScope(LogConfig);

		{
			std::lock_guard<std::mutex> LockGuard(Mutex);
			if (!SeenBlockHashSet.insert(Block.HashStrong.ToHash128()).second)
			{
				return;
			}
		}

		FPackWriteContext::FCompressedBlock CompressedBlock = FPackWriteContext::CompressBlock(Data);

		ProcessedRawBytes += Block.Size;
		CompressedBytes += CompressedBlock.Data.GetSize();

		PackWriter.CompressAndAddBlock(Block, Data);
	};

	FComputeBlocksParams BlockParams;
	BlockParams.Algorithm = Options.Algorithm;
	BlockParams.BlockSize = Options.BlockSize;

	// TODO: threading makes pack files non-deterministic...
	// Perhaps a strictly ordered parallel pipeline mechanism could be implemented?
	BlockParams.bAllowThreading	 = true;
	BlockParams.OnBlockGenerated = OnBlockGenerated;

	if (OldDirectoryManifest.IsValid())
	{
		// Copy file blocks from old manifest, if possible
		if (AlgorithmOptionsCompatible(DirectoryManifest.Algorithm, OldDirectoryManifest.Algorithm))
		{
			MoveCompatibleManifestBlocks(DirectoryManifest, std::move(OldDirectoryManifest));
		}
		else
		{
			UNSYNC_LOG(L"Incremental file block generation is not possible due to algorithm options mismatch");
		}
	}

	UNSYNC_LOG(L"Computing file blocks");
	{
		// Invalidate files with blocks that are not available in the pack DB
		auto BlockValidator = [&SeenBlockHashSet](std::pair<const std::wstring, FFileManifest>& It)
		{
			FFileManifest& FileManifest = It.second;
			uint64		   ValidFileSize = 0;

			for (const FGenericBlock& Block : FileManifest.Blocks)
			{
				if (SeenBlockHashSet.find(Block.HashStrong.ToHash128()) == SeenBlockHashSet.end())
				{
					break;
				}

				ValidFileSize += Block.Size;
			}

			if (ValidFileSize != FileManifest.Size)
			{
				FileManifest.BlockSize = 0;
				FileManifest.Blocks.clear();
				FileManifest.MacroBlocks.clear();
			}
		};
		ParallelForEach(DirectoryManifest.Files, BlockValidator);

		// Scan files and generate new unique blocks
		UpdateDirectoryManifestBlocks(DirectoryManifest, InputRoot, BlockParams);
	}

	uint64 ManifestUniqueBytes	   = 0;
	uint64 ManifestCompressedBytes = 0;

	UNSYNC_LOG(L"Saving directory manifest");

	{
		FBuffer			 ManifestBuffer;
		FVectorStreamOut ManifestStream(ManifestBuffer);
		bool			 bManifestSerialized = SaveDirectoryManifest(DirectoryManifest, ManifestStream);
		if (!bManifestSerialized)
		{
			UNSYNC_FATAL(L"Failed to serialize directory manifest to memory");
			return -1;
		}

		std::vector<FGenericBlock> ManifestBlocks;
		FOnBlockGenerated		   OnManifestBlockGenerated =
			[&OnBlockGenerated, &Mutex, &ManifestBlocks](const FGenericBlock& Block, const FBlockSourceInfo& SourceInfo, FBufferView Data)
		{
			{
				std::lock_guard<std::mutex> LockGuard(Mutex);
				ManifestBlocks.push_back(Block);
			}
			OnBlockGenerated(Block, SourceInfo, Data);
		};

		ManifestUniqueBytes -= ProcessedRawBytes.load();
		ManifestCompressedBytes -= CompressedBytes.load();

		BlockParams.OnBlockGenerated = OnManifestBlockGenerated;
		FMemReader ManifestDataReader(ManifestBuffer);
		ComputeBlocks(ManifestDataReader, BlockParams);

		if (!WriteBufferToFile(DirectoryManifestPath, ManifestBuffer))
		{
			UNSYNC_FATAL(L"Failed to save directory manifest to file '%ls'", DirectoryManifestPath.wstring().c_str());
			return 1;
		}

		std::sort(ManifestBlocks.begin(), ManifestBlocks.end(), FGenericBlock::FCompareByOffset());

		for (const FGenericBlock& Block : ManifestBlocks)
		{
			UNSYNC_ASSERT(SeenBlockHashSet.find(Block.HashStrong.ToHash128()) != SeenBlockHashSet.end());
		}

		FBufferView ManifestBlocksBuffer;
		ManifestBlocksBuffer.Data = reinterpret_cast<const uint8*>(ManifestBlocks.data());
		ManifestBlocksBuffer.Size = sizeof(ManifestBlocks[0]) * ManifestBlocks.size();

		FHash128 ManifestBlocksBufferHash = HashBlake3Bytes<FHash128>(ManifestBlocksBuffer.Data, ManifestBlocksBuffer.Size);

		std::string SnapshotId	 = HashToHexString(ManifestBlocksBufferHash);  // TODO: allow overriding this from command line
		FPath		SnapshotPath = SnapshotRoot / (SnapshotId + ".unsync_snapshot");

		UNSYNC_LOG(L"Writing snapshot: %hs", SnapshotId.c_str());

		bool bSnapshotWritten = WriteBufferToFile(SnapshotPath, ManifestBlocksBuffer.Data, ManifestBlocksBuffer.Size);
		if (!bSnapshotWritten)
		{
			UNSYNC_FATAL(L"Failed to write snapthot file '%ls'", SnapshotPath.wstring().c_str());
			return -1;
		}

		// If explicit snapshot name is provided, save it as a separate "tag" file,
		// never overwriting normal snapshots which are named based on content hash.
		if (!Options.SnapshotName.empty())
		{
			UNSYNC_LOG(L"Saving snapshot %hs as tag '%hs'", SnapshotId.c_str(), Options.SnapshotName.c_str());
			FPath			TagPath = TagRoot / (Options.SnapshotName + ".unsync_tag");
			std::error_code ErrorCode;
			if (!FileCopyOverwrite(SnapshotPath, TagPath, ErrorCode))
			{
				std::string SystemError = FormatSystemErrorMessage(ErrorCode.value());
				UNSYNC_FATAL(L"Failed to save snapshot tag '%ls'. %hs", TagPath.wstring().c_str(), SystemError.c_str());
				return -1;
			}
		}

		ManifestUniqueBytes += ProcessedRawBytes.load();
		ManifestCompressedBytes += CompressedBytes.load();
	}

	PackWriter.FinishPack();

	uint64 SourceSize = 0;
	for (const auto& It : DirectoryManifest.Files)
	{
		SourceSize += It.second.Size;
	}

	const uint64 NumSourceFiles = DirectoryManifest.Files.size();
	UNSYNC_LOG(L"Source files: %llu", llu(NumSourceFiles));
	UNSYNC_LOG(L"Source size: %llu bytes (%.2f MB)", llu(SourceSize), SizeMb(SourceSize));
	UNSYNC_LOG(L"Manifest unique data size: %llu bytes (%.2f MB)", llu(ManifestUniqueBytes), SizeMb(ManifestUniqueBytes));
	UNSYNC_LOG(L"Manifest unique compressed size: %llu bytes (%.2f MB)", llu(ManifestCompressedBytes), SizeMb(ManifestCompressedBytes));
	UNSYNC_LOG(L"New data size: %llu bytes (%.2f MB)", llu(ProcessedRawBytes), SizeMb(ProcessedRawBytes));
	UNSYNC_LOG(L"Compressed size: %llu bytes (%.2f MB), %.0f%%",
			   llu(CompressedBytes.load()),
			   SizeMb(CompressedBytes.load()),
			   ProcessedRawBytes > 0 ? (100.0 * double(CompressedBytes.load()) / double(ProcessedRawBytes)) : 0);

	return 0;
}

bool
BuildTargetFromPack(FIOWriter& Output, const FPackDatabase& PackDb, TArrayView<FGenericBlock> Manifest)
{
	struct FScheduleItem
	{
		uint32				   PackIndex  = ~0u;
		const FPackIndexEntry* IndexEntry = nullptr;
		const FGenericBlock*   Block;

		bool operator<(const FScheduleItem& Other) const
		{
			if (PackIndex != Other.PackIndex)
			{
				return PackIndex < Other.PackIndex;
			}
			return IndexEntry->PackBlockOffset < Other.IndexEntry->PackBlockOffset;
		}
	};

	std::vector<FScheduleItem> Schedule;
	Schedule.reserve(Manifest.Size());

	for (const FGenericBlock& Block : Manifest)
	{
		FHash128 BlockHash = Block.HashStrong.ToHash128();

		auto FindIt = PackDb.BlockMap.find(BlockHash);
		if (FindIt == PackDb.BlockMap.end())
		{
			UNSYNC_ERROR(L"Pack database does not contain required block");
			return false;
		}

		const FPackDatabase::FEntry& PackEntry = FindIt->second;

		FScheduleItem ScheduleItem;
		ScheduleItem.PackIndex	= PackEntry.PackIndex;
		ScheduleItem.IndexEntry = &PackEntry.IndexEntry;
		ScheduleItem.Block		= &Block;

		Schedule.push_back(ScheduleItem);
	}

	std::sort(Schedule.begin(), Schedule.end());

	std::shared_ptr<FNativeFile> PackFile;
	uint32						 CurrentPackId = ~0u;

	// TODO: multi-threaded decompression

	auto ReadCallback = [&Schedule, &Output](FIOBuffer Buffer, uint64 SourceOffset, uint64 ReadSize, uint64 ScheduleIndex)
	{
		const FScheduleItem& Item = Schedule[ScheduleIndex];

		UNSYNC_ASSERT(ReadSize == Item.IndexEntry->PackBlockSize);

		FHash128 CompressedHash = HashBlake3Bytes<FHash128>(Buffer.GetData(), Buffer.GetSize());
		UNSYNC_ASSERT(CompressedHash == Item.IndexEntry->CompressedHash);

		FIOBuffer Decompressed = FIOBuffer::Alloc(Item.Block->Size, L"PackBlockDecompress");
		if (!Decompress(Buffer.GetBufferView(), Decompressed.GetMutBufferView()))
		{
			UNSYNC_FATAL(L"Failed to decompress block while reading from pack");
			return;
		}

		UNSYNC_ASSERT(Decompressed.GetSize() == Item.Block->Size);

		Output.Write(Decompressed.GetData(), Item.Block->Offset, Item.Block->Size);
	};

	std::unique_ptr<FAsyncReader> AsyncReader;

	for (uint64 ScheduleIndex = 0; ScheduleIndex < Schedule.size(); ++ScheduleIndex)
	{
		const FScheduleItem& Item = Schedule[ScheduleIndex];
		if (CurrentPackId != Item.PackIndex)
		{
			std::shared_ptr<FNativeFile> NextPackFile = PackDb.GetPackFile(Item.PackIndex);

			if (!NextPackFile->IsValid())
			{
				UNSYNC_ERROR(L"Failed to open pack file '%ls'", PackDb.PackFilenames[Item.PackIndex].wstring().c_str());
				return false;
			}

			AsyncReader	  = NextPackFile->CreateAsyncReader();
			std::swap(PackFile, NextPackFile);

			CurrentPackId = Item.PackIndex;
		}

		UNSYNC_ASSERT(Item.IndexEntry->PackBlockOffset < PackFile->GetSize());
		UNSYNC_ASSERT(Item.IndexEntry->PackBlockOffset + Item.IndexEntry->PackBlockSize <= PackFile->GetSize());
		AsyncReader->EnqueueRead(Item.IndexEntry->PackBlockOffset, Item.IndexEntry->PackBlockSize, ScheduleIndex, ReadCallback);
	}

	if (AsyncReader)
	{
		AsyncReader->Flush();
	}
	AsyncReader = {};

	return true;
}

struct FDirectoryCreationCache
{
	bool EnsureDirectoryExists(const FPath& Path)
	{
		if (GDryRun)
		{
			return true;
		}

		std::lock_guard<std::mutex> LockGuard(Mutex);
		if (CreatedDirectories.find(Path.native()) != CreatedDirectories.end())
		{
			return true;
		}
		else
		{
			bool bCreated = unsync::EnsureDirectoryExists(Path);
			if (bCreated)
			{
				CreatedDirectories.insert(Path.native());
			}
			return bCreated;
		}
	}

	THashSet<FPath::string_type> CreatedDirectories;
	std::mutex					 Mutex;
};

uint64 ComputeManifestTotalSize(const FDirectoryManifest& Manifest)
{
	uint64 Result = 0;

	for (const auto& It : Manifest.Files)
	{
		Result += It.second.Size;
	}

	return Result;
}

bool
SyncDirectoryFromPack(const FPath& OutputRoot, const FPackDatabase& PackDb, const FDirectoryManifest& NewDirectoryManifest)
{
	// TODO: incrementally patch individual files
	// TODO: fetch blocks from server

	FTimingLogger SyncTimingLogger("Sync time", ELogLevel::Info);

	FDirectoryManifest OldDirectoryManifest;
	FPath			   OldManifestPath = OutputRoot / ".unsync" / "manifest.bin";
	if (PathExists(OldManifestPath))
	{
		UNSYNC_LOG(L"Loading existing unsync manifest for '%ls'", OutputRoot.wstring().c_str());
		UNSYNC_LOG_INDENT;

		if (LoadDirectoryManifest(OldDirectoryManifest, OutputRoot, OldManifestPath))
		{
			if (OldDirectoryManifest.bHasFileRevisionControl)
			{
				UNSYNC_LOG(L"Loaded existing manifest with revision control data");
			}
			else
			{
				UNSYNC_LOG(L"Loaded existing manifest without revision control data");
			}

			UNSYNC_LOG(L"Updating file metadata");
			FTimingLogger TimingLogger("Manifest generation time", ELogLevel::Info);

			std::atomic<uint64> NumDirtyFiles = 0;

			FLogProgressScope ScanProgressLogger(OldDirectoryManifest.Files.size(), ELogProgressUnits::Raw, 1000, false /*bVerboseOnly*/);

			FThreadLogConfig LogConfig;
			auto UpdateFileMetadata = [&NumDirtyFiles, &ScanProgressLogger, &LogConfig](std::pair<const std::wstring, FFileManifest>& It)
			{
				FThreadLogConfig::FScope LogConfigScope(LogConfig);

				FFileAttributes Attrib = GetFileAttrib(It.second.CurrentPath);

				// Check if the file on disk still matches the manifest and invalidate it otherwise
				if (!Attrib.bValid || It.second.Mtime != Attrib.Mtime || It.second.Size != Attrib.Size ||
					It.second.bReadOnly != Attrib.bReadOnly || It.second.bIsExecutable != Attrib.bIsExecutable)
				{
					It.second = FFileManifest();
					NumDirtyFiles++;
				}

				ScanProgressLogger.Add(1);
			};
			ParallelForEach(OldDirectoryManifest.Files, UpdateFileMetadata);

			ScanProgressLogger.Complete();

			if (NumDirtyFiles)
			{
				UNSYNC_LOG(L"Found dirty files: %llu", llu(NumDirtyFiles));
			}
		}
	}

	if (!OldDirectoryManifest.IsValid())
	{
		UNSYNC_LOG(L"Creating unsync manifest for '%ls'", OutputRoot.wstring().c_str());
		UNSYNC_LOG_INDENT;

		FTimingLogger TimingLogger("Manifest generation complete", ELogLevel::Info);

		FComputeBlocksParams LightweightManifestParams;
		LightweightManifestParams.Algorithm	  = NewDirectoryManifest.Algorithm;
		LightweightManifestParams.bNeedBlocks = false;
		LightweightManifestParams.BlockSize	  = 0;

		OldDirectoryManifest = CreateDirectoryManifest(OutputRoot, LightweightManifestParams);
	}

	if (!OldDirectoryManifest.IsValid())
	{
		UNSYNC_ERROR(L"Failed to load or create manifest for target directory");
		return false;
	}

	UNSYNC_LOG("Writing target files");

	FAtomicError Error;

	FDirectoryCreationCache DirCache;

	struct FStats
	{
		std::atomic<uint64> NumFilesSkipped;
		std::atomic<uint64> NumFilesSynced;

		std::atomic<uint64> BytesSkipped;
		std::atomic<uint64> BytesSynced;
	};

	FStats Stats;

	const uint64 TotalSizeInBytes = ComputeManifestTotalSize(NewDirectoryManifest);

	FLogProgressScope SyncProgressLogger(TotalSizeInBytes, ELogProgressUnits::MB, 1000, false /*bVerboseOnly*/);

	FThreadLogConfig LogConfig;
	auto BuildTargetCallback = [&Error, &PackDb, &DirCache, &OldDirectoryManifest, &Stats, &SyncProgressLogger, &LogConfig](
								   const std::pair<const std::wstring, FFileManifest>& FileIt)
	{
		if (Error)
		{
			return;
		}

		FThreadLogConfig::FScope LogConfigScope(LogConfig);

		const FFileManifest& FileManifest	= FileIt.second;
		const FPath&		 TargetFilePath = FileManifest.CurrentPath;

		auto				 OldManifestIt	 = OldDirectoryManifest.Files.find(FileIt.first);
		const FFileManifest* OldFileManifest = nullptr;
		if (OldManifestIt != OldDirectoryManifest.Files.end())
		{
			OldFileManifest = &OldManifestIt->second;

			// TODO: option to diff blocks instead of just checking revision and metadata
			// TODO: fix only the the read-only flag when that's the only difference
			if (OldFileManifest->RevisionControlIdentity == FileManifest.RevisionControlIdentity &&
				OldFileManifest->Mtime == FileManifest.Mtime && OldFileManifest->Size == FileManifest.Size &&
				OldFileManifest->bReadOnly == FileManifest.bReadOnly &&
				OldFileManifest->bIsExecutable == FileManifest.bIsExecutable)
			{
				UNSYNC_VERBOSE2(L"Skipped '%ls' (up to date)", FileIt.first.c_str());
				Stats.NumFilesSkipped += 1;
				Stats.BytesSkipped += FileManifest.Size;

				SyncProgressLogger.Add(FileManifest.Size);

				return;
			}
		}

		FPath TargetFileParent = TargetFilePath.parent_path();
		if (!DirCache.EnsureDirectoryExists(TargetFileParent))
		{
			UNSYNC_ERROR(L"Failed to create target directory '%ls'", TargetFileParent.wstring().c_str());
			Error.Set(AppError("Failed to create target directory"));
			return;
		}

		// TODO: perform block diff before the expensive copy

		if (GDryRun)
		{
			FNullReaderWriter NullFile(FileManifest.Size);
			if (!BuildTargetFromPack(NullFile, PackDb, MakeView(FileManifest.Blocks)))
			{
				UNSYNC_FATAL(L"Failed to reconstruct target file from pack '%ls'", TargetFilePath.wstring().c_str());
				Error.Set(AppError("Failed to reconstruct target file from pack"));
				return;
			}
		}
		else
		{
			SetFileReadOnly(TargetFilePath, false);

			FNativeFile TargetFile(TargetFilePath, EFileMode::CreateWriteOnly, FileManifest.Size);

			if (!TargetFile.IsValid())
			{
				std::string SystemError = FormatSystemErrorMessage(TargetFile.GetError());
				UNSYNC_ERROR(L"Failed to create target file '%ls'. %hs", TargetFilePath.wstring().c_str(), SystemError.c_str());
				Error.Set(AppError("Failed to create target file", TargetFile.GetError()));
				return;
			}

			if (!BuildTargetFromPack(TargetFile, PackDb, MakeView(FileManifest.Blocks)))
			{
				UNSYNC_FATAL(L"Failed to reconstruct target file from pack '%ls'", TargetFilePath.wstring().c_str());
				Error.Set(AppError("Failed to reconstruct target file from pack"));
				return;
			}

			TargetFile.Close();

			if (FileManifest.Mtime)
			{
				SetFileMtime(TargetFilePath, FileManifest.Mtime);
			}

			if (FileManifest.bReadOnly)
			{
				SetFileReadOnly(TargetFilePath, true);
			}

			if (FileManifest.bIsExecutable)
			{
				SetFileExecutable(TargetFilePath, true);
			}
		}

		Stats.NumFilesSynced += 1;
		Stats.BytesSynced += FileManifest.Size;

		SyncProgressLogger.Add(FileManifest.Size);
	};

	ParallelForEach(NewDirectoryManifest.Files, BuildTargetCallback);
	SyncProgressLogger.Complete();

	UNSYNC_LOG(L"Skipped files: %llu (%.2f MB)", llu(Stats.NumFilesSkipped), SizeMb(Stats.BytesSkipped));
	UNSYNC_LOG(L"Synced files: %llu (%.2f MB)", llu(Stats.NumFilesSynced), SizeMb(Stats.BytesSynced));

	return !Error;
}

bool
VerifyManifest(const FPackDatabase& PackDb, const FDirectoryManifest& NewDirectoryManifest)
{
	FAtomicError Error;

	auto VerifyFileManifest = [&Error, &PackDb](const std::pair<const std::wstring, FFileManifest>& FileIt)
	{
		if (Error)
		{
			return;
		}

		const FFileManifest& FileManifest = FileIt.second;
		for (const FGenericBlock& Block : FileManifest.Blocks)
		{
			if (PackDb.BlockMap.find(Block.HashStrong.ToHash128()) == PackDb.BlockMap.end())
			{
				Error.Set(AppError("Found unknown block in the manifest"));
			}
		}
	};

	ParallelForEach(NewDirectoryManifest.Files, VerifyFileManifest);

	return !Error;
}

enum class ERevisionControlFileFormat
{
	IdentityOnly, // only include revision control identity of the file
	P4Have,  // use the same format as `p4 have` output, i.e. "//depot/file.txt#123 - C:\Local\Path\file.txt"
};

bool
SaveRevisionControlData(const FPath& OutputPath, const FDirectoryManifest& Manifest, ERevisionControlFileFormat Format)
{
	if (OutputPath.has_parent_path())
	{
		FPath ParentPath = OutputPath.parent_path();
		if (!EnsureDirectoryExists(ParentPath))
		{
			UNSYNC_ERROR(L"Failed to create output directory '%ls'", ParentPath.wstring().c_str());
			return false;
		}
	}

	std::string OutputString;

	std::string CurrentPathUtf8;
	for (const auto& It : Manifest.Files)
	{
		const FFileManifest& FileManifest	 = It.second;
		const std::string&	 Identity		 = FileManifest.RevisionControlIdentity;

		ConvertWideToUtf8(FileManifest.CurrentPath.wstring(), CurrentPathUtf8);

		OutputString.append(Identity);

		if (Format == ERevisionControlFileFormat::P4Have)
		{
			OutputString.append(" - ");
			OutputString.append(CurrentPathUtf8);
		}

		OutputString.append("\n");
	}

	return WriteBufferToFile(OutputPath, OutputString);
}

int32
CmdUnpack(const FCmdUnpackOptions& Options)
{
	UNSYNC_LOG(L"Unpacking snapshot '%hs' to '%ls'", Options.SnapshotName.c_str(), Options.OutputPath.wstring().c_str());
	UNSYNC_LOG_INDENT;

	if (!EnsureDirectoryExists(Options.OutputPath))
	{
		UNSYNC_ERROR(L"Failed to create output directory");
		return -1;
	}

	const FPath& StoreRoot = Options.StorePath;
	const FPath	 PackRoot  = StoreRoot / "pack";

	const FPath ManifestRoot		  = Options.OutputPath / ".unsync";
	const FPath DirectoryManifestPath = ManifestRoot / "manifest.bin";
	const FPath RevisionFilePath	  = ManifestRoot / "revisions.txt";

	FPackDatabase PackDb;
	{
		UNSYNC_LOG(L"Loading block database");
		UNSYNC_LOG_INDENT;

		// TODO: sync index files to local cache for faster unpack next time
		PackDb.Load(PackRoot);
		UNSYNC_LOG("Known pack files: %llu", llu(PackDb.PackFilenames.size()));
		UNSYNC_LOG("Known blocks: %llu", llu(PackDb.BlockMap.size()));
		UNSYNC_LOG("Total compressed size: %llu (%.3f GB)",
				   llu(PackDb.TotalCompressedSize),
				   double(PackDb.TotalCompressedSize) / double(1 << 30));
	}

	// Named/tagged snapshots take precendence over regular snapshots
	FPath TagPath	   = Options.StorePath / "tag" / (Options.SnapshotName + ".unsync_tag");
	FPath SnapshotPath = Options.StorePath / "snapshot" / (Options.SnapshotName + ".unsync_snapshot");

	FBuffer SnapshotBuffer;

	if (PathExists(TagPath))
	{
		UNSYNC_LOG(L"Loading snapshot '%hs' from tag file", Options.SnapshotName.c_str());
		SnapshotBuffer = ReadFileToBuffer(TagPath);
	}
	else if (PathExists(SnapshotPath))
	{
		UNSYNC_LOG(L"Loading snapshot '%hs'", Options.SnapshotName.c_str());
		SnapshotBuffer = ReadFileToBuffer(SnapshotPath);
	}
	else
	{
		UNSYNC_ERROR(L"Could not find snapshot file or named tag '%hs'", Options.SnapshotName.c_str());
		return -1;
	}

	if (SnapshotBuffer.Empty())
	{
		UNSYNC_ERROR(L"Failed read directory snapshot manifest");
		return -1;
	}

	TArrayView<FGenericBlock> ManifestBlocks = ReinterpretView<FGenericBlock>(SnapshotBuffer);

	uint64 ManifestFileSize = 0;
	for (const FGenericBlock& Block : ManifestBlocks)
	{
		if (Block.Offset != ManifestFileSize)
		{
			UNSYNC_FATAL(L"Unexpected block offset found in the snapshot file");
			return -1;
		}
		ManifestFileSize += Block.Size;
	}

	UNSYNC_LOG(L"Reconstructing directory manifest");

	FBuffer ManifestBuffer;
	{
		ManifestBuffer.Resize(ManifestFileSize);
		FMemReaderWriter ManifestWriter(ManifestBuffer);
		if (!BuildTargetFromPack(ManifestWriter, PackDb, ManifestBlocks))
		{
			UNSYNC_FATAL(L"Failed to reconstruct directory manfiest from snapshot");
			return -1;
		}
	}

	FDirectoryManifest NewDirectoryManifest;
	{
		UNSYNC_LOG(L"Parsing directory manifest");

		FMemReader		   ManifestMemReader(ManifestBuffer);
		FIOReaderStream	   ManifestReaderStream(ManifestMemReader);
		if (!LoadDirectoryManifest(NewDirectoryManifest, Options.OutputPath, ManifestReaderStream))
		{
			UNSYNC_FATAL(L"Failed to deserialize directory manfiest from snapshot");
			return -1;
		}
	}

	{
		UNSYNC_LOG(L"Verifying manifest against known block database");
		if (!VerifyManifest(PackDb, NewDirectoryManifest))
		{
			UNSYNC_FATAL(L"Failed to verify manifest");
			return -1;
		}
	}

	if (Options.bOutputFiles)
	{
		UNSYNC_LOG(L"Synchronizing directory from pack");
		{
			UNSYNC_LOG_INDENT;
			if (!SyncDirectoryFromPack(Options.OutputPath, PackDb, NewDirectoryManifest))
			{
				UNSYNC_FATAL(L"Failed to sync directory pack");
				return -1;
			}
		}
	}

	{
		UNSYNC_LOG(L"Saving directory manifest");
		UNSYNC_LOG_INDENT;
		if (!GDryRun && EnsureDirectoryExists(ManifestRoot))
		{
			SaveDirectoryManifest(NewDirectoryManifest, DirectoryManifestPath);
		}
	}

	if (NewDirectoryManifest.bHasFileRevisionControl)
	{
		if (Options.bOutputRevisions)
		{
			UNSYNC_LOG(L"Extracting revision control data to file: '%ls'", RevisionFilePath.wstring().c_str());
			UNSYNC_LOG_INDENT;

			if (!SaveRevisionControlData(RevisionFilePath, NewDirectoryManifest, ERevisionControlFileFormat::IdentityOnly))
			{
				UNSYNC_FATAL(L"Failed to save revision control data");
				return -1;
			}
		}

		if (!Options.P4HaveOutputPath.empty())
		{
			UNSYNC_LOG(L"Extracting revision control data to `p4 have` file '%ls'", Options.P4HaveOutputPath.wstring().c_str());
			UNSYNC_LOG_INDENT;

			if (!SaveRevisionControlData(Options.P4HaveOutputPath, NewDirectoryManifest, ERevisionControlFileFormat::P4Have))
			{
				UNSYNC_FATAL(L"Failed to save revision control data");
				return -1;
			}
		}
	}
	else
	{
		if (!Options.P4HaveOutputPath.empty())
		{
			UNSYNC_ERROR(L"P4 have output is requested, but the manifest does not contain revision control data");
			return -1;
		}

		// Delete previously cached revision control file if current manifest doesn't have revision data
		if (!GDryRun)
		{
			if (PathExists(RevisionFilePath))
			{
				std::error_code ErrorCode;
				if (!FileRemove(RevisionFilePath, ErrorCode))
				{
					UNSYNC_WARNING(L"Failed to delete file '%ls'", RevisionFilePath.wstring().c_str())
				}
			}
		}
	}

	return 0;
}

}  // namespace unsync
