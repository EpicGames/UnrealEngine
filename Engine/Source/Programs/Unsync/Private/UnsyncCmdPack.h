// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"

namespace unsync {

struct FCmdPackOptions
{
	FPath			  RootPath;
	FPath			  P4HavePath;  // optional
	bool			  bRunP4Have = false; // mutually exclusive with P4HavePath
	FPath			  StorePath;   // optional
	uint32			  BlockSize = uint32(64_KB);
	FAlgorithmOptions Algorithm;

	// Optional tag for this snapshot which can be used to reference it later using a human-friendly name.
	// Overwrites the tag if it already exists.
	std::string		  SnapshotName;
};

int32 CmdPack(const FCmdPackOptions& Options);

struct FCmdUnpackOptions
{
	FPath		OutputPath;
	FPath		P4HaveOutputPath;  // extract embedded p4 have data into the specified file
	FPath		StorePath;
	std::string SnapshotName;
	bool		bOutputFiles	 = true;  // set to false to skip unpack process for files (useful for extracting metadata)
	bool		bOutputRevisions = true;  // write [output]/.unsync/revisions.txt if manifest contains revision control data

	// bool			  bRunP4Sync = false; // TODO: run batches of `p4 sync -k` as files get finalized
};

int32 CmdUnpack(const FCmdUnpackOptions& Options);

}
