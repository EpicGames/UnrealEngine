// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdPatch.h"
#include "UnsyncFile.h"
#include "UnsyncTarget.h"

namespace unsync {

int32  // TODO: return a TResult
CmdPatch(const FCmdPatchOptions& Options)
{
	UNSYNC_LOG(L"Patching base file '%ls' using '%ls'", Options.Base.wstring().c_str(), Options.Patch.wstring().c_str());
	UNSYNC_LOG(L"Output file '%ls'", Options.Output.wstring().c_str());

	FBuffer BaseFile = ReadFileToBuffer(Options.Base);
	if (BaseFile.Empty())
	{
		UNSYNC_ERROR(L"Failed to open file '%ls'", Options.Base.wstring().c_str());
		return 1;
	}

	FBuffer PatchFile = ReadFileToBuffer(Options.Patch);
	if (PatchFile.Empty())
	{
		UNSYNC_ERROR(L"Failed to open patch file '%ls'", Options.Patch.wstring().c_str());
		return 1;
	}

	FBuffer TargetData = BuildTargetWithPatch(PatchFile.Data(), PatchFile.Size(), BaseFile.Data(), BaseFile.Size());

	if (TargetData.Empty())
	{
		return 1;
	}

	if (!GDryRun)
	{
		UNSYNC_LOG(L"Writing output file to '%ls'", Options.Output.wstring().c_str());
		return WriteBufferToFile(Options.Output, TargetData) ? 0 : 1;
	}

	return 0;
}

}  // namespace unsync
