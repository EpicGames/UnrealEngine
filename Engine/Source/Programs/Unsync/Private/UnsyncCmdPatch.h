// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"

namespace unsync {

struct FCmdPatchOptions
{
	FPath Base;
	FPath Patch;
	FPath Output;
};

int32 CmdPatch(const FCmdPatchOptions& Options);

}  // namespace unsync
