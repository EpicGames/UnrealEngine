// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdPush.h"
#include "UnsyncCompression.h"
#include "UnsyncCore.h"
#include "UnsyncError.h"
#include "UnsyncHttp.h"
#include "UnsyncFile.h"
#include "UnsyncJupiter.h"
#include "UnsyncLog.h"
#include "UnsyncMiniCb.h"
#include "UnsyncPool.h"
#include "UnsyncSerialization.h"
#include "UnsyncSocket.h"
#include "UnsyncThread.h"
#include "UnsyncUtil.h"

#include <functional>
#include <unordered_set>

namespace unsync {

int32  // TODO: return a TResult
CmdPush(const FCmdPushOptions& Options)
{
	UNSYNC_LOG_INDENT;

	if (Options.Remote.Protocol != EProtocolFlavor::Jupiter)
	{
		UNSYNC_ERROR(L"Push command is only implemented for Jupiter remotes");
		return 1;
	}

	// Default on-demand manifest generation options (used if manifest was not previously generated)

	uint32 BlockSize = uint32(64_KB);

	FAlgorithmOptions AlgorithmOptions;
	AlgorithmOptions.ChunkingAlgorithmId   = EChunkingAlgorithmID::VariableBlocks;
	AlgorithmOptions.WeakHashAlgorithmId   = EWeakHashAlgorithmID::BuzHash;
	AlgorithmOptions.StrongHashAlgorithmId = EStrongHashAlgorithmID::Blake3_128;

	FComputeBlocksParams ComputeBlocksParams;
	ComputeBlocksParams.Algorithm		 = AlgorithmOptions;
	ComputeBlocksParams.BlockSize		 = BlockSize;
	ComputeBlocksParams.bNeedMacroBlocks = true;

	FDirectoryManifest Manifest;
	FPath			   ManifestPath	  = Options.Input / ".unsync" / "manifest.bin";	 // TODO: allow manifest path override
	bool			   bManifestValid = LoadOrCreateDirectoryManifest(Manifest, Options.Input, ComputeBlocksParams);

	if (!bManifestValid)
	{
		UNSYNC_ERROR(L"Failed to load or create directory manifest");
		return 1;
	}

	FDirectoryManifestInfo ManifestInfo			= GetManifestInfo(Manifest);
	FHash160			   ManifestSignature	= ToHash160(ManifestInfo.StableSignature);
	std::string			   ManifestSignatureStr = BytesToHexString(ManifestSignature.Data, sizeof(ManifestSignature.Data));

	LogManifestInfo(ELogLevel::Debug, ManifestInfo);

	if (Options.Remote.Protocol == EProtocolFlavor::Jupiter)
	{
		for (const auto& It : Manifest.Files)
		{
			if (It.second.MacroBlocks.empty() && It.second.Size != 0)
			{
				UNSYNC_ERROR(L"Pushing to Jupiter requires a manifest with file macro blocks.");
				// TODO: perhaps could just always generate the manifest here
				return 1;
			}
		}

		FTlsClientSettings TlsSettings = Options.Remote.GetTlsClientSettings();

		if (Options.Remote.StorageNamespace.empty())
		{
			UNSYNC_ERROR(L"Jupiter remote URL must have a namespace, i.e. https://example.com#my.name.space");
			return 1;
		}

		// Push must run several times, until Jupiter repots empty missing reference list.
		int32		 Result		   = -1;
		const uint32 MaxAttempts   = 5;
		bool		 bPushComplete = false;
		for (uint32 AttemptIndex = 0; AttemptIndex < MaxAttempts; ++AttemptIndex)
		{
			TResult<uint64> PushResult = JupiterPush(Manifest, Options.Remote, TlsSettings);

			if (const uint64* PushedBlocks = PushResult.TryData())
			{
				if (*PushedBlocks == 0)
				{
					bPushComplete = true;
					Result		  = 0;
					break;
				}
			}
			else
			{
				const FError& E = PushResult.GetError();
				if (E.Kind == EErrorKind::Http && (E.Code == 0 || E.Code == 401))
				{
					// No point in retrying if we can't connect to the server or if we get auth error
					break;
				}
				else
				{
					LogError(E);
					UNSYNC_BREAK_ON_ERROR;
					break;
				}
			}
		}

		if (bPushComplete)
		{
			UNSYNC_LOG(L"Push completed successfully", MaxAttempts);
		}
		else
		{
			UNSYNC_ERROR(L"Could not upload all blocks to Jupiter after %d attempts.", MaxAttempts);
		}

		return Result;
	}
	else
	{
		UNSYNC_ERROR(L"Unsupported protocol flavor (%d)", (int)Options.Remote.Protocol);
		return -1;
	}
}

}  // namespace unsync
