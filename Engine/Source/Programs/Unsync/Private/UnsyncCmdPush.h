// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"
#include "UnsyncRemote.h"

namespace unsync {

struct FCmdPushOptions
{
	FPath		Input;
	FRemoteDesc Remote;
};

int32 CmdPush(const FCmdPushOptions& Options);

}  // namespace unsync
