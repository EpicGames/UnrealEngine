// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdQuery.h"
#include "UnsyncFile.h"
#include "UnsyncHttp.h"
#include "UnsyncProxy.h"
#include "UnsyncThread.h"
#include "UnsyncUtil.h"
#include "UnsyncAuth.h"
#include "UnsyncPool.h"
#include "UnsyncScheduler.h"

#include <float.h>
#include <algorithm>
#include <regex>
#include <json11.hpp>

#include <fmt/format.h>

namespace unsync {

using FMirrorInfoResult = TResult<std::vector<FMirrorInfo>>;

// Runs a basic HTTP request against the remote server and returns the time it took to get the response, -1 if connection could not be
// established.
static double
RunHttpPing(std::string_view Address, uint16 Port)
{
	
	FTlsClientSettings TlsSettings;
	TlsSettings.Subject = Address;

	ETlsRequirement TlsRequirement = ETlsRequirement::None;

	if (Port == 443)
	{
		TlsRequirement = ETlsRequirement::Preferred;
	}

	constexpr uint32 RecvTimeoutSeconds = 4;

	FHttpConnection Connection(Address, Port, TlsRequirement, TlsSettings, RecvTimeoutSeconds);

	FHttpRequest	Request;
	Request.Url				   = "/api/v1/ping";
	Request.Method			   = EHttpMethod::GET;

	// Don't time the connection handshake, only query time itself
	if (!Connection.Open())
	{
		return -1;
	}

	FTimePoint	  TimeBegin	   = TimePointNow();
	FHttpResponse PingResponse = HttpRequest(Connection, Request);
	FTimePoint	  TimeEnd	   = TimePointNow();

	if (PingResponse.Success())
	{
		return DurationSec(TimeBegin, TimeEnd);
	}
	else
	{
		return -1;
	}
}

FMirrorInfoResult
RunQueryMirrors(const FRemoteDesc& RemoteDesc)
{
	const char*	  Url	   = "/api/v1/mirrors";
	FHttpResponse Response = HttpRequest(RemoteDesc, EHttpMethod::GET, Url);

	if (!Response.Success())
	{
		return HttpError(Url, Response.Code);
	}

	using namespace json11;
	std::string JsonString = std::string(Response.AsStringView());

	std::string JsonErrorString;
	Json		JsonObject = Json::parse(JsonString, JsonErrorString);

	if (!JsonErrorString.empty())
	{
		return AppError(std::string("JSON parse error while getting server mirrors: ") + JsonErrorString);
	}

	std::vector<FMirrorInfo> Result;

	for (const auto& Elem : JsonObject.array_items())
	{
		FMirrorInfo Info;
		for (const auto& Field : Elem.object_items())
		{
			if (Field.first == "name")
			{
				Info.Name = Field.second.string_value();
			}
			else if (Field.first == "description")
			{
				Info.Description = Field.second.string_value();
			}
			else if (Field.first == "address")
			{
				Info.Address = Field.second.string_value();
			}
			else if (Field.first == "port")
			{
				int PortValue = Field.second.int_value();
				if (PortValue > 0 && PortValue < 65536)
				{
					Info.Port = uint16(PortValue);
				}
				else
				{
					UNSYNC_WARNING(L"Unexpected port value: %d", PortValue);
					Info.Port = 0;
				}
			}
		}
		Result.push_back(Info);
	}

	return ResultOk(Result);
}

int32
CmdQueryMirrors(const FCmdQueryOptions& Options)
{
	FMirrorInfoResult MirrorsResult = RunQueryMirrors(Options.Remote);
	if (MirrorsResult.IsError())
	{
		LogError(MirrorsResult.GetError(), L"Failed to get mirror list from the server");
		return 1;
	}

	std::vector<FMirrorInfo> Mirrors = MirrorsResult.GetData();

	ParallelForEach(Mirrors, [](FMirrorInfo& Mirror) { Mirror.Ping = RunHttpPing(Mirror.Address, Mirror.Port); });

	std::sort(Mirrors.begin(), Mirrors.end(), [](const FMirrorInfo& InA, const FMirrorInfo& InB) {
		double A = InA.Ping > 0 ? InA.Ping : FLT_MAX;
		double B = InB.Ping > 0 ? InB.Ping : FLT_MAX;
		return A < B;
	});

	LogPrintf(ELogLevel::MachineReadable, L"[\n");

	for (size_t I = 0; I < Mirrors.size(); ++I)
	{
		const FMirrorInfo& Mirror = Mirrors[I];

		int32 PingMs = (Mirror.Ping == 0) ? 0 : std::max(1, int32(Mirror.Ping * 1000.0));

		LogPrintf(ELogLevel::MachineReadable,
				  L"  {\"address\":\"%hs\", \"port\":%d, \"ok\":%hs, \"ping\":%d, \"name\":\"%hs\", \"description\":\"%hs\"}%hs\n",
				  StringEscape(Mirror.Address).c_str(),
				  Mirror.Port,
				  Mirror.Ping > 0 ? "true" : "false",
				  PingMs,
				  StringEscape(Mirror.Name).c_str(),
				  StringEscape(Mirror.Description).c_str(),
				  I + 1 == Mirrors.size() ? "" : ",");
	}

	LogPrintf(ELogLevel::MachineReadable, L"]\n");

	return 0;
}

int32
CmdQueryList(const FCmdQueryOptions& Options)
{
	FHttpConnection						Connection	  = FHttpConnection::CreateDefaultHttps(Options.Remote);
	TResult<ProxyQuery::FHelloResponse> HelloResponse = ProxyQuery::Hello(Options.Remote.Protocol, Connection);

	if (HelloResponse.IsError())
	{
		UNSYNC_ERROR("Failed establish a handshake with server '%hs'", Options.Remote.Host.Address.c_str());
		LogError(HelloResponse.GetError());
		return -1;
	}
	FAuthDesc AuthDesc = FAuthDesc::FromHelloResponse(*HelloResponse);


	if (Options.Args.empty())
	{
		UNSYNC_ERROR(L"Path argument is required");
		return -1;
	}

	TResult<ProxyQuery::FDirectoryListing> ListingResult = ProxyQuery::ListDirectory(Options.Remote.Protocol, Connection, &AuthDesc, Options.Args[0]);

	if (ListingResult.IsError())
	{
		LogError(ListingResult.GetError(), L"Failed to list remote directory");
		return -1;
	}

	std::string ListingJson = ListingResult->ToJson();

	LogPrintf(ELogLevel::MachineReadable, L"%hs\n", ListingJson.c_str());

	return 0;
}

int32
CmdQuerySearch(const FCmdQueryOptions& Options)
{
	using namespace ProxyQuery;

	if (Options.Args.empty())
	{
		UNSYNC_ERROR(L"Path argument is required");
		return -1;
	}

	auto CreateConnection = [Remote = Options.Remote]
	{
		FTlsClientSettings TlsSettings = Remote.GetTlsClientSettings();
		return new FHttpConnection(Remote.Host.Address, Remote.Host.Port, Remote.TlsRequirement, TlsSettings, Remote.RecvTimeoutSeconds);
	};

	TObjectPool<FHttpConnection> ConnectionPool(CreateConnection);

	TResult<ProxyQuery::FHelloResponse> HelloResponse = [&ConnectionPool, &Options]
	{
		std::unique_ptr<FHttpConnection> Connection = ConnectionPool.Acquire();
		TResult<ProxyQuery::FHelloResponse> Result = ProxyQuery::Hello(Options.Remote.Protocol, *Connection);
		ConnectionPool.Release(std::move(Connection));
		return Result;
	}();

	if (HelloResponse.IsError())
	{
		UNSYNC_ERROR("Failed establish a handshake with server '%hs'", Options.Remote.Host.Address.c_str());
		LogError(HelloResponse.GetError());
		return -1;
	}

	FAuthDesc AuthDesc = FAuthDesc::FromHelloResponse(*HelloResponse);


	const std::string& RootPath = Options.Args[0];
	std::vector<std::regex> SubdirPatterns;

	UNSYNC_LOG("Searching '%hs'", RootPath.c_str());
	if (Options.Args.size() > 1)
	{
		UNSYNC_LOG_INDENT;
		UNSYNC_LOG("Subdirectory patterns:", RootPath.c_str());
		UNSYNC_LOG_INDENT;
		for (size_t i = 1; i < Options.Args.size(); ++i)
		{
			UNSYNC_LOG("%hs", Options.Args[i].c_str());
			std::regex Pattern = std::regex(Options.Args[i], std::regex_constants::icase);
			SubdirPatterns.push_back(Pattern);
		}
	}

	struct FEntry
	{
		std::string Path;
		uint32		Depth = 0;
	};

	std::vector<FEntry> PendingDirectories;
	{
		FEntry RootEntry;
		RootEntry.Path	= RootPath;
		RootEntry.Depth = 0;
		PendingDirectories.push_back(RootEntry);
	}

	struct FResultEntry : FEntry
	{
		FDirectoryListingEntry DirEntry;
	};

	struct FTaskContext
	{
		std::mutex						Mutex;
		std::vector<FResultEntry>		FoundEntries;
		std::unordered_set<std::string> VisitedDirectories;
		bool							bParentThreadVerbose = false;
		int32							ParentThreadIndent	 = 0;
	};

	FTaskGroup	 Tasks = GScheduler->CreateTaskGroup();
	FTaskContext Context;

	Context.bParentThreadVerbose = GLogVerbose;
	Context.ParentThreadIndent	 = GLogIndent;

	std::function<void(std::string, int32, const FDirectoryListing*)> ExploreDirectory =
		[&Context, &AuthDesc, &ConnectionPool, &ExploreDirectory, &SubdirPatterns, &Tasks, &Options](
			std::string					   Path,
			int32						   CurrentDepth,
			const FDirectoryListing* DirectoryListingPtr)
	{
		FLogVerbosityScope VerboseScope(Context.bParentThreadVerbose);
		FLogIndentScope	   IndentScope(Context.ParentThreadIndent, true);

		FDirectoryListing RemoteDirectoryListing;

		if (!DirectoryListingPtr)
		{
			UNSYNC_VERBOSE2(L"Listing '%hs'", Path.c_str());

			GScheduler->NetworkSemaphore.Acquire(false);
			std::unique_ptr<FHttpConnection> Connection = ConnectionPool.Acquire();

			TResult<FDirectoryListing> DirectoryListingResult =
				ProxyQuery::ListDirectory(Options.Remote.Protocol, *Connection, &AuthDesc, Path);

			ConnectionPool.Release(std::move(Connection));
			GScheduler->NetworkSemaphore.Release();

			if (DirectoryListingResult.IsError())
			{
				LogError(DirectoryListingResult.GetError(), L"Failed to list remote directory");
				return;
			}

			std::swap(RemoteDirectoryListing, DirectoryListingResult.GetData());
			DirectoryListingPtr = &RemoteDirectoryListing;
		}

		const FDirectoryListing& DirectoryListing = *DirectoryListingPtr;

		for (const FDirectoryListingEntry& DirEntry : DirectoryListing.Entries)
		{
			std::string DirEntryName = DirEntry.Name;

			// Only include one subdirectory level
			const size_t SeparatorPos = DirEntryName.find(PATH_SEPARATOR);
			if (SeparatorPos != std::string::npos)
			{
				DirEntryName = DirEntryName.substr(0, PATH_SEPARATOR);
			}

			FEntry NextEntry;
			NextEntry.Path	= Path + PATH_SEPARATOR + DirEntryName;
			NextEntry.Depth = CurrentDepth + 1;

			// Include only leaf directory entries in the final output
			if (SeparatorPos == std::string::npos)
			{
				std::lock_guard<std::mutex> LockGuard(Context.Mutex);

				FResultEntry ResultEntry;
				ResultEntry.Path	 = NextEntry.Path;
				ResultEntry.Depth	 = NextEntry.Depth;
				ResultEntry.DirEntry = DirEntry;

				Context.FoundEntries.push_back(ResultEntry);
			}

			if (NextEntry.Depth > SubdirPatterns.size())
			{
				continue;
			}

			if (DirEntry.bDirectory && std::regex_match(DirEntryName, SubdirPatterns[CurrentDepth], std::regex_constants::match_any))
			{
				UNSYNC_VERBOSE2(L"Matched: '%hs'", DirEntryName.c_str());

				// Directory listing may already include some child sub-directories, so we can skip some network requests
				FDirectoryListing SubDirectoryListing;
				std::string		  RequiredPrefix = DirEntryName + PATH_SEPARATOR;

				for (const FDirectoryListingEntry& DirEntry2 : DirectoryListing.Entries)
				{
					if (DirEntry2.Name.starts_with(RequiredPrefix))
					{
						FDirectoryListingEntry SubDirEntry = DirEntry2;
						SubDirEntry.Name				   = SubDirEntry.Name.substr(RequiredPrefix.length());
						SubDirectoryListing.Entries.push_back(SubDirEntry);
					}
				}

				{
					// Only visit each sub-directory once
					std::lock_guard<std::mutex> LockGuard(Context.Mutex);
					if (!Context.VisitedDirectories.insert(NextEntry.Path).second)
					{
						continue;
					}
				}

				if (SubDirectoryListing.Entries.empty())
				{
					Tasks.run([ExploreDirectory, NextEntry]() { ExploreDirectory(NextEntry.Path, NextEntry.Depth, nullptr); });
				}
				else
				{
					ExploreDirectory(NextEntry.Path, NextEntry.Depth, &SubDirectoryListing);
				}
			}
		}
	};

	ExploreDirectory(RootPath, 0, nullptr);
	Tasks.wait();

	std::vector<FResultEntry>& ResultEntries = Context.FoundEntries;

	std::sort(ResultEntries.begin(), ResultEntries.end(), [](const FResultEntry& A, const FResultEntry& B) { return A.Path < B.Path; });

	LogPrintf(ELogLevel::MachineReadable, L"{\n");
	LogPrintf(ELogLevel::MachineReadable, L"  \"root\": \"%hs\",\n", StringEscape(RootPath).c_str());
	LogPrintf(ELogLevel::MachineReadable, L"  \"entries\": [\n");
	for (size_t i = 0; i < ResultEntries.size(); ++i)
	{
		const FResultEntry& ResultEntry	  = ResultEntries[i];
		const char*			TrailingComma = i + 1 == ResultEntries.size() ? "" : ",";

		std::string_view RelativePath = std::string_view(ResultEntry.Path).substr(RootPath.length() + 1);

		LogPrintf(ELogLevel::MachineReadable,
				  L"    { \"path\": \"%hs\", \"is_directory\": %hs, \"mtime\": %llu, \"size\": %llu }%hs\n",
				  StringEscape(RelativePath).c_str(),
				  ResultEntry.DirEntry.bDirectory ? "true" : "false",
				  llu(ResultEntry.DirEntry.Mtime),
				  llu(ResultEntry.DirEntry.Size),
				  TrailingComma);
	}
	LogPrintf(ELogLevel::MachineReadable, L"  ]\n");
	LogPrintf(ELogLevel::MachineReadable, L"}\n");

	return 0;
}

int32
CmdQueryFile(const FCmdQueryOptions& Options)
{
	if (Options.Args.empty())
	{
		UNSYNC_ERROR(L"Path argument is required");
		return -1;
	}

	// TODO: use a global connection pool and use it for ProxyQuery::DownloadFile too
	FHttpConnection						HelloConnection = FHttpConnection::CreateDefaultHttps(Options.Remote);
	TResult<ProxyQuery::FHelloResponse> HelloResponse	= ProxyQuery::Hello(Options.Remote.Protocol, HelloConnection);
	if (HelloResponse.IsError())
	{
		UNSYNC_ERROR("Failed establish a handshake with server '%hs'", Options.Remote.Host.Address.c_str());
		LogError(HelloResponse.GetError());
		return -1;
	}
	FAuthDesc AuthDesc = FAuthDesc::FromHelloResponse(*HelloResponse);

	HelloConnection.Close();

	TResult<FAuthToken> AuthToken = Authenticate(AuthDesc, Options.Remote.RecvTimeoutSeconds);

	if (!AuthToken.IsOk())
	{
		LogError(AuthToken.GetError(), L"Failed to authenticate");
		return -1;
	}

	UNSYNC_LOG(L"Downloading file: '%hs'", Options.Args[0].c_str());
	UNSYNC_LOG_INDENT;

	std::unique_ptr<FNativeFile> ResultWriter;

	FPath OutputPath = Options.OutputPath;
	if (OutputPath.empty())
	{
		FPath RequestPath = Options.Args[0];
		if (RequestPath.has_filename())
		{
			OutputPath = RequestPath.filename();
			OutputPath = GetAbsoluteNormalPath(OutputPath);
		}
		else
		{
			UNSYNC_ERROR(
				L"Output could not be derived from the request string. "
				L"Use `-o <filename>` command line argument to specify it explicitly.");
			return 1;
		}
	}

	UNSYNC_LOG(L"Output file: '%ls'", OutputPath.wstring().c_str());

	auto OutputCallback = [&ResultWriter, &Options, OutputPath](uint64 Size) -> FIOWriter&
	{
		UNSYNC_LOG(L"Size: %llu bytes (%.3f MB)", llu(Size), SizeMb(Size));
		ResultWriter = std::make_unique<FNativeFile>(OutputPath, EFileMode::CreateWriteOnly, Size);
		return *ResultWriter;
	};

	FHttpConnection Connection = FHttpConnection::CreateDefaultHttps(Options.Remote);
	TResult<> Response = ProxyQuery::DownloadFile(Connection, &AuthDesc, Options.Args[0], OutputCallback);

	if (Response.IsOk())
	{
		UNSYNC_LOG(L"Output written to file '%ls'", OutputPath.wstring().c_str());
	}
	else
	{
		LogError(Response.GetError(), L"Failed to download file");
	}

	return 0;
}

int32
CmdQueryHttpGet(const FCmdQueryOptions& Options)
{
	FHttpConnection HttpConnection = FHttpConnection::CreateDefaultHttps(Options.Remote);

	std::string BearerToken;

	if (Options.Remote.bAuthenticationRequired)
	{
		TResult<ProxyQuery::FHelloResponse> HelloResponse = ProxyQuery::Hello(Options.Remote.Protocol, HttpConnection);
		if (!HelloResponse.IsOk())
		{
			LogError(HelloResponse.GetError(), L"Failed to query basic server information");
			return -1;
		}

		FAuthDesc AuthDesc = FAuthDesc::FromHelloResponse(*HelloResponse);

		TResult<FAuthToken> AuthToken = Authenticate(AuthDesc, Options.Remote.RecvTimeoutSeconds);

		if (!AuthToken.IsOk())
		{
			LogError(AuthToken.GetError(), L"Failed to authenticate with the server");
			return -1;
		}

		BearerToken = AuthToken->Access;
	}

	// TODO: RequestPath should include leading slash
	std::string RequestUrl = fmt::format("/{}", Options.Remote.RequestPath);

	FHttpRequest Request;
	Request.Method		= EHttpMethod::GET;
	Request.BearerToken = BearerToken;
	Request.Url			= RequestUrl;

	FHttpResponse Response = HttpRequest(HttpConnection, Request);

	if (!Response.Success())
	{
		LogError(HttpError(std::move(RequestUrl), Response.Code));
		return -1;
	}

	FPath OutputPath = Options.OutputPath;
	if (OutputPath.empty())
	{
		if (Response.Buffer.Empty())
		{
			return 0;
		}
		else if (Response.ContentType == EHttpContentType::Application_Json || Response.ContentType == EHttpContentType::Text_Plain ||
			Response.ContentType == EHttpContentType::Text_Html)
		{
			Response.Buffer.PushBack('\n');
			Response.Buffer.PushBack(0);
			LogPrintf(ELogLevel::MachineReadable, L"%hs", (const char*)Response.Buffer.Data());
			return 0;
		}
		else
		{
			UNSYNC_ERROR(L"Unexpected response content type. Only plain text or json are supported. Use `-o <filename>` command line argument to write response body to a file.");
			return -1;
		}
	}
	else
	{
		UNSYNC_LOG(L"Output file: '%ls'", OutputPath.wstring().c_str());
		OutputPath = GetAbsoluteNormalPath(OutputPath);

		if (EnsureDirectoryExists(OutputPath.parent_path()))
		{
			if (WriteBufferToFile(OutputPath, Response.Buffer))
			{
				UNSYNC_VERBOSE(L"Wrote bytes: %llu", llu(Response.Buffer.Size()));
				return 0;
			}
			else
			{
				UNSYNC_ERROR(L"Failed to write output file '%ls'", OutputPath.wstring().c_str());
			}
		}
		else
		{
			UNSYNC_ERROR(L"Failed to create output directory '%ls'", OutputPath.parent_path().wstring().c_str());
		}

		return -1;
	}
}

int32
CmdQuery(const FCmdQueryOptions& Options)
{
	if (!Options.Remote.IsValid())
	{
		UNSYNC_ERROR(L"Server address is not specified or is invalid");
		return 1;
	}

	if (Options.Query == "mirrors")
	{
		return CmdQueryMirrors(Options);
	}
	else if (Options.Query == "list")
	{
		return CmdQueryList(Options);
	}
	else if (Options.Query == "search" || Options.Query == "explore")
	{
		return CmdQuerySearch(Options);
	}
	else if (Options.Query == "file")
	{
		return CmdQueryFile(Options);
	}
	if (Options.Query == "http-get")
	{
		return CmdQueryHttpGet(Options);
	}
	else
	{
		UNSYNC_ERROR(L"Unknown query command. Allowed options: mirrors, list, search, file, http-get");
		return 1;
	}
}

TResult<FMirrorInfo>
FindClosestMirror(const FRemoteDesc& Remote)
{
	FMirrorInfoResult MirrorsResult = RunQueryMirrors(Remote);
	if (MirrorsResult.IsError())
	{
		return FError(MirrorsResult.GetError());
	}

	std::vector<FMirrorInfo> Mirrors = MirrorsResult.GetData();
	ParallelForEach(Mirrors, [](FMirrorInfo& Mirror) { Mirror.Ping = RunHttpPing(Mirror.Address, Mirror.Port); });

	std::sort(Mirrors.begin(), Mirrors.end(), [](const FMirrorInfo& InA, const FMirrorInfo& InB) {
		double A = InA.Ping > 0 ? InA.Ping : FLT_MAX;
		double B = InB.Ping > 0 ? InB.Ping : FLT_MAX;
		return A < B;
	});

	for (const FMirrorInfo& Mirror : Mirrors)
	{
		if (Mirror.Ping > 0)
		{
			return ResultOk(Mirror);
		}
	}

	return AppError("No reachable mirror found");
}

}  // namespace unsync
