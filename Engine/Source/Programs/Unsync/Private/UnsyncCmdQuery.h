// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncRemote.h"

#include <string>

namespace unsync {

struct FCmdQueryOptions
{
	std::string				 Query;
	std::vector<std::string> Args;
	FPath					 OutputPath;
	FRemoteDesc				 Remote;
};

int32 CmdQuery(const FCmdQueryOptions& Options);

struct FMirrorInfo
{
	std::string Name;
	std::string Address;
	std::string Description;
	uint16		Port = UNSYNC_DEFAULT_PORT;
	double		Ping = 0;
};

TResult<FMirrorInfo> FindClosestMirror(const FRemoteDesc& Remote);

} // namespace unsync
