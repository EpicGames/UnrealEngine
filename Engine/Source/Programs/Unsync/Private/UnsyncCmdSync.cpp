// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdSync.h"
#include "UnsyncFile.h"
#include "UnsyncProxy.h"
#include "UnsyncFilter.h"

namespace unsync {

int32  // TODO: return a TResult
CmdSync(const FCmdSyncOptions& Options)
{
	auto ResolvePath = [&Options](FPath Path) -> FPath
	{
#if UNSYNC_PLATFORM_WINDOWS
		if (!Path.native().starts_with(L"\\\\"))
		{
			Path = GetUniversalPath(Path);
		}
#endif // UNSYNC_PLATFORM_WINDOWS

		return Options.Filter ? Options.Filter->Resolve(Path) : Path;
	};

	if (Options.Source.empty())
	{
		UNSYNC_ERROR(L"Sync source location is invalid");
		return 1;
	}

	if (Options.Target.empty())
	{
		UNSYNC_ERROR(L"Sync target location is invalid");
		return 1;
	}

	for (const FPath& Path : Options.Overlays)
	{
		if (Path.empty())
		{
			UNSYNC_ERROR(L"Sync overlay location is invalid");
			return 1;
		}
	}

	FProxyPool ProxyPool(Options.Remote, Options.AuthDesc);

	std::error_code ErrorCode	   = {};
	FPath			ResolvedSource = ResolvePath(Options.Source);

	if (Options.Source == ResolvedSource)
	{
		UNSYNC_LOG(L"Sync source: '%ls'", Options.Source.wstring().c_str());
	}
	else
	{
		UNSYNC_LOG(L"Sync source: '%ls' ('%ls')", Options.Source.wstring().c_str(), ResolvedSource.wstring().c_str());
	}

	if (!Options.Filter->SyncIncludedWords.empty())
	{
		UNSYNC_LOG(L"Include filter: ");
		UNSYNC_LOG_INDENT;
		for( const std::wstring& include : Options.Filter->SyncIncludedWords)
		{ 
			UNSYNC_LOG(L" %ls", include.c_str());
		}
	}
	if(!Options.Filter->SyncExcludedWords.empty())
	{
		UNSYNC_LOG(L"Exclude filter: ");
		UNSYNC_LOG_INDENT;
		for( const std::wstring& exclude : Options.Filter->SyncExcludedWords)
		{ 
			UNSYNC_LOG(L"%ls", exclude.c_str());
		}
	}

	bool bSourceFileSystemRequired = true;
	bool bSourcePathExists		   = false;
	bool bSourceIsDirectory		   = false;
	bool bSourceIsManifestHash	   = LooksLikeHash160(Options.Source.native());

	if (ProxyPool.IsValid())
	{
		const FRemoteProtocolFeatures& Features = ProxyPool.GetFeatures();
		if (ProxyPool.RemoteDesc.Protocol == EProtocolFlavor::Horde)
		{
			UNSYNC_LOG(L"Horde server supports direct file, manifest and block download");
			bSourceFileSystemRequired = false;
		}
		else if (Features.bFileDownload && Features.bDirectoryListing)
		{
			UNSYNC_LOG(L"Server supports direct file access");
			bSourceFileSystemRequired = false;
		}
		else if (Features.bManifestDownload && bSourceIsManifestHash)
		{
			UNSYNC_LOG(L"Server supports access by manifest ID");
			bSourceFileSystemRequired = false;
		}
		else
		{
			UNSYNC_VERBOSE2(L"Server does not support direct file access or download by manifest ID. Source file system access is required.");
		}
	}

	if (bSourceFileSystemRequired)
	{
		bSourcePathExists  = PathExists(ResolvedSource, ErrorCode);
		bSourceIsDirectory = bSourcePathExists && unsync::IsDirectory(ResolvedSource);
	}

	std::vector<FPath> ResolvedOverlays;

	if (!Options.Overlays.empty())
	{
		for (const FPath& Entry : Options.Overlays)
		{
			FPath ResolvedEntry = ResolvePath(Entry);
			if (ResolvedEntry == Entry)
			{
				UNSYNC_LOG(L"Sync overlay: '%ls'", ResolvedEntry.wstring().c_str());
			}
			else
			{
				UNSYNC_LOG(L"Sync overlay: '%ls' ('%ls')", Entry.wstring().c_str(), ResolvedEntry.wstring().c_str());
			}
			ResolvedOverlays.push_back(ResolvedEntry);
		}

		if (bSourceFileSystemRequired)
		{
			if (!bSourcePathExists || !bSourceIsDirectory)
			{
				UNSYNC_ERROR(L"Sync overlay option requires sync source to be a directory that exists on disk.");
				return 1;
			}
		}
		
		if (bSourceIsManifestHash)
		{
			UNSYNC_ERROR(L"Sync overlay option is not compatible with sync by manifest ID.");
			return 1;
		}

		if (!Options.SourceManifestOverride.empty())
		{
			UNSYNC_ERROR(L"Sync overlay option is not compatible with manifest override.");
			return 1;
		}
	}

	UNSYNC_LOG(L"Sync target: '%ls'", Options.Target.wstring().c_str());

	if (!Options.SourceManifestOverride.empty())
	{
		UNSYNC_LOG(L"Manifest override: %ls", Options.SourceManifestOverride.wstring().c_str());

		if (Options.Remote.IsValid() && !Options.bFullSourceScan)
		{
			bSourceFileSystemRequired = false;
		}
	}

	UNSYNC_VERBOSE(L"Source directory access is %ls", bSourceFileSystemRequired ? L"required" : L"NOT required");

	if (bSourcePathExists || !bSourceFileSystemRequired)
	{
		if (!bSourceFileSystemRequired || bSourceIsDirectory)
		{
			if (bSourceIsDirectory)
			{
				UNSYNC_LOG(L"'%ls' is a directory", Options.Source.wstring().c_str());
			}
			else
			{
				UNSYNC_LOG(L"Assuming '%ls' is a directory", Options.Source.wstring().c_str());
			}

			FSyncDirectoryOptions SyncOptions;

			if (bSourceFileSystemRequired)
			{
				SyncOptions.SourceType = ESourceType::FileSystem;
			}
			else if (bSourceIsManifestHash || ProxyPool.RemoteDesc.Protocol == EProtocolFlavor::Horde)
			{
				SyncOptions.SourceType = ESourceType::ServerWithManifestId;
			}
			else
			{
				SyncOptions.SourceType = ESourceType::Server;
			}

			SyncOptions.Source					   = ResolvedSource;
			SyncOptions.Base					   = Options.Target;  // read base data from existing target
			SyncOptions.Target					   = Options.Target;
			SyncOptions.ScavengeRoot			   = Options.ScavengeRoot;
			SyncOptions.ScavengeDepth			   = Options.ScavengeDepth;
			SyncOptions.Overlays				   = ResolvedOverlays;
			SyncOptions.SourceManifestOverride	   = Options.SourceManifestOverride;
			SyncOptions.ProxyPool				   = &ProxyPool;
			SyncOptions.SyncFilter				   = Options.Filter;
			SyncOptions.bCleanup				   = Options.bCleanup;
			SyncOptions.bValidateSourceFiles	   = Options.bFullSourceScan;
			SyncOptions.bFullDifference			   = Options.bFullDifference;
			SyncOptions.bValidateTargetFiles	   = Options.bValidateTargetFiles;
			SyncOptions.bCheckAvailableSpace	   = Options.bCheckAvailableSpace;
			SyncOptions.BackgroundTaskMemoryBudget = Options.BackgroundTaskMemoryBudget;

			return SyncDirectory(SyncOptions) ? 0 : 1;
		}
		else
		{
			UNSYNC_LOG(L"'%ls' is a file", Options.Source.wstring().c_str());

			FSyncFileOptions SyncFileOptions;
			SyncFileOptions.Algorithm			 = Options.Algorithm;
			SyncFileOptions.BlockSize			 = uint32(64_KB);
			SyncFileOptions.bValidateTargetFiles = Options.bValidateTargetFiles;
			SyncFileOptions.SourceType			 = ESourceType::FileSystem;

			return SyncFile(Options.Source, Options.Target, Options.Target, SyncFileOptions).Succeeded() ? 0 : 1;
		}
	}
	else
	{
		if (ErrorCode)
		{
			UNSYNC_ERROR(L"System error code %d: %hs", ErrorCode.value(), ErrorCode.message().c_str());
			return ErrorCode.value();
		}
		else
		{
			UNSYNC_ERROR(L"Source path '%ls' does not exist", ResolvedSource.wstring().c_str());
			return 1;
		}
	}

	return 0;
}

}  // namespace unsync
