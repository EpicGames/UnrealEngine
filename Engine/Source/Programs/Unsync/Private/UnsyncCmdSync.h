// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"
#include "UnsyncRemote.h"

namespace unsync {

struct FAuthDesc;

struct FCmdSyncOptions
{
	FAlgorithmOptions Algorithm;

	FPath Source;
	FPath Target;
	FPath SourceManifestOverride;
	FPath ScavengeRoot;
	uint32 ScavengeDepth = 5;
	uint64 BackgroundTaskMemoryBudget = 2_GB;

	std::vector<FPath> Overlays;

	FRemoteDesc Remote;
	FAuthDesc*	AuthDesc = nullptr;

	bool bFullDifference = false;
	bool bFullSourceScan = false;
	bool bCleanup		 = false;

	bool bCheckAvailableSpace = true;

	bool bValidateTargetFiles = true;  // WARNING: turning this off is intended only for testing/profiling

	FSyncFilter* Filter = nullptr;
};

int32 CmdSync(const FCmdSyncOptions& Options);

}  // namespace unsync
