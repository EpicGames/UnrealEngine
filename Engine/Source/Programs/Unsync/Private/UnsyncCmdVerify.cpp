// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCmdVerify.h"
#include "UnsyncFile.h"
#include "UnsyncLog.h"
#include "UnsyncManifest.h"
#include "UnsyncSerialization.h"

namespace unsync {

int32
CmdVerify(const FCmdVerifyOptions& Options)
{
	FPath			   Root					 = Options.Input;
	FPath			   ManifestRoot			 = Root / ".unsync";
	FPath			   DirectoryManifestPath = ManifestRoot / "manifest.bin";
	FDirectoryManifest DirectoryManifest;

	UNSYNC_LOG(L"Validating manifest for directory '%ls'", Root.wstring().c_str());
	UNSYNC_LOG_INDENT;

	if (!PathExists(DirectoryManifestPath))
	{
		UNSYNC_ERROR(L"Directory does not have a manifest file");
		return -1;
	}

	const bool bExistingManifestLoaded = LoadDirectoryManifest(DirectoryManifest, Root, DirectoryManifestPath);
	if (!bExistingManifestLoaded)
	{
		UNSYNC_ERROR(L"Failed to load manifest file '%ls'", DirectoryManifestPath.wstring().c_str());
		return -1;
	}

	const EStrongHashAlgorithmID StrongHashAlgorithmId = DirectoryManifest.Algorithm.StrongHashAlgorithmId;

	bool bDirectoryValid = true;

	for (const auto& FileIt : DirectoryManifest.Files)
	{
		const std::wstring&	 FileName	  = FileIt.first;
		const FFileManifest& FileManifest = FileIt.second;

		UNSYNC_VERBOSE(L"Verifying file '%ls'", FileName.c_str());

		const FFileAttributes FileAttrib = GetFileAttrib(FileManifest.CurrentPath);
		if (!FileAttrib.bValid)
		{
			bDirectoryValid = false;
			UNSYNC_ERROR(L"File '%ls' does not exist", FileName.c_str());
			continue;
		}

		if (FileAttrib.Size != FileManifest.Size)
		{
			bDirectoryValid = false;
			UNSYNC_ERROR(L"File '%ls' size mismatch. Expected %llu, actual %llu.",
						 FileName.c_str(),
						 llu(FileManifest.Size),
						 llu(FileAttrib.Size));
			continue;
		}

		if (FileAttrib.Mtime != FileManifest.Mtime)
		{
			bDirectoryValid = false;
			UNSYNC_ERROR(L"File '%ls' timestamp mismatch. Expected %llu, actual %llu.",
						 FileName.c_str(),
						 llu(FileManifest.Mtime),
						 llu(FileAttrib.Mtime));
			continue;
		}

		FNativeFile File(FileManifest.CurrentPath, EFileMode::ReadOnlyUnbuffered);
		if (!File.IsValid())
		{
			bDirectoryValid = false;
			UNSYNC_ERROR(L"Failed to open file '%ls'. Error code: %d", FileName.c_str(), File.GetError());
			continue;
		}

		const bool bFileValid = ValidateTarget(File, FileManifest.Blocks, StrongHashAlgorithmId);

		if (!bFileValid)
		{
			UNSYNC_ERROR(L"Validation failed for file '%ls'", FileName.c_str());
			bDirectoryValid = false;
		}
	}

	if (bDirectoryValid)
	{
		UNSYNC_LOG(L"Directory manifest is valid");
		return 0;
	}
	else
	{
		UNSYNC_LOG(L"Directory manifest is invalid");
		return -1;
	}
}

}  // namespace unsync
