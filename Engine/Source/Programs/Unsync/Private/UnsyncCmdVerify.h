// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCommon.h"

namespace unsync {

struct FCmdVerifyOptions
{
	FPath			   Input;
};

int32 CmdVerify(const FCmdVerifyOptions& Options);

}  // namespace unsync
