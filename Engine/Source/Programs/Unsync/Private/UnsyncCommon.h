// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(UNSYNC_COMPILER_MSVC) && defined(_MSC_VER) && !defined(__clang__)
#	define UNSYNC_COMPILER_MSVC 1
#elif !defined(UNSYNC_COMPILER_MSVC)
#	define UNSYNC_COMPILER_MSVC 0
#endif

#if UNSYNC_COMPILER_MSVC
#	define UNSYNC_BREAK __debugbreak
#else
#	define UNSYNC_BREAK __builtin_trap
#endif

#if UNSYNC_COMPILER_MSVC
#	pragma warning(disable : 4100)	 // unreferenced formal parameter
#	define UNSYNC_ATTRIB_FORCEINLINE [[msvc::forceinline]]
#	define UNSYNC_ATTRIB_NOINLINE	  __declspec(noinline)
#	define UNSYNC_THIRD_PARTY_INCLUDES_START \
		__pragma(warning(push)) \
		__pragma(warning(disable : 4458)) \
		__pragma(warning(disable : 4668)) /* undefined macro, replaced with 0 */ \
		__pragma(warning(disable : 4996)) /* deprecated */
#	define UNSYNC_THIRD_PARTY_INCLUDES_END __pragma(warning(pop))
#else
#	define UNSYNC_ATTRIB_FORCEINLINE __attribute__((always_inline))
#	define UNSYNC_ATTRIB_NOINLINE	  __attribute__((noinline))
#	define UNSYNC_THIRD_PARTY_INCLUDES_START \
		_Pragma("GCC diagnostic push") \
		_Pragma("GCC diagnostic ignored \"-Wdeprecated-declarations\"") \
		_Pragma("GCC diagnostic ignored \"-Wunknown-pragmas\"") \
		_Pragma("GCC diagnostic ignored \"-Wshadow\"")
#	define UNSYNC_THIRD_PARTY_INCLUDES_END _Pragma("GCC diagnostic pop")
#endif

UNSYNC_THIRD_PARTY_INCLUDES_START
#include <stdint.h>
#include <cstring>
#include <string_view>
#include <filesystem>
UNSYNC_THIRD_PARTY_INCLUDES_END

namespace unsync {

using llu = long long unsigned;

using uint8	 = uint8_t;
using uint16 = uint16_t;
using uint32 = uint32_t;
using uint64 = uint64_t;

using int8	= int8_t;
using int16 = int16_t;
using int32 = int32_t;
using int64 = int64_t;

using FPath = std::filesystem::path;
using FPathStringView = std::basic_string_view<FPath::value_type>;

static const char PATH_SEPARATOR = char(FPath::preferred_separator);

#define UNSYNC_DISALLOW_COPY_ASSIGN(T) \
	T(const T&) = delete;              \
	T& operator=(const T&) = delete;

#define UNSYNC_CONCAT1(X, Y) X##Y
#define UNSYNC_CONCAT(X, Y)	 UNSYNC_CONCAT1(X, Y)

#define UNSYNC_WSTR(x) L## #x

#define UNSYNC_UNUSED(x) (void)(x)

template<typename T>
void
ClobberT(T& Object)
{
	std::memset(&Object, 0xEE, sizeof(Object));
}

#define UNSYNC_CLOBBER(Object) ClobberT(Object)

#define UNSYNC_ENUM_CLASS_FLAGS(T, S)                      \
	inline constexpr T operator^(T a, T b)                 \
	{                                                      \
		return (T)(static_cast<S>(a) ^ static_cast<S>(b)); \
	}                                                      \
	inline constexpr T operator&(T a, T b)                 \
	{                                                      \
		return (T)(static_cast<S>(a) & static_cast<S>(b)); \
	}                                                      \
	inline constexpr T operator|(T a, T b)                 \
	{                                                      \
		return (T)(static_cast<S>(a) | static_cast<S>(b)); \
	}                                                      \
	inline constexpr bool operator!(T a)                   \
	{                                                      \
		return !static_cast<S>(a);                         \
	}                                                      \
	inline constexpr bool operator==(T a, S b)             \
	{                                                      \
		return static_cast<S>(a) == b;                     \
	}                                                      \
	inline constexpr bool operator!=(T a, S b)             \
	{                                                      \
		return static_cast<S>(a) != b;                     \
	}

template<typename Enum>
constexpr bool
EnumHasAllFlags(Enum Flags, Enum Contains)
{
	return (Flags & Contains) == Contains;
}

template<typename Enum>
constexpr bool
EnumHasAnyFlags(Enum Flags, Enum Contains)
{
	return (Flags & Contains) != 0;
}

}  // namespace unsync
