// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCompression.h"
#include <zstd.h>

namespace unsync {

uint64
GetMaxCompressedSize(uint64 RawSize)
{
	return ZSTD_compressBound(RawSize);
}

uint64
CompressInto(FBufferView Input, FMutBufferView Output, int ZstdCompressionLevel)
{
	const uint64 ExpectedMaxCompressedSize = GetMaxCompressedSize(Input.Size);
	if (Output.Size < ExpectedMaxCompressedSize)
	{
		UNSYNC_FATAL(L"Compressed output buffer is too small");
		return 0;
	}

	uint64 ActualCompressedSize = ZSTD_compress(Output.Data, Output.Size, Input.Data, Input.Size, ZstdCompressionLevel);
	if (ZSTD_isError(ActualCompressedSize))
	{
		const char* ZstdError = ZSTD_getErrorName(ActualCompressedSize);
		UNSYNC_ERROR(L"ZSTD_compress failed: %hs", ZstdError);
		return 0;
	}

	return ActualCompressedSize;
}

FBuffer
Compress(const uint8* Data, uint64 DataSize, int ZstdCompressionLevel)
{
	FBuffer Result;
	if (DataSize)
	{
		Result.Resize(GetMaxCompressedSize(DataSize));

		uint64 CompressedSize = ZSTD_compress(&Result[0], Result.Size(), Data, DataSize, ZstdCompressionLevel);
		if (ZSTD_isError(CompressedSize))
		{
			const char* ZstdError = ZSTD_getErrorName(CompressedSize);
			UNSYNC_ERROR(L"ZSTD_compress failed: %hs", ZstdError);
		}
		Result.Resize(CompressedSize);
	}
	return Result;
}

bool
Decompress(const uint8* InputData, uint64 InputDataSize, uint8* OutputData, uint64 OutputDataSize)
{
	uint64 DecompressedSizeExpected = ZSTD_getFrameContentSize(InputData, InputDataSize);
	if (OutputDataSize >= DecompressedSizeExpected)
	{
		uint64 DecompressedSizeActual = ZSTD_decompress(OutputData, OutputDataSize, InputData, InputDataSize);
		return DecompressedSizeActual == OutputDataSize;
	}

	return false;
}

FBuffer
Decompress(const uint8* Data, uint64 DataSize)
{
	uint64	DecompressedSizeExpected = ZSTD_getFrameContentSize(Data, DataSize);
	FBuffer Result(DecompressedSizeExpected);

	if (DecompressedSizeExpected == 0)
	{
		UNSYNC_ERROR(L"Compressed zstd frame is invalid!");
		return Result;
	}

	uint64 DecompressedSizeActual = ZSTD_decompress(Result.Data(), DecompressedSizeExpected, Data, DataSize);
	if (DecompressedSizeExpected != DecompressedSizeActual)
	{
		UNSYNC_ERROR(L"Failed to decompress zstd frame. DecompressedSizeExpected=%llu, DecompressedSizeActual=%llu.",
					 llu(DecompressedSizeExpected),
					 llu(DecompressedSizeActual));

		Result.Clear();
	}
	return Result;
}

FBuffer
Decompress(const FBuffer& Buffer)
{
	return Decompress(Buffer.Data(), Buffer.Size());
}

bool Decompress(FBufferView Input, FMutBufferView Output)
{
	return Decompress(Input.Data, Input.Size, Output.Data, Output.Size);
}

}  // namespace unsync
