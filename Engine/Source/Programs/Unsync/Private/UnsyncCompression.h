// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncBuffer.h"
#include "UnsyncUtil.h"

namespace unsync {

uint64 GetMaxCompressedSize(uint64 RawSize);

FBuffer Compress(const uint8* Data, uint64 DataSize, int ZstdCompressionLevel = 3);

// Compress input into pre-allocated buffer that's at least as large as indicated by GetMaxCompressedSize()
// Returns compressed data actual size.
uint64 CompressInto(FBufferView Input, FMutBufferView Output, int ZstdCompressionLevel = 3);

FBuffer Decompress(const uint8* Data, uint64 DataSize);
FBuffer Decompress(const FBuffer& Buffer);
bool	Decompress(const uint8* InputData, uint64 InputDataSize, uint8* OutputData, uint64 OutputDataSize);
bool	Decompress(FBufferView Input, FMutBufferView Output);

}  // namespace unsync
