// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCommon.h"

#include "UnsyncChunking.h"
#include "UnsyncCompression.h"
#include "UnsyncCore.h"
#include "UnsyncDiff.h"
#include "UnsyncFile.h"
#include "UnsyncHashTable.h"
#include "UnsyncHttp.h"
#include "UnsyncProgress.h"
#include "UnsyncProxy.h"
#include "UnsyncScan.h"
#include "UnsyncScavenger.h"
#include "UnsyncScheduler.h"
#include "UnsyncSerialization.h"
#include "UnsyncTarget.h"
#include "UnsyncThread.h"
#include "UnsyncUtil.h"
#include "UnsyncVersion.h"
#include "UnsyncFilter.h"
#include "UnsyncSource.h"
#include "UnsyncPack.h"

#include <condition_variable>
#include <filesystem>
#include <mutex>
#include <string>
#include <unordered_map>
#include <unordered_set>
#include <optional>

UNSYNC_THIRD_PARTY_INCLUDES_START
#include <blake3.h>
#include <md5-sse2.h>
UNSYNC_THIRD_PARTY_INCLUDES_END

namespace unsync {

bool GDryRun = false;
bool GExperimental = false;
bool GExperimentalStreaming = false;

FBlock128
ToBlock128(const FGenericBlock& GenericBlock)
{
	FBlock128 Result;
	Result.HashStrong = GenericBlock.HashStrong.ToHash128();
	Result.HashWeak	  = GenericBlock.HashWeak;
	Result.Offset	  = GenericBlock.Offset;
	Result.Size		  = GenericBlock.Size;
	return Result;
}

std::vector<FBlock128>
ToBlock128(FGenericBlockArray& GenericBlocks)
{
	std::vector<FBlock128> Result;
	Result.reserve(GenericBlocks.size());

	for (const FGenericBlock& It : GenericBlocks)
	{
		Result.push_back(ToBlock128(It));
	}

	return Result;
}

std::vector<FCopyCommand>
OptimizeNeedList(const std::vector<FNeedBlock>& Input, uint64 MaxMergedBlockSize)
{
	std::vector<FCopyCommand> Result;
	Result.reserve(Input.size());
	for (const FNeedBlock& Block : Input)
	{
		FCopyCommand Cmd;
		Cmd.SourceOffset = Block.SourceOffset;
		Cmd.TargetOffset = Block.TargetOffset;
		Cmd.Size		 = Block.Size;
		Result.push_back(Cmd);
	}

	std::sort(Result.begin(), Result.end(), FCopyCommand::FCompareBySourceOffset());

	for (uint64 I = 1; I < Result.size(); ++I)
	{
		FCopyCommand& PrevBlock = Result[I - 1];
		FCopyCommand& ThisBlock = Result[I];
		if (PrevBlock.SourceOffset + PrevBlock.Size == ThisBlock.SourceOffset &&
			PrevBlock.TargetOffset + PrevBlock.Size == ThisBlock.TargetOffset && PrevBlock.Size + ThisBlock.Size <= MaxMergedBlockSize)
		{
			ThisBlock.SourceOffset = PrevBlock.SourceOffset;
			ThisBlock.TargetOffset = PrevBlock.TargetOffset;
			ThisBlock.Size += PrevBlock.Size;
			UNSYNC_ASSERT(ThisBlock.Size <= MaxMergedBlockSize);
			PrevBlock.Size = 0;
		}
	}

	for (uint64 I = 0; I < Result.size(); ++I)
	{
		UNSYNC_ASSERT(Result[I].Size <= MaxMergedBlockSize);
	}

	auto It = std::remove_if(Result.begin(), Result.end(), [](const FCopyCommand& Block) { return Block.Size == 0; });

	Result.erase(It, Result.end());

	return Result;
}

FReadSchedule
BuildReadSchedule(const std::vector<FNeedBlock>& Blocks)
{
	FReadSchedule Result;
	Result.Blocks = OptimizeNeedList(Blocks);
	std::sort(Result.Blocks.begin(), Result.Blocks.end(), [](const FCopyCommand& A, const FCopyCommand& B) {
		if (A.Size == B.Size)
		{
			return A.SourceOffset < B.SourceOffset;
		}
		else
		{
			return A.Size < B.Size;
		}
	});

	for (uint64 I = 0; I < Result.Blocks.size(); ++I)
	{
		Result.Requests.push_back(I);
	}

	return Result;
}

bool
IsSynchronized(const FNeedList& NeedList, const FGenericBlockArray& SourceBlocks)
{
	if (NeedList.Source.size() != 0)
	{
		return false;
	}

	if (NeedList.Base.size() != SourceBlocks.size())
	{
		return false;
	}

	if (NeedList.Sequence.size() != SourceBlocks.size())
	{
		return false;
	}

	for (uint64 I = 0; I < SourceBlocks.size(); ++I)
	{
		if (NeedList.Sequence[I] != SourceBlocks[I].HashStrong.ToHash128())	 // #wip-widehash
		{
			return false;
		}
	}

	return true;
}

bool
ValidateTarget(FIOReader& Reader, const FNeedList& NeedList, EStrongHashAlgorithmID StrongHasher)
{
	FGenericBlockArray ValidationBlocks;
	for (const FNeedBlock& It : NeedList.Source)
	{
		FGenericBlock Block;
		Block.Size		 = CheckedNarrow(It.Size);
		Block.Offset	 = It.TargetOffset;
		Block.HashStrong = It.Hash;
		ValidationBlocks.push_back(Block);
	}
	for (const FNeedBlock& It : NeedList.Base)
	{
		FGenericBlock Block;
		Block.Size		 = CheckedNarrow(It.Size);
		Block.Offset	 = It.TargetOffset;
		Block.HashStrong = It.Hash;
		ValidationBlocks.push_back(Block);
	}

	std::sort(ValidationBlocks.begin(),
			  ValidationBlocks.end(),
			  [](const FGenericBlock& A, const FGenericBlock& B) { return A.Offset < B.Offset; });

	return ValidateTarget(Reader, ValidationBlocks, StrongHasher);
}

bool
ValidateTarget(FIOReader& Reader, const FGenericBlockArray& ValidationBlocks, EStrongHashAlgorithmID StrongHasher)
{
	const uint64		TotalStreamBytes = Reader.GetSize();
	std::atomic<uint64> NumInvalidBlocks = {};
	FSchedulerSemaphore IoSemaphore(*GScheduler, 16);
	FTaskGroup			TaskGroup		 = GScheduler->CreateTaskGroup(&IoSemaphore);

	FLogProgressScope ValidationProgressLogger(TotalStreamBytes, ELogProgressUnits::MB);

	// Inherit verbosity and indentation from parent theread
	// TODO: make a helper that sets verbosity and indentation automatically
	const bool	 bLogVerbose = GLogVerbose;
	const uint32 LogIndent	 = GLogIndent;

	uint64 MaxBatchSizeBytes = 8_MB;

	uint64 BatchBegin	  = 0;
	uint64 BatchSizeBytes = 0;

	std::unique_ptr<FAsyncReader> AsyncReader = Reader.CreateAsyncReader();

	for (uint64 BlockIndex = 0; BlockIndex < ValidationBlocks.size(); ++BlockIndex)
	{
		const FGenericBlock& CurrBlock = ValidationBlocks[BlockIndex];

		if (BlockIndex > 0)
		{
			const FGenericBlock& PrevBlock = ValidationBlocks[BlockIndex - 1];
			if (PrevBlock.Offset + PrevBlock.Size != CurrBlock.Offset)
			{
				UNSYNC_ERROR(L"Found block at unexpected offset");
				return false;
			}
		}

		BatchSizeBytes += CurrBlock.Size;

		if (BlockIndex + 1 < ValidationBlocks.size() && BatchSizeBytes + ValidationBlocks[BlockIndex + 1].Size < MaxBatchSizeBytes)
		{
			continue;
		}

		UNSYNC_ASSERT(BatchSizeBytes <= MaxBatchSizeBytes || BatchBegin == BlockIndex);

		const uint64 ReadOffset = ValidationBlocks[BatchBegin].Offset;
		UNSYNC_ASSERT(BlockIndex + 1 == ValidationBlocks.size() ||
					  (ReadOffset + BatchSizeBytes) == ValidationBlocks[BlockIndex + 1].Offset);

		auto ReadCallback = [StrongHasher,
							 bLogVerbose,
							 LogIndent,
							 BatchBegin,
							 BatchEnd = BlockIndex + 1,
							 BatchSizeBytes,
							 &NumInvalidBlocks,
							 &TaskGroup,
							 &ValidationProgressLogger,
							 &ValidationBlocks](FIOBuffer CmdBuffer, uint64 CmdSourceOffset, uint64 CmdReadSize, uint64 CmdUserData) {
			if (CmdReadSize != BatchSizeBytes)
			{
				UNSYNC_ERROR(L"Expected to read %lld bytes, but read %lld", BatchSizeBytes, CmdReadSize);
				NumInvalidBlocks++;
				return;
			}

			TaskGroup.run([CmdBuffer = std::make_shared<FIOBuffer>(std::move(CmdBuffer)),
						   BatchBegin,
						   BatchEnd,
						   StrongHasher,
						   bLogVerbose,
						   LogIndent,
						   &NumInvalidBlocks,
						   &ValidationProgressLogger,
						   &ValidationBlocks]() {
				FLogIndentScope	   IndentScope(LogIndent, true);
				FLogVerbosityScope VerbosityScope(bLogVerbose);

				const uint64 FirstBlockOffset = ValidationBlocks[BatchBegin].Offset;
				for (uint64 I = BatchBegin; I < BatchEnd; ++I)
				{
					const FGenericBlock& Block			   = ValidationBlocks[I];
					const uint64		 BlockBufferOffset = Block.Offset - FirstBlockOffset;
					FGenericHash		 Hash = ComputeHash(CmdBuffer->GetData() + BlockBufferOffset, Block.Size, StrongHasher);
					if (Hash != Block.HashStrong)
					{
						UNSYNC_ERROR(L"Found block hash mismatch at offset %llu", llu(BlockBufferOffset));
						NumInvalidBlocks++;
						return;
					}

					ValidationProgressLogger.Add(Block.Size);
				}
			});
		};

		AsyncReader->EnqueueRead(ReadOffset, BatchSizeBytes, 0, ReadCallback);

		if (NumInvalidBlocks)
		{
			break;
		}

		BatchSizeBytes = 0;
		BatchBegin	   = BlockIndex + 1;
	}

	AsyncReader->Flush();
	TaskGroup.wait();

	ValidationProgressLogger.Complete();

	return NumInvalidBlocks == 0;
}

static FBuildTargetParams
GetBuildTargetParams(const FSyncFileOptions& Options)
{
	FBuildTargetParams Result;

	Result.StrongHasher		= Options.Algorithm.StrongHashAlgorithmId;
	Result.ProxyPool		= Options.ProxyPool;
	Result.BlockCache		= Options.BlockCache;
	Result.ScavengeDatabase = Options.ScavengeDatabase;

	if (IsFileSystemSource(Options.SourceType))
	{
		Result.SourceType = FBuildTargetParams::ESourceType::File;
	}
	else
	{
		Result.SourceType = FBuildTargetParams::ESourceType::Server;
	}
	

	return Result;
}

FFileSyncResult&
FFileSyncResult::operator+=(const FBuildTargetResult& BuildTargetResult)
{
	SourceBytes += BuildTargetResult.SourceBytes;
	BaseBytes += BuildTargetResult.BaseBytes;
	InSituBytes += BuildTargetResult.InSituBytes;
	return *this;
}

FFileSyncResult
SyncFile(const FNeedList&		   NeedList,
		 const FPath&			   SourceFilePath,
		 const FGenericBlockArray& SourceBlocks,
		 FIOReader&				   BaseDataReader,
		 const FPath&			   TargetFilePath,
		 const FSyncFileOptions&   Options)
{
	UNSYNC_LOG_INDENT;

	FFileSyncResult Result;

	if (Options.SourceType == ESourceType::Unknown)
	{
		Result.Status = EFileSyncStatus::ErrorInvalidParameters;
		UNSYNC_ERROR(L"Sync source type must be specified");
		return Result;
	}

	uint64 NeedFromSource = ComputeSize(NeedList.Source);
	uint64 NeedFromBase	  = ComputeSize(NeedList.Base);
	UNSYNC_VERBOSE(L"Need from source %.2f MB, from base: %.2f MB", SizeMb(NeedFromSource), SizeMb(NeedFromBase));

	const FFileAttributes TargetFileAttributes = GetFileAttrib(TargetFilePath);

	if (!TargetFileAttributes.bValid && NeedList.Sequence.empty())
	{
		UNSYNC_VERBOSE(L"Creating empty file '%ls'", TargetFilePath.wstring().c_str());

		if (GDryRun)
		{
			Result.Status = EFileSyncStatus::Ok;
		}
		else
		{
			FPath TargetFileParent = TargetFilePath.parent_path();
			if (!PathExists(TargetFileParent))
			{
				CreateDirectories(TargetFileParent);
			}

			auto TargetFile = FNativeFile(TargetFilePath, EFileMode::CreateWriteOnly, 0);
			if (TargetFile.IsValid())
			{
				Result.Status = EFileSyncStatus::Ok;
			}
			else
			{
				Result.Status		   = EFileSyncStatus::ErrorTargetFileCreate;
				Result.SystemErrorCode = std::error_code(TargetFile.GetError(), std::system_category());
			}
		}
	}
	else if (!IsSynchronized(NeedList, SourceBlocks))
	{
		LogStatus(TargetFilePath.wstring().c_str(), L"Initializing");

		FPath TempTargetFilePath = TargetFilePath;
		TempTargetFilePath.replace_extension(TargetFilePath.extension().wstring() + L".tmp");
		const FNeedListSize TargetFileSizeInfo = ComputeNeedListSize(NeedList);

		FBuffer							 TargetFileBuffer;
		std::unique_ptr<FIOReaderWriter> TargetFile;
		if (GDryRun)
		{
			if (Options.bValidateTargetFiles)
			{
				TargetFileBuffer.Resize(TargetFileSizeInfo.TotalBytes);
				TargetFile = std::make_unique<FMemReaderWriter>(TargetFileBuffer.Data(), TargetFileBuffer.Size());
			}
			else
			{
				TargetFile = std::make_unique<FNullReaderWriter>(TargetFileSizeInfo.TotalBytes);
			}
		}
		else
		{
			FPath TargetFileParent = TempTargetFilePath.parent_path();
			if (!PathExists(TargetFileParent))
			{
				CreateDirectories(TargetFileParent);
			}

			TargetFile = std::make_unique<FNativeFile>(TempTargetFilePath, EFileMode::CreateWriteOnly, TargetFileSizeInfo.TotalBytes);
			if (TargetFile->GetError() != 0)
			{
				UNSYNC_FATAL(L"Failed to create output file '%ls'. %hs",
							 TempTargetFilePath.wstring().c_str(),
							 FormatSystemErrorMessage(TargetFile->GetError()).c_str());
			}
		}

		LogStatus(TargetFilePath.wstring().c_str(), L"Patching");

		FDeferredOpenReader SourceFile(
			[SourceFilePath, TargetFilePath, Options]() -> std::unique_ptr<FIOReader>
			{
				if (IsFileSystemSource(Options.SourceType))
				{
					UNSYNC_VERBOSE(L"Opening source file '%ls'", SourceFilePath.wstring().c_str());
					LogStatus(TargetFilePath.wstring().c_str(), L"Opening source file");
					return std::unique_ptr<FNativeFile>(new FNativeFile(SourceFilePath, EFileMode::ReadOnlyUnbuffered));
				}
				else
				{
					UNSYNC_ERROR(L"Sync source is not directly accessible");
					return std::unique_ptr<FIOReader>(new FNullReaderWriter(FNullReaderWriter::FInvalid()));
				}
			});

		FBuildTargetParams BuildParams = GetBuildTargetParams(Options);
		FBuildTargetResult BuildResult = BuildTarget(*TargetFile, SourceFile, BaseDataReader, NeedList, BuildParams);

		Result += BuildResult;

		if (!BuildResult.bSuccess)
		{
			Result.Status = EFileSyncStatus::ErrorBuildTargetFailed;
			return Result;
		}

		if (Options.bValidateTargetFiles)
		{
			LogStatus(TargetFilePath.wstring().c_str(), L"Verifying");
			UNSYNC_VERBOSE(L"Verifying patched file '%ls'", TargetFilePath.wstring().c_str());
			UNSYNC_LOG_INDENT;

			if (!GDryRun)
			{
				// Reopen the file in unuffered read mode for optimal reading performance
				TargetFile = nullptr;
				TargetFile = std::make_unique<FNativeFile>(TempTargetFilePath, EFileMode::ReadOnlyUnbuffered);
			}

			if (TargetFileSizeInfo.TotalBytes > 0)
			{
				if (!ValidateTarget(*TargetFile, NeedList, Options.Algorithm.StrongHashAlgorithmId))
				{
					Result.Status = EFileSyncStatus::ErrorValidation;
					return Result;
				}
			}
		}

		if (GDryRun)
		{
			Result.Status = EFileSyncStatus::Ok;
		}
		else
		{
			LogStatus(TargetFilePath.wstring().c_str(), L"Finalizing");
			UNSYNC_VERBOSE(L"Finalizing target file '%ls'", TargetFilePath.wstring().c_str());
			BaseDataReader.Close();
			if (TargetFile)
			{
				TargetFile->Close();
			}

			if (GetFileAttrib(TargetFilePath).bReadOnly)
			{
				UNSYNC_VERBOSE(L"Clearing read-only flag from target file '%ls'", TargetFilePath.wstring().c_str());
				bool bClearReadOnlyOk = SetFileReadOnly(TargetFilePath, false);
				if (!bClearReadOnlyOk)
				{
					UNSYNC_ERROR(L"Failed to clear read-only flag from '%ls'", TargetFilePath.wstring().c_str());
				}
			}

			std::error_code ErrorCode = {};
			FileRename(TempTargetFilePath, TargetFilePath, ErrorCode);

			if (ErrorCode.value() == 0)
			{
				Result.Status = EFileSyncStatus::Ok;
			}
			else
			{
				Result.Status		   = EFileSyncStatus::ErrorFinalRename;
				Result.SystemErrorCode = ErrorCode;
			}
		}

		const uint64 ExpectedSourceBytes = ComputeSize(NeedList.Source);
		const uint64 ExpectedBaseBytes	 = ComputeSize(NeedList.Base);

		const uint64 ActualProcessedBytes	= BuildResult.SourceBytes + BuildResult.BaseBytes;
		const uint64 ExpectedProcessedBytes = ExpectedSourceBytes + ExpectedBaseBytes;

		if (ActualProcessedBytes != ExpectedProcessedBytes)
		{
			Result.Status = EFileSyncStatus::ErrorValidation;
			UNSYNC_ERROR(L"Failed to patch file '%ls'. Expected to write %llu bytes, but actually wrote %llu bytes.",
						 TargetFilePath.wstring().c_str(),
						 llu(ExpectedProcessedBytes),
						 llu(ActualProcessedBytes));
		}
	}
	else
	{
		UNSYNC_VERBOSE(L"Target file '%ls' already synchronized", TargetFilePath.wstring().c_str());
		Result.Status	 = EFileSyncStatus::Ok;
		Result.BaseBytes = NeedFromBase;
	}

	return Result;
}

FFileSyncResult
SyncFile(const FPath&			   SourceFilePath,
		 const FGenericBlockArray& SourceBlocks,
		 FIOReader&				   BaseDataReader,
		 const FPath&			   TargetFilePath,
		 const FSyncFileOptions&   Options)
{
	UNSYNC_LOG_INDENT;
	UNSYNC_VERBOSE(L"Computing difference for target '%ls' (base size: %.2f MB)",
				   TargetFilePath.wstring().c_str(),
				   SizeMb(BaseDataReader.GetSize()));
	FNeedList NeedList = DiffBlocks(BaseDataReader,
									Options.BlockSize,
									Options.Algorithm.WeakHashAlgorithmId,
									Options.Algorithm.StrongHashAlgorithmId,
									SourceBlocks);
	return SyncFile(NeedList, SourceFilePath, SourceBlocks, BaseDataReader, TargetFilePath, Options);
}

FFileSyncResult
SyncFile(const FPath& SourceFilePath, const FPath& BaseFilePath, const FPath& TargetFilePath, const FSyncFileOptions& InOptions)
{
	UNSYNC_LOG_INDENT;

	FSyncFileOptions Options = InOptions;  // This may be modified by LoadBlocks()

	FFileSyncResult Result;

	FNativeFile BaseFile(BaseFilePath, EFileMode::ReadOnlyUnbuffered);
	if (!BaseFile.IsValid())
	{
		BaseFile.Close();

		UNSYNC_VERBOSE(L"Full copy required for '%ls' (base does not exist)", BaseFilePath.wstring().c_str());
		std::error_code ErrorCode;
		bool			bCopyOk = FileCopy(SourceFilePath, TargetFilePath, ErrorCode);
		if (bCopyOk)
		{
			Result.Status = EFileSyncStatus::Ok;
		}
		else
		{
			Result.Status		   = EFileSyncStatus::ErrorFullCopy;
			Result.SystemErrorCode = ErrorCode;
		}
		Result.SourceBytes = GetFileAttrib(SourceFilePath).Size;
		return Result;
	}

	FGenericBlockArray SourceBlocks;
	FPath			   BlockFilename = BaseFilePath.wstring() + std::wstring(L".unsync");

	UNSYNC_VERBOSE(L"Loading block manifest from '%ls'", BlockFilename.wstring().c_str());
	if (LoadBlocks(SourceBlocks, Options.BlockSize, BlockFilename.c_str()))
	{
		UNSYNC_VERBOSE(L"Loaded blocks: %d", uint32(SourceBlocks.size()));
	}
	else
	{
		UNSYNC_VERBOSE(L"Full copy required (manifest file does not exist or is invalid)");

		std::error_code ErrorCode;
		bool			bCopyOk = FileCopy(SourceFilePath, TargetFilePath, ErrorCode);
		if (bCopyOk)
		{
			Result.Status = EFileSyncStatus::Ok;
		}
		else
		{
			Result.Status		   = EFileSyncStatus::ErrorFullCopy;
			Result.SystemErrorCode = ErrorCode;
		}

		Result.SourceBytes = GetFileAttrib(SourceFilePath).Size;

		return Result;
	}

	return SyncFile(SourceFilePath, SourceBlocks, BaseFile, TargetFilePath, Options);
}

struct FPendingFileRename
{
	std::wstring Old;
	std::wstring New;
};

// Updates the target directory manifest filename case to be consistent with reference.
// Internally we always perform case-sensitive path comparisons, however on non-case-sensitive filesystems some local files may be renamed
// to a mismatching case. We can update the locally-generated manifest to take the case from the reference manifest for equivalent paths.
// Returns a list of files that should be renamed on disk.
static std::vector<FPendingFileRename>
FixManifestFileNameCases(FDirectoryManifest& TargetDirectoryManifest, const FDirectoryManifest& ReferenceManifest)
{
	// Build a lookup table of lowercase -> original file names and detect potential case conflicts (which will explode on Windows and Mac)

	std::unordered_map<std::wstring, std::wstring> ReferenceFileNamesLowerCase;
	bool										   bFoundCaseConflicts = false;
	for (auto& ReferenceManifestEntry : ReferenceManifest.Files)
	{
		std::wstring FileNameLowerCase = StringToLower(ReferenceManifestEntry.first);
		auto		 InsertResult =
			ReferenceFileNamesLowerCase.insert(std::pair<std::wstring, std::wstring>(FileNameLowerCase, ReferenceManifestEntry.first));

		if (!InsertResult.second)
		{
			UNSYNC_WARNING(L"Found file name case conflict: '%ls'", ReferenceManifestEntry.first.c_str());
			bFoundCaseConflicts = true;
		}
	}

	if (bFoundCaseConflicts)
	{
		UNSYNC_WARNING(L"File name case conflicts will result in issues on case-insensitive systems, such as Windows and macOS.");
	}

	// Find inconsistently-cased files and add them to a list to be fixed up

	std::vector<FPendingFileRename> FixupEntries;

	for (auto& TargetManifestEntry : TargetDirectoryManifest.Files)
	{
		const std::wstring& TargetFileName = TargetManifestEntry.first;
		if (ReferenceManifest.Files.find(TargetFileName) == ReferenceManifest.Files.end())
		{
			std::wstring TargetFileNameLowerCase = StringToLower(TargetFileName);
			auto		 ReferenceIt			 = ReferenceFileNamesLowerCase.find(TargetFileNameLowerCase);
			if (ReferenceIt != ReferenceFileNamesLowerCase.end())
			{
				FixupEntries.push_back({TargetFileName, ReferenceIt->second});
			}
		}
	}

	// Re-add file manifests under the correct names

	for (const FPendingFileRename& Entry : FixupEntries)
	{
		auto It = TargetDirectoryManifest.Files.find(Entry.Old);
		UNSYNC_ASSERT(It != TargetDirectoryManifest.Files.end());

		FFileManifest Manifest;
		std::swap(It->second, Manifest);
		TargetDirectoryManifest.Files.erase(Entry.Old);
		TargetDirectoryManifest.Files.insert(std::pair(Entry.New, std::move(Manifest)));
	}

	return FixupEntries;
}

// Takes a list of file names that require case fixup and performs the necessary renaming.
// Handles renaming of intermediate directories as well as the leaf files.
// Quite wasteful in terms of mallocs, but doesn't matter since we're about to touch the file system anyway.
static bool
FixFileNameCases(const FPath& RootPath, const std::vector<FPendingFileRename>& PendingRenames)
{
	std::vector<FPendingFileRename>		   UniqueRenames;
	std::unordered_set<FPath::string_type> UniqueRenamesSet;

	// Build a rename schedule, with only unique entries (taking subdirectories into account)

	for (const FPendingFileRename& Entry : PendingRenames)
	{
		UNSYNC_ASSERTF(StringToLower(Entry.Old) == StringToLower(Entry.New),
					   L"FixFileNameCases expects inputs that are different only by case. Old: '%ls', New: '%ls'",
					   Entry.Old.c_str(),
					   Entry.New.c_str());

		FPath OldPath = Entry.Old;
		FPath NewPath = Entry.New;

		auto ItOld = OldPath.begin();
		auto ItNew = NewPath.begin();

		FPath OldPathPart;
		FPath NewPathPart;

		while (ItOld != OldPath.end())
		{
			OldPathPart /= *ItOld;
			NewPathPart /= *ItNew;

			if (*ItOld != *ItNew)
			{
				auto InsertResult = UniqueRenamesSet.insert(OldPathPart.native());
				if (InsertResult.second)
				{
					UniqueRenames.push_back({OldPathPart.wstring(), NewPathPart.wstring()});
				}
			}

			++ItOld;
			++ItNew;
		}
	}

	std::sort(UniqueRenames.begin(), UniqueRenames.end(), [](const FPendingFileRename& A, const FPendingFileRename& B) {
		return A.Old < B.Old;
	});

	// Perform actual renaming

	for (const FPendingFileRename& Entry : UniqueRenames)
	{
		FPath OldPath = RootPath / Entry.Old;
		FPath NewPath = RootPath / Entry.New;

		std::error_code ErrorCode;

		if (GDryRun)
		{
			UNSYNC_VERBOSE(L"Renaming '%ls' -> '%ls' (skipped due to dry run mode)", Entry.Old.c_str(), Entry.New.c_str());
		}
		else
		{
			UNSYNC_VERBOSE(L"Renaming '%ls' -> '%ls'", Entry.Old.c_str(), Entry.New.c_str());
			FileRename(OldPath, NewPath, ErrorCode);
		}

		if (ErrorCode)
		{
			UNSYNC_VERBOSE(L"Failed to rename file. System error code %d: %hs", ErrorCode.value(), ErrorCode.message().c_str());
			return false;
		}
	}

	return true;
}

// Delete files from target directory that are not in the source directory manifest
static void
DeleteUnnecessaryFiles(const FPath&				 TargetDirectory,
					   const FDirectoryManifest& TargetDirectoryManifest,
					   const FDirectoryManifest& ReferenceManifest,
					   const FSyncFilter*		 SyncFilter)
{
	auto ShouldCleanup = [SyncFilter](const FPath& Filename) -> bool {
		if (SyncFilter)
		{
			return SyncFilter->ShouldCleanup(Filename);
		}
		else
		{
			return true;
		}
	};

	for (const auto& TargetManifestEntry : TargetDirectoryManifest.Files)
	{
		const std::wstring& TargetFileName = TargetManifestEntry.first;

		auto Cleanup = [&ShouldCleanup, &TargetDirectory](const std::wstring& TargetFileName, const wchar_t* Reason) {
			FPath FilePath = TargetDirectory / TargetFileName;

			if (!ShouldCleanup(TargetFileName))
			{
				UNSYNC_VERBOSE2(L"Skipped deleting '%ls' (excluded by cleanup filter)", FilePath.wstring().c_str());
				return;
			}

			if (GDryRun)
			{
				UNSYNC_VERBOSE(L"Deleting '%ls' (%ls, skipped due to dry run mode)", FilePath.wstring().c_str(), Reason);
			}
			else
			{
				UNSYNC_VERBOSE(L"Deleting '%ls' (%ls)", FilePath.wstring().c_str(), Reason);
				std::error_code ErrorCode = {};
				FileRemove(FilePath, ErrorCode);
				if (ErrorCode)
				{
					UNSYNC_VERBOSE(L"System error code %d: %hs", ErrorCode.value(), ErrorCode.message().c_str());
				}
			}
		};

		if (ReferenceManifest.Files.find(TargetFileName) == ReferenceManifest.Files.end())
		{
			Cleanup(TargetFileName, L"not in manifest");
		}
		else if (!SyncFilter->ShouldSync(TargetFileName))
		{
			Cleanup(TargetFileName, L"excluded from sync");
		}
	}
}

FPath
ToPath(const std::wstring_view& Str)
{
#if UNSYNC_PLATFORM_UNIX
	// TODO: ensure that all serialized path separators are unix style ('/')
	std::wstring Temp = std::wstring(Str);
	std::replace(Temp.begin(), Temp.end(), L'\\', L'/');
	return FPath(Temp);
#else	// UNSYNC_PLATFORM_UNIX
	return FPath(Str);
#endif	// UNSYNC_PLATFORM_UNIX
}

struct FPooledProxy
{
	FPooledProxy(FProxyPool& InProxyPool) : ProxyPool(InProxyPool) { Proxy = ProxyPool.Alloc(); }
	~FPooledProxy() { ProxyPool.Dealloc(std::move(Proxy)); }
	const FProxy&			operator->() const { return *Proxy; }
	FProxyPool&				ProxyPool;
	std::unique_ptr<FProxy> Proxy;
};

struct FRemoteFileInfo
{
	FPath							   Path;
	ProxyQuery::FDirectoryListingEntry Entry;
};

struct FFoundManifest
{
	FRemoteFileInfo				Manifest;
	std::vector<FRemoteFileInfo> PackDataFiles;
	std::vector<FRemoteFileInfo> PackIndexFiles;
};

static TResult<FFoundManifest>
FindUnsyncManifest(FProxyFileSystem& FileSystem)
{
	using FDirectoryListing		 = ProxyQuery::FDirectoryListing;
	using FDirectoryListingEntry = ProxyQuery::FDirectoryListingEntry;

	TResult<FDirectoryListing> RootDirectoryListingResult = FileSystem.ListDirectory("");

	UNSYNC_RETURN_ON_ERROR(RootDirectoryListingResult);

	std::optional<FDirectoryListingEntry> FoundUnsyncSubdirectory;
	std::optional<FDirectoryListingEntry> FoundUnsyncManifestDotfile;

	for (const FDirectoryListingEntry& Entry : RootDirectoryListingResult.GetData().Entries)
	{
		if (Entry.Name == ".unsyncmanifest" && !Entry.bDirectory)
		{
			FoundUnsyncManifestDotfile = Entry;
		}

		if (Entry.Name == ".unsync" && Entry.bDirectory)
		{
			FoundUnsyncSubdirectory = Entry;
		}
	}

	if (FoundUnsyncManifestDotfile && !FoundUnsyncSubdirectory)
	{
		FFoundManifest Result;
		Result.Manifest.Path = FoundUnsyncManifestDotfile->Name;
		Result.Manifest.Entry = *FoundUnsyncManifestDotfile;
		return ResultOk(Result);
	}

	if (FoundUnsyncSubdirectory)
	{
		std::string				   UnsyncDirectory				= ".unsync";
		TResult<FDirectoryListing> UnsyncDirectoryListingResult = FileSystem.ListDirectory(UnsyncDirectory);

		UNSYNC_RETURN_ON_ERROR(UnsyncDirectoryListingResult);

		FFoundManifest Result;

		for (const FDirectoryListingEntry& Entry : UnsyncDirectoryListingResult.GetData().Entries)
		{
			if (Entry.Name == "manifest.bin" && !Entry.bDirectory)
			{
				Result.Manifest.Path = FPath(UnsyncDirectory) / Entry.Name;
				Result.Manifest.Entry = Entry;
			}

			if (Entry.Name == "pack" && Entry.bDirectory)
			{
				FPath					   PackDirectory		 = FPath(UnsyncDirectory) / "pack";
				TResult<FDirectoryListing> PackDirectoryListring = FileSystem.ListDirectory(ToString(PackDirectory));
				if (const FDirectoryListing* Listing = PackDirectoryListring.TryData())
				{
					for (const FDirectoryListingEntry& PackEntry : Listing->Entries)
					{
						if (PackEntry.bDirectory)
						{
							continue;
						}

						FRemoteFileInfo Info;
						Info.Path = PackDirectory / PackEntry.Name;
						Info.Entry = PackEntry;

						if (PackEntry.Name.ends_with(".unsync_pack"))
						{
							Result.PackDataFiles.push_back(Info);
						}

						if (PackEntry.Name.ends_with(".unsync_index"))
						{
							Result.PackIndexFiles.push_back(Info);
						}
					}
				}
			}
		}

		if (!Result.Manifest.Path.empty())
		{
			return ResultOk(std::move(Result));
		}
	}

	return AppError("Could not find unsync manifest file");
}

static bool
CopyFileIfPossiblyDifferent(FProxyFileSystem&	   FileSystem,
							const FRemoteFileInfo& Source,
							const FPath&		   Target,
							EFileMode			   TargetFileMode = EFileMode::CreateWriteOnly)
{
	FFileAttributes TargetAttr = GetFileAttrib(Target);
	if (Source.Entry.Size != TargetAttr.Size || Source.Entry.Mtime != TargetAttr.Mtime)
	{
		UNSYNC_VERBOSE(L"Reading '%ls'", Source.Path.wstring().c_str());

		TResult<FBuffer> FileBuffer = FileSystem.ReadFile(ToString(Source.Path));
		if (FileBuffer.IsError())
		{
			LogError(FileBuffer.GetError(), L"Failed to read source file");
			return false;
		}

		if (FileBuffer->Size() != Source.Entry.Size)
		{
			UNSYNC_ERROR(L"Read file size mismatch. Expected %llu, actual %llu.", llu(Source.Entry.Size), llu(FileBuffer->Size()));
			return false;
		}

		const bool bFileWritten = WriteBufferToFile(Target, *FileBuffer, TargetFileMode);
		if (!bFileWritten)
		{
			UNSYNC_ERROR(L"Failed to write file '%ls'", Target.wstring().c_str());
			return false;
		}

		const bool bAllowInDryRun = true;
		if (Source.Entry.Mtime)
		{
			SetFileMtime(Target, Source.Entry.Mtime, bAllowInDryRun);
		}
	}

	return true;
}

static bool
LoadAndMergeSourceManifest(FDirectoryManifest& Output,
						   std::vector<FPackIndexDatabase>& OutIndexFiles,
						   FProxyFileSystem&   ProxyFileSystem,
						   const FPath&		   SourcePath,
						   const FPath&		   TempPath,
						   FSyncFilter*		   SyncFilter,
						   bool				   bCaseSensitiveTargetFileSystem)
{
	UNSYNC_VERBOSE2(L"LoadAndMergeSourceManifest: '%ls'", SourcePath.wstring().c_str());

	FDirectoryManifest LoadedManifest;

	FPath SourceManifestRelativePath = FPath(".unsync") / "manifest.bin";
	FPath SourceManifestPath		 = SourcePath / SourceManifestRelativePath;

	FHash128 SourcePathHash =
		HashBlake3Bytes<FHash128>((const uint8*)SourcePath.native().c_str(), SourcePath.native().length() * sizeof(SourcePath.native()[0]));

	std::string SourcePathHashStr  = BytesToHexString(SourcePathHash.Data, sizeof(SourcePathHash.Data));
	FPath		CachedManifestPath = TempPath / SourcePathHashStr;

	LogGlobalStatus(L"Caching source manifest");
	UNSYNC_VERBOSE(L"Caching source manifest");

	UNSYNC_LOG_INDENT;
	UNSYNC_VERBOSE(L"Source '%ls'", SourceManifestPath.wstring().c_str());
	UNSYNC_VERBOSE(L"Target '%ls'", CachedManifestPath.wstring().c_str());

	TResult<FFoundManifest> FindManifestResult = FindUnsyncManifest(ProxyFileSystem);
	if (FindManifestResult.IsError())
	{
		LogError(FindManifestResult.GetError(), L"Failed to find remote manifest");
		return false;
	}

	if (!CopyFileIfPossiblyDifferent(ProxyFileSystem,
		FindManifestResult->Manifest,
		CachedManifestPath,
		EFileMode::CreateReadWrite | EFileMode::IgnoreDryRun))
	{
		return false;
	}

	if (!LoadDirectoryManifest(LoadedManifest, SourcePath, CachedManifestPath))
	{
		UNSYNC_ERROR(L"Failed to load source directory manifest '%ls'", SourceManifestPath.wstring().c_str());

		return false;
	}

	if (!FindManifestResult->PackIndexFiles.empty())
	{
		UNSYNC_VERBOSE(L"Loading pack index database");
		UNSYNC_LOG_INDENT;

		std::unordered_set<FPathStringView> FoundPackFiles;
		for (const FRemoteFileInfo& PackFileInfo : FindManifestResult->PackDataFiles)
		{
			FoundPackFiles.insert(PackFileInfo.Path.native());
		}

		for (const FRemoteFileInfo& IndexFileInfo : FindManifestResult->PackIndexFiles)
		{
			FPath PackDataFilePath = IndexFileInfo.Path;
			PackDataFilePath.replace_extension(".unsync_pack");
			if (!FoundPackFiles.contains(PackDataFilePath.native()))
			{
				UNSYNC_WARNING(L"Could not find pack file '%ls'", PackDataFilePath.wstring().c_str());
				continue;
			}

			UNSYNC_VERBOSE(L"Reading '%ls'", IndexFileInfo.Path.wstring().c_str());
			TResult<FBuffer> FileBuffer = ProxyFileSystem.ReadFile(ToString(IndexFileInfo.Path));
			if (FileBuffer.IsError())
			{
				LogError(FileBuffer.GetError(), L"Failed to read remote file");
				return false;
			}

			FMemReader		 Reader(*FileBuffer);
			FIOReaderStream	 Stream(Reader);

			FPackIndexDatabase IndexFile;
			IndexFile.IndexPath = SourcePath / IndexFileInfo.Path;
			IndexFile.DataPath	= SourcePath / PackDataFilePath;

			if (LoadPackIndexDatabase(IndexFile, Stream))
			{
				OutIndexFiles.emplace_back(std::move(IndexFile));
			}
		}
	}

	if (Output.IsValid() && !AlgorithmOptionsCompatible(Output.Algorithm, LoadedManifest.Algorithm))
	{
		UNSYNC_ERROR(L"Can't merge manifest '%ls' as it uses different algorithm options", SourcePath.wstring().c_str());
		return false;
	}

	return MergeManifests(Output, LoadedManifest, bCaseSensitiveTargetFileSystem);
}

struct FFileSyncTaskBatch
{
	std::vector<const FFileSyncTask*> FileTasks;

	uint64 TotalSizeBytes	   = 0;
	uint64 NeedBytesFromSource = 0;

	std::unique_ptr<FBlockCache> CreateBlockCache(FProxyPool& ProxyPool, EStrongHashAlgorithmID StrongHasher) const
	{
		FTimePoint TimeBegin = TimePointNow();

		std::unique_ptr<FBlockCache> Result = std::make_unique<FBlockCache>();

		Result->BlockData.Resize(NeedBytesFromSource);

		uint64 OutputCursor = 0;

		THashSet<FHash128>		UniqueBlockSet;
		std::vector<FNeedBlock> UniqueNeedBlocks;
		for (const FFileSyncTask* Task : FileTasks)
		{
			for (const FNeedBlock& Block : Task->NeedList.Source)
			{
				if (UniqueBlockSet.insert(Block.Hash.ToHash128()).second)
				{
					UniqueNeedBlocks.push_back(Block);
				}
			}
		}

		Result->BlockMap.reserve(UniqueNeedBlocks.size());

		GScheduler->NetworkSemaphore.Acquire(false);
		std::unique_ptr<FProxy> Proxy = ProxyPool.Alloc();

		if (Proxy)
		{
			auto DownloadCallback =
				[&OutputCursor, &Result, &UniqueBlockSet, StrongHasher](const FDownloadedBlock& Block, FHash128 BlockHash)
			{
				if (OutputCursor + Block.DecompressedSize <= Result->BlockData.Size())
				{
					if (UniqueBlockSet.find(BlockHash) != UniqueBlockSet.end())
					{
						FMutBufferView OutputView = Result->BlockData.MutView(OutputCursor, Block.DecompressedSize);

						bool bOk = true;

						if (Block.bCompressed)
						{
							bOk = Decompress(Block.Data, Block.CompressedSize, OutputView.Data, OutputView.Size);
						}
						else
						{
							memcpy(OutputView.Data, Block.Data, OutputView.Size);
						}

						if (bOk)
						{
							const FHash128 ActualBlockHash = ComputeHash(OutputView.Data, OutputView.Size, StrongHasher).ToHash128();

							bOk = BlockHash == ActualBlockHash;
						}

						if (bOk)
						{
							Result->BlockMap[BlockHash] = FBufferView{OutputView.Data, OutputView.Size};
							OutputCursor += Block.DecompressedSize;
						}
						else
						{
							UNSYNC_WARNING(L"Received a corrupt block");
						}
					}
					else
					{
						UNSYNC_WARNING(L"Received a block with unexpected hash");
					}
				}
			};

			FDownloadResult DownloadResult =
				Proxy->Download(MakeView<FNeedBlock>(UniqueNeedBlocks.data(), UniqueNeedBlocks.size()), DownloadCallback);

			const uint64 NumExpected = UniqueNeedBlocks.size();
			const uint64 NumDownloaded = Result->BlockMap.size();
			if (NumExpected != NumDownloaded)
			{
				THashSet<FHash128> MissingBlocks = UniqueBlockSet;
				for (const auto& It : Result->BlockMap)
				{
					MissingBlocks.erase(It.first);
				}

				if (MissingBlocks.size() <= 10)
				{
					std::string MissingBlockStr;
					for (const FHash128& Hash : MissingBlocks)
					{
						if (!MissingBlockStr.empty())
						{
							MissingBlockStr += ", ";
						}
						MissingBlockStr += HashToHexString(Hash);
					}

					UNSYNC_WARNING(
						L"Could not download all required data while building block cache. "
						L"Blocks expected: %llu, actual: %llu. Missing blocks: %hs",
						llu(NumExpected),
						llu(NumDownloaded),
						MissingBlockStr.c_str());
				}
				else
				{
					UNSYNC_WARNING(
						L"Could not download all required data while building block cache. "
						L"Blocks expected: %llu, actual: %llu.",
						llu(NumExpected),
						llu(NumDownloaded));
				}
			}

			UNSYNC_UNUSED(DownloadResult);
		}

		ProxyPool.Dealloc(std::move(Proxy));

		GScheduler->NetworkSemaphore.Release();

		Result->InitDuration = TimePointNow() - TimeBegin;

		return Result;
	}
};

bool  // TODO: return a TResult
SyncDirectory(const FSyncDirectoryOptions& SyncOptions)
{
	FProxyPool	DummyProxyPool;
	FProxyPool& ProxyPool = SyncOptions.ProxyPool ? *SyncOptions.ProxyPool : DummyProxyPool;

	FTimePoint TimeBegin = TimePointNow();

	const bool bFileSystemSource = SyncOptions.SourceType == ESourceType::FileSystem;
	const bool bServerSource =
		SyncOptions.SourceType == ESourceType::Server || SyncOptions.SourceType == ESourceType::ServerWithManifestId;

	UNSYNC_ASSERT(bFileSystemSource || bServerSource);

	const FPath	 SourcePath				= bFileSystemSource ? std::filesystem::absolute(SyncOptions.Source) : SyncOptions.Source;
	const FPath	 BasePath				= std::filesystem::absolute(SyncOptions.Base);
	const FPath	 TargetPath				= std::filesystem::absolute(SyncOptions.Target);

	FSyncFilter* SyncFilter = SyncOptions.SyncFilter;

	bool bSourceManifestOk = true;

	UNSYNC_LOG_INDENT;

	if (SyncOptions.bCleanup)
	{
		UNSYNC_LOG(L"Unnecessary files will be deleted after sync (cleanup mode)");
	}

	const FPath BaseManifestRoot = BasePath / ".unsync";
	const FPath BaseManifestPath = BaseManifestRoot / "manifest.bin";

	const FPath TargetManifestRoot = TargetPath / ".unsync";
	const FPath TargetManifestPath = TargetManifestRoot / "manifest.bin";
	const FPath TargetTempPath	 = TargetManifestRoot / "temp";

	const bool bTempDirectoryExists = (PathExists(TargetTempPath) && IsDirectory(TargetTempPath)) || CreateDirectories(TargetTempPath);

	if (!bTempDirectoryExists)
	{
		UNSYNC_ERROR(L"Failed to create temporary working directory");
		return false;
	}

	// Delete oldest cached manifest files if there are more than N
	{
		UNSYNC_VERBOSE(L"Cleaning temporary directory");
		UNSYNC_LOG_INDENT;
		const uint32 MaxFilesToKeep = uint32(5 + SyncOptions.Overlays.size());
		DeleteOldFilesInDirectory(TargetTempPath, MaxFilesToKeep);
	}

	const FPath			LogFilePath = TargetManifestRoot / L"unsync.log";
	const FLogFileScope LogFileScope(LogFilePath.wstring().c_str());
	SetCrashDumpPath(TargetManifestRoot);

	auto ShouldSync = [SyncFilter](const FPath& Filename) -> bool {
		if (SyncFilter)
		{
			return SyncFilter->ShouldSync(Filename);
		}
		else
		{
			return true;
		}
	};

	FDirectoryManifest SourceDirectoryManifest;
	FPath			   SourceManifestTempPath;

	const bool bCaseSensitiveTargetFileSystem = IsCaseSensitiveFileSystem(TargetTempPath);

	FTimingLogger ManifestLoadTimingLogger("Manifest load time", ELogLevel::Info);

	std::vector<FPackIndexDatabase> PackIndexFiles;

	std::vector<FPath> AllSources;
	AllSources.push_back(SourcePath);
	for (const FPath& OverlayPath : SyncOptions.Overlays)
	{
		AllSources.push_back(OverlayPath);
	}
	
	auto ResolvePath = [SyncFilter](const FPath& Filename) -> FPath { return SyncFilter ? SyncFilter->Resolve(Filename) : Filename; };

	// Used to build block request map when syncing from multiple sources.
	THashMap<FHash256, uint32> FileSourceIdMap;

	if (SyncOptions.SourceType == ESourceType::ServerWithManifestId)
	{
		if (!ProxyPool.IsValid())
		{
			UNSYNC_ERROR(L"Remote server connection is required when syncing by manifest ID");
			return false;
		}

		std::unique_ptr<FProxy> Proxy = ProxyPool.Alloc();

		uint32 SourceIndex = 0;
		for (const FPath& ThisSourcePath : AllSources)
		{
			std::string SourceManifestName = ConvertWideToUtf8(ThisSourcePath.wstring());
			FHash128	SourcePathHash	   = HashBlake3String<FHash128>(SourceManifestName);
			std::string SourcePathHashStr  = BytesToHexString(SourcePathHash.Data, sizeof(SourcePathHash.Data));
			FPath		CachedManifestPath = TargetTempPath / SourcePathHashStr;

			FPath EmptyPath; // no physical path for downloaded manifests

			FDirectoryManifest LoadedManifest;
			if (!PathExists(CachedManifestPath) || !LoadDirectoryManifest(LoadedManifest, EmptyPath, CachedManifestPath))
			{
				LogGlobalStatus(L"Caching source manifest");
				UNSYNC_VERBOSE(L"Caching source manifest");

				UNSYNC_LOG_INDENT;
				UNSYNC_VERBOSE(L"Source '%hs'", SourceManifestName.c_str());
				UNSYNC_VERBOSE(L"Target '%ls'", CachedManifestPath.wstring().c_str());

				TResult<FDirectoryManifest> DownloadResult = Proxy->DownloadManifest(SourceManifestName);

				if (FDirectoryManifest* Manifest = DownloadResult.TryData())
				{
					std::swap(LoadedManifest, *Manifest);
				}
				else
				{
					LogError(DownloadResult.GetError(), L"Failed to download manifest");
					UNSYNC_BREAK_ON_ERROR;
					return false;
				}

				const bool bAllowInDryRun = true;
				SaveDirectoryManifest(LoadedManifest, CachedManifestPath, bAllowInDryRun);
			}

			for (const auto& It : LoadedManifest.Files)
			{
				FHash256 NameHash = HashBlake3String<FHash256>(It.first);
				FileSourceIdMap[NameHash] = SourceIndex;
			}

			bSourceManifestOk = MergeManifests(SourceDirectoryManifest, LoadedManifest, bCaseSensitiveTargetFileSystem);

			if (!bSourceManifestOk)
			{
				break;
			}

			++SourceIndex;
		}

		ProxyPool.Dealloc(std::move(Proxy)); // TODO: RAII helper for pooled proxy connections
	}
	else if (!SyncOptions.SourceManifestOverride.empty())
	{
		bSourceManifestOk = LoadDirectoryManifest(SourceDirectoryManifest, SourcePath, SyncOptions.SourceManifestOverride);
		// TODO: load pack index files
		if (!bSourceManifestOk)
		{
			UNSYNC_ERROR(L"Could not load explicit manifest file");
			return false;
		}
	}
	else
	{
		for (const FPath& ThisSourcePath : AllSources)
		{
			std::unique_ptr<FProxyFileSystem> ProxyFileSystem;
			if (bServerSource)
			{
				const FRemoteProtocolFeatures& RemoteFeatures = ProxyPool.GetFeatures();

				if (!RemoteFeatures.bDirectoryListing)
				{
					UNSYNC_ERROR(L"Remote server does not support directory listing");
					return false;
				}

				if (!RemoteFeatures.bFileDownload)
				{
					UNSYNC_ERROR(L"Remote server does not support direct file downloads");
					return false;
				}

				ProxyFileSystem = std::make_unique<FRemoteFileSystem>(ToString(ThisSourcePath), ProxyPool);
			}
			else
			{
				ProxyFileSystem = std::make_unique<FPhysicalFileSystem>(ThisSourcePath);
			}

			if (!LoadAndMergeSourceManifest(SourceDirectoryManifest,
											PackIndexFiles,
											*ProxyFileSystem,
											ThisSourcePath,
											TargetTempPath,
											SyncFilter,
											bCaseSensitiveTargetFileSystem))
			{
				return false;
			}
		}
	}

	{
		UNSYNC_VERBOSE(L"Loaded manifest properties:");
		UNSYNC_LOG_INDENT;
		FDirectoryManifestInfo ManifestInfo = GetManifestInfo(SourceDirectoryManifest, false /*bGenerateSignature*/);
		LogManifestInfo(ELogLevel::Debug, ManifestInfo);
		if (ProxyPool.RemoteDesc.Protocol == EProtocolFlavor::Jupiter && ManifestInfo.NumMacroBlocks == 0)
		{
			UNSYNC_ERROR(L"Manifest must contain macro blocks when using Jupiter");
			return false;
		}
	}

	ManifestLoadTimingLogger.Finish();

	FTimingLogger TargetManifestTimingLogger("Target directory manifest generation time", ELogLevel::Info);
	UNSYNC_LOG(L"Creating manifest for directory '%ls'", TargetPath.wstring().c_str());


	// Propagate algorithm selection from source
	const FAlgorithmOptions Algorithm = SourceDirectoryManifest.Algorithm;

	FComputeBlocksParams LightweightManifestParams;
	LightweightManifestParams.Algorithm	  = Algorithm;
	LightweightManifestParams.bNeedBlocks = false;
	LightweightManifestParams.BlockSize	  = 0;

	FDirectoryManifest TargetDirectoryManifest = CreateDirectoryManifest(TargetPath, LightweightManifestParams);

	TargetManifestTimingLogger.Finish();

	if (!bCaseSensitiveTargetFileSystem)
	{
		std::vector<FPendingFileRename> PendingRenames;
		PendingRenames = FixManifestFileNameCases(TargetDirectoryManifest, SourceDirectoryManifest);
		if (!PendingRenames.empty())
		{
			UNSYNC_VERBOSE(L"Fixing inconsistent case of target files");
			UNSYNC_LOG_INDENT;
			if (!FixFileNameCases(TargetPath, PendingRenames))
			{
				return false;
			}
		}
	}

	uint32 StatSkipped	   = 0;
	uint32 StatFullCopy	   = 0;
	uint32 StatPartialCopy = 0;

	std::atomic<uint64> NumFailedTasks = {};

	std::atomic<uint64> StatSourceBytes = {};
	std::atomic<uint64> StatBaseBytes	= {};
	std::atomic<uint64> StatInSituBytes	= {};

	std::vector<FFileSyncTask> AllFileTasks;

	LogGlobalStatus(L"Scanning base directory");
	UNSYNC_LOG(L"Scanning base directory");
	FFileAttributeCache BaseAttribCache = CreateFileAttributeCache(BasePath, SyncFilter);
	UNSYNC_LOG(L"Base files: %d", (uint32)BaseAttribCache.Map.size());

	FFileAttributeCache SourceAttribCache;
	if (bFileSystemSource && SyncOptions.bValidateSourceFiles)
	{
		LogGlobalStatus(L"Scanning source directory");
		UNSYNC_LOG(L"Scanning source directory");
		SourceAttribCache = CreateFileAttributeCache(SourcePath, SyncFilter);
	}

	// If variable blocks are used and we already have a manifest file from previous sync,
	// then we can compute difference quickly based only on file timestamps and previously computed chunks.

	FDirectoryManifest BaseDirectoryManifest;
	bool			   bBaseDirectoryManifestValid = false;
	bool			   bQuickDifferencePossible	   = false;

	if (!SyncOptions.bFullDifference && SourceDirectoryManifest.Algorithm.ChunkingAlgorithmId == EChunkingAlgorithmID::VariableBlocks &&
		PathExists(BaseManifestPath))
	{
		bBaseDirectoryManifestValid = LoadDirectoryManifest(BaseDirectoryManifest, BasePath, BaseManifestPath);
		if (bBaseDirectoryManifestValid && AlgorithmOptionsCompatible(SourceDirectoryManifest.Algorithm, TargetDirectoryManifest.Algorithm))
		{
			bQuickDifferencePossible = true;
		}
	}

	if (bQuickDifferencePossible)
	{
		UNSYNC_LOG(L"Quick file difference is allowed (use --full-diff option to override)");
	}

	uint64 TotalSourceSize = 0;

	for (const auto& SourceManifestIt : SourceDirectoryManifest.Files)
	{
		const std::wstring& SourceFilename = SourceManifestIt.first;
		const FHash256		SourceFilenameHash = HashBlake3String<FHash256>(SourceFilename);

		if (!ShouldSync(SourceFilename))
		{
			StatSkipped++;
			UNSYNC_VERBOSE2(L"Skipped '%ls' (excluded by sync filter)", SourceManifestIt.first.c_str());
			continue;
		}

		const FFileManifest& SourceFileManifest = SourceManifestIt.second;

		TotalSourceSize += SourceFileManifest.Size;

		bool bTargetFileAttributesMatch = false;
		auto TargetManifestIt			= TargetDirectoryManifest.Files.find(SourceFilename);
		if (TargetManifestIt != TargetDirectoryManifest.Files.end())
		{
			const FFileManifest& TargetFileManifest = TargetManifestIt->second;

			if (SourceFileManifest.Size == TargetFileManifest.Size && SourceFileManifest.Mtime == TargetFileManifest.Mtime)
			{
				bTargetFileAttributesMatch = true;
			}
		}

		if (bTargetFileAttributesMatch && !SyncOptions.bFullDifference)
		{
			UNSYNC_VERBOSE2(L"Skipped '%ls' (up to date)", SourceManifestIt.first.c_str());
			StatSkipped++;
			continue;
		}

		FPath SourceFilePath = SourceManifestIt.second.CurrentPath;
		FPath BaseFilePath	 = BasePath / ToPath(SourceManifestIt.first);
		FPath TargetFilePath = TargetPath / ToPath(SourceManifestIt.first);

		FPath ResolvedSourceFilePath = ResolvePath(SourceFilePath);
		uint32 SourceId = 0;

		{
			FHash256 NameHash	   = HashBlake3String<FHash256>(SourceFilename);
			auto	 FoundSourceId = FileSourceIdMap.find(NameHash);
			if (FoundSourceId != FileSourceIdMap.end())
			{
				SourceId = FoundSourceId->second;
			}
		}

		if (bFileSystemSource && SyncOptions.bValidateSourceFiles)
		{
			FFileAttributes SourceFileAttrib = GetFileAttrib(ResolvedSourceFilePath, &SourceAttribCache);

			if (!SourceFileAttrib.bValid)
			{
				UNSYNC_ERROR(L"Source file '%ls' is declared in manifest but does not exist. Manifest may be wrong or out of date.",
							 SourceFilePath.wstring().c_str());
				bSourceManifestOk = false;
			}

			if (bSourceManifestOk && SourceFileAttrib.Size != SourceFileManifest.Size)
			{
				UNSYNC_ERROR(
					L"Source file '%ls' size (%lld bytes) does not match the manifest (%lld bytes). Manifest may be wrong or out of date.",
					SourceFilePath.wstring().c_str(),
					SourceFileAttrib.Size,
					SourceFileManifest.Size);
				bSourceManifestOk = false;
			}

			if (bSourceManifestOk && SourceFileAttrib.Mtime != SourceFileManifest.Mtime)
			{
				UNSYNC_ERROR(
					L"Source file '%ls' modification time (%lld) does not match the manifest (%lld). Manifest may be wrong or out of date.",
					SourceFilePath.wstring().c_str(),
					SourceFileAttrib.Mtime,
					SourceFileManifest.Mtime);
				bSourceManifestOk = false;
			}
		}

		if (bSourceManifestOk)
		{
			FFileAttributes BaseFileAttrib = GetCachedFileAttrib(BaseFilePath, BaseAttribCache);

			if (!BaseFileAttrib.bValid)
			{
				UNSYNC_VERBOSE2(L"Dirty file: '%ls' (no base data)", SourceFilename.c_str());
				StatFullCopy++;
			}
			else
			{
				if (bTargetFileAttributesMatch && SyncOptions.bFullDifference)
				{
					UNSYNC_VERBOSE2(L"Dirty file: '%ls' (forced by --full-diff)", SourceManifestIt.first.c_str());
				}
				else
				{
					UNSYNC_VERBOSE2(L"Dirty file: '%ls'", SourceManifestIt.first.c_str());
				}

				StatPartialCopy++;

				if (bFileSystemSource && SyncOptions.bValidateSourceFiles && !SourceAttribCache.Exists(ResolvedSourceFilePath) &&
					!PathExists(ResolvedSourceFilePath))
				{
					UNSYNC_VERBOSE(L"Source file '%ls' does not exist", SourceFilePath.wstring().c_str());
					continue;
				}
			}

			FFileSyncTask Task;
			Task.OriginalSourceFilePath = std::move(SourceFilePath);
			Task.ResolvedSourceFilePath = std::move(ResolvedSourceFilePath);
			Task.BaseFilePath			= std::move(BaseFilePath);
			Task.TargetFilePath			= std::move(TargetFilePath);
			Task.SourceManifest			= &SourceFileManifest;
			Task.SourceId				= SourceId;

			if (bQuickDifferencePossible)
			{
				UNSYNC_ASSERT(bBaseDirectoryManifestValid);
				auto BaseManifestIt = BaseDirectoryManifest.Files.find(SourceFilename);
				if (BaseManifestIt != BaseDirectoryManifest.Files.end())
				{
					const FFileManifest& BaseFileManifest = BaseManifestIt->second;
					if (BaseFileManifest.Mtime == BaseFileAttrib.Mtime && BaseFileManifest.Size == BaseFileAttrib.Size)
					{
						if (ValidateBlockListT(BaseFileManifest.Blocks))
						{
							Task.BaseManifest = &BaseFileManifest;
						}
					}
				}
			}

			AllFileTasks.push_back(Task);
		}
	}

	if (SourceDirectoryManifest.Files.empty())
	{
		UNSYNC_ERROR(L"Source directory manifest is empty");
		bSourceManifestOk = false;
	}

	if (!bSourceManifestOk)
	{
		return false;
	}

	LogGlobalStatus(L"Computing difference");
	UNSYNC_LOG(L"Computing difference ...");

	uint64 EstimatedNeedBytesFromSource = 0;
	uint64 EstimatedNeedBytesFromBase	= 0;
	uint64 TotalSyncSizeBytes			= 0;

	{
		UNSYNC_LOG_INDENT;
		auto TimeDiffBegin = TimePointNow();

		auto DiffTask = [Algorithm, bQuickDifferencePossible](FFileSyncTask& Item) {
			FLogVerbosityScope VerbosityScope(false);  // turn off logging from threads

			const FGenericBlockArray& SourceBlocks = Item.SourceManifest->Blocks;

			if (Item.IsBaseValid() && PathExists(Item.BaseFilePath))
			{
				FNativeFile BaseFile(Item.BaseFilePath, EFileMode::ReadOnlyUnbuffered);
				uint32	   SourceBlockSize = Item.SourceManifest->BlockSize;
				UNSYNC_VERBOSE(L"Computing difference for target '%ls' (base size: %.2f MB)",
							   Item.BaseFilePath.wstring().c_str(),
							   SizeMb(BaseFile.GetSize()));

				if (bQuickDifferencePossible && Item.BaseManifest)
				{
					Item.NeedList = DiffManifestBlocks(Item.SourceManifest->Blocks, Item.BaseManifest->Blocks);
				}
				else if (Algorithm.ChunkingAlgorithmId == EChunkingAlgorithmID::FixedBlocks)
				{
					Item.NeedList =
						DiffBlocks(BaseFile, SourceBlockSize, Algorithm.WeakHashAlgorithmId, Algorithm.StrongHashAlgorithmId, SourceBlocks);
				}
				else if (Algorithm.ChunkingAlgorithmId == EChunkingAlgorithmID::VariableBlocks)
				{
					Item.NeedList = DiffBlocksVariable(BaseFile,
													   SourceBlockSize,
													   Algorithm.WeakHashAlgorithmId,
													   Algorithm.StrongHashAlgorithmId,
													   SourceBlocks);
				}
				else
				{
					UNSYNC_FATAL(L"Unexpected file difference calculation mode");
				}
			}
			else
			{
				Item.NeedList.Sequence.reserve(SourceBlocks.size());
				Item.NeedList.Source.reserve(SourceBlocks.size());
				for (const FGenericBlock& Block : SourceBlocks)
				{
					FNeedBlock NeedBlock;
					NeedBlock.Size		   = Block.Size;
					NeedBlock.SourceOffset = Block.Offset;
					NeedBlock.TargetOffset = Block.Offset;
					NeedBlock.Hash		   = Block.HashStrong;
					Item.NeedList.Source.push_back(NeedBlock);
					Item.NeedList.Sequence.push_back(NeedBlock.Hash.ToHash128());  // #wip-widehash
				}
			}

			Item.NeedBytesFromSource = ComputeSize(Item.NeedList.Source);
			Item.NeedBytesFromBase	 = ComputeSize(Item.NeedList.Base);
			Item.TotalSizeBytes		 = Item.SourceManifest->Size;

			UNSYNC_ASSERT(Item.NeedBytesFromSource + Item.NeedBytesFromBase == Item.TotalSizeBytes);
		};

		ParallelForEach(AllFileTasks, DiffTask);

		auto TimeDiffEnd = TimePointNow();

		double Duration = DurationSec(TimeDiffBegin, TimeDiffEnd);
		UNSYNC_LOG(L"Difference complete in %.3f sec", Duration);

		for (FFileSyncTask& Item : AllFileTasks)
		{
			EstimatedNeedBytesFromSource += Item.NeedBytesFromSource;
			EstimatedNeedBytesFromBase += Item.NeedBytesFromBase;
			TotalSyncSizeBytes += Item.TotalSizeBytes;
		}

		UNSYNC_LOG(L"Total need from source: %.2f MB", SizeMb(EstimatedNeedBytesFromSource));
		UNSYNC_LOG(L"Total need from base: %.2f MB", SizeMb(EstimatedNeedBytesFromBase));

		uint64 AvailableDiskBytes = SyncOptions.bCheckAvailableSpace ? GetAvailableDiskSpace(TargetPath) : ~0ull;
		if (TotalSyncSizeBytes > AvailableDiskBytes)
		{
			UNSYNC_ERROR(
				L"Sync requires %.0f MB (%llu bytes) of disk space, but only %.0f MB (%llu bytes) is available. "
				L"Use --no-space-validation flag to suppress this check.",
				SizeMb(TotalSyncSizeBytes),
				TotalSyncSizeBytes,
				SizeMb(AvailableDiskBytes),
				AvailableDiskBytes);
			return false;
		}
	}

	GGlobalProgressCurrent = 0;
	GGlobalProgressTotal =
		EstimatedNeedBytesFromSource * GLOBAL_PROGRESS_SOURCE_SCALE + EstimatedNeedBytesFromBase * GLOBAL_PROGRESS_BASE_SCALE;

	std::unique_ptr<FScavengeDatabase> ScavengeDatabase;
	if (!SyncOptions.ScavengeRoot.empty())
	{
		UNSYNC_LOG(L"Scavenging blocks from existing data sets");
		UNSYNC_LOG_INDENT;

		FTimePoint ScavengeDbTimeBegin = TimePointNow();

		TArrayView<FFileSyncTask> AllFileTasksView = MakeView(AllFileTasks.data(), AllFileTasks.size());
		ScavengeDatabase = std::unique_ptr<FScavengeDatabase>(FScavengeDatabase::BuildFromFileSyncTasks(SyncOptions, AllFileTasksView));

		double Duration = DurationSec(ScavengeDbTimeBegin, TimePointNow());

		UNSYNC_LOG(L"Done in %.3f sec", Duration);
	}

	LogGlobalProgress();

	if (ProxyPool.IsValid())
	{
		LogGlobalStatus(L"Connecting to server");
		UNSYNC_LOG(L"Connecting to %hs server '%hs:%d' ...",
					   ToString(ProxyPool.RemoteDesc.Protocol),
					   ProxyPool.RemoteDesc.Host.Address.c_str(),
					   ProxyPool.RemoteDesc.Host.Port);
		UNSYNC_LOG_INDENT;

		std::unique_ptr<FProxy> Proxy = ProxyPool.Alloc();

		if (Proxy.get() && Proxy->IsValid())
		{
			// TODO: report TLS status
			// ESocketSecurity security = proxy->get_socket_security();
			// UNSYNC_LOG(L"Connection established (security: %hs)", ToString(security));

			UNSYNC_LOG(L"Connection established");
			UNSYNC_LOG(L"Building block request map");

			const bool bProxyHasData = Proxy->Contains(SourceDirectoryManifest);

			ProxyPool.Dealloc(std::move(Proxy));

			if (bProxyHasData)
			{
				FBlockRequestMap BlockRequestMap;
				BlockRequestMap.Init(SourceDirectoryManifest.Algorithm.StrongHashAlgorithmId, AllSources);

				for (const FFileSyncTask& Task : AllFileTasks)
				{
					BlockRequestMap.AddFileBlocks(Task.SourceId, Task.OriginalSourceFilePath, Task.ResolvedSourceFilePath, *Task.SourceManifest);
				}

				// Override loose file blocks with pack files
				for (const FPackIndexDatabase& Pack : PackIndexFiles)
				{
					FPath ResolvedDataPackPath = ResolvePath(Pack.DataPath);
					BlockRequestMap.AddPackBlocks(Pack.DataPath, ResolvedDataPackPath, MakeView(Pack.Entries));
				}

				ProxyPool.SetRequestMap(std::move(BlockRequestMap));
			}
			else
			{
				UNSYNC_WARNING(L"Remote server does not have the data referenced by manifest");
				ProxyPool.Invalidate();
			}
		}
		else
		{
			Proxy = nullptr;
			ProxyPool.Invalidate();
		}
	}
	else
	{
		// TODO: bail out if remote connection is required for the download,
		// such as when downloading data purely from Jupiter.
		UNSYNC_VERBOSE(L"Attempting to sync without remote server connection");
	}

	LogGlobalStatus(L"Copying files");
	UNSYNC_LOG(L"Copying files ...");

	{
		// Throttle background tasks by trying to keep them to some sensible memory budget. Best effort only, not a hard limit.
		const uint64 BackgroundTaskMemoryBudget	 = SyncOptions.BackgroundTaskMemoryBudget;
		const uint64 TargetTotalSizePerTaskBatch = BackgroundTaskMemoryBudget;
		const uint64 MaxFilesPerTaskBatch		 = SyncOptions.MaxFilesPerTask;

		UNSYNC_VERBOSE2(L"Background task memory budget: %llu GB", BackgroundTaskMemoryBudget >> 30);

		struct FBackgroundTaskResult
		{
			FPath			TargetFilePath;
			FFileSyncResult SyncResult;
			bool			bIsPartialCopy = false;
		};

		std::deque<FFileSyncTaskBatch> SyncTaskList;

		std::mutex						   BackgroundTaskStatMutex;
		std::vector<FBackgroundTaskResult> BackgroundTaskResults;

		// Tasks are sorted by download size and processed by multiple threads.
		// Large downloads are processed on the foreground thread and small ones on the background.
		std::sort(AllFileTasks.begin(), AllFileTasks.end(), [](const FFileSyncTask& A, const FFileSyncTask& B) {
			return A.NeedBytesFromSource < B.NeedBytesFromSource;
		});

		// Blocks for multiple files can be downloaded in one request.
		// Group small file tasks into batches to reduce the number of individual download requests.
		{
			const uint64 MaxBatchDownloadSize = 4_MB;

			FFileSyncTaskBatch CurrentBatch;
			for (const FFileSyncTask& FileTask : AllFileTasks)
			{
				bool bShouldBreakBatch = false;

				if (!CurrentBatch.FileTasks.empty())
				{
					if (CurrentBatch.NeedBytesFromSource + FileTask.NeedBytesFromSource > MaxBatchDownloadSize)
					{
						bShouldBreakBatch = true;
					}
					else if (CurrentBatch.FileTasks.size() >= MaxFilesPerTaskBatch)
					{
						bShouldBreakBatch = true;
					}
					else if (CurrentBatch.TotalSizeBytes >= TargetTotalSizePerTaskBatch)
					{
						bShouldBreakBatch = true;
					}
				}

				if (bShouldBreakBatch)
				{
					SyncTaskList.push_back(std::move(CurrentBatch));
					CurrentBatch = {};
				}

				CurrentBatch.FileTasks.push_back(&FileTask);
				CurrentBatch.NeedBytesFromSource += FileTask.NeedBytesFromSource;
				CurrentBatch.TotalSizeBytes += FileTask.TotalSizeBytes;
			}

			if (CurrentBatch.FileTasks.size())
			{
				SyncTaskList.push_back(std::move(CurrentBatch));
			}
		}

		// Validate batching
		{
			uint64 TotalSyncSizeBatched = 0;
			uint64 TotalFilesBatched	= 0;
			for (const FFileSyncTaskBatch& Batch : SyncTaskList)
			{
				TotalSyncSizeBatched += Batch.TotalSizeBytes;
				TotalFilesBatched += Batch.FileTasks.size();
			}
			UNSYNC_ASSERT(TotalFilesBatched == AllFileTasks.size());
			UNSYNC_ASSERT(TotalSyncSizeBatched == TotalSyncSizeBytes);
		}

		auto FileSyncTaskBody = [Algorithm,
								 &SyncOptions,
								 &StatSourceBytes,
								 &StatBaseBytes,
								 &StatInSituBytes,
								 &BackgroundTaskStatMutex,
								 &BackgroundTaskResults,
								 &NumFailedTasks,
								 &ScavengeDatabase,
								 &ProxyPool](const FFileSyncTask& Item, FBlockCache* BlockCache, bool bBackground) {
			UNSYNC_VERBOSE(L"Copy '%ls' (%ls)", Item.TargetFilePath.wstring().c_str(), (Item.NeedBytesFromBase) ? L"partial" : L"full");

			FDeferredOpenReader BaseFile(
				[&Item]
				{
					if (Item.IsBaseValid())
					{
						UNSYNC_VERBOSE(L"Opening base file '%ls'", Item.BaseFilePath.wstring().c_str());
						LogStatus(Item.BaseFilePath.wstring().c_str(), L"Opening base file");
						return std::unique_ptr<FIOReader>(new FNativeFile(Item.BaseFilePath, EFileMode::ReadOnlyUnbuffered));
					}
					else
					{
						return std::unique_ptr<FIOReader>(new FNullReaderWriter(0));
					}
				});

			const FGenericBlockArray& SourceBlocks	  = Item.SourceManifest->Blocks;
			uint32					  SourceBlockSize = Item.SourceManifest->BlockSize;

			FSyncFileOptions SyncFileOptions;
			SyncFileOptions.Algorithm			 = Algorithm;
			SyncFileOptions.BlockSize			 = SourceBlockSize;
			SyncFileOptions.ProxyPool			 = &ProxyPool;
			SyncFileOptions.BlockCache			 = BlockCache;
			SyncFileOptions.ScavengeDatabase	 = ScavengeDatabase.get();
			SyncFileOptions.bValidateTargetFiles = SyncOptions.bValidateTargetFiles;
			SyncFileOptions.SourceType			 = SyncOptions.SourceType;

			FFileSyncResult SyncResult =
				SyncFile(Item.NeedList, Item.ResolvedSourceFilePath, SourceBlocks, BaseFile, Item.TargetFilePath, SyncFileOptions);

			LogStatus(Item.TargetFilePath.wstring().c_str(), SyncResult.Succeeded() ? L"Succeeded" : L"Failed");

			if (SyncResult.Succeeded())
			{
				StatSourceBytes += SyncResult.SourceBytes;
				StatBaseBytes += SyncResult.BaseBytes;
				StatInSituBytes += SyncResult.InSituBytes;
				UNSYNC_ASSERT(SyncResult.SourceBytes + SyncResult.BaseBytes == Item.TotalSizeBytes);

				if (!GDryRun)
				{
					BaseFile.Close();
					if (Item.SourceManifest->Mtime)
					{
						SetFileMtime(Item.TargetFilePath, Item.SourceManifest->Mtime);
					}

					if (Item.SourceManifest->bReadOnly)
					{
						SetFileReadOnly(Item.TargetFilePath, true);
					}

					if (Item.SourceManifest->bIsExecutable)
					{
						SetFileExecutable(Item.TargetFilePath, true);
					}
				}

				if (bBackground)
				{
					FBackgroundTaskResult Result;
					Result.TargetFilePath = Item.TargetFilePath;
					Result.SyncResult	  = SyncResult;
					Result.bIsPartialCopy = Item.NeedBytesFromBase != 0;

					std::lock_guard<std::mutex> LockGuard(BackgroundTaskStatMutex);
					BackgroundTaskResults.push_back(Result);
				}	
			}
			else
			{
				if (SyncResult.SystemErrorCode.value())
				{
					UNSYNC_ERROR(L"Sync failed from '%ls' to '%ls'. Status: %ls, system error code: %d %hs",
								 Item.ResolvedSourceFilePath.wstring().c_str(),
								 Item.TargetFilePath.wstring().c_str(),
								 ToString(SyncResult.Status),
								 SyncResult.SystemErrorCode.value(),
								 SyncResult.SystemErrorCode.message().c_str());
				}
				else
				{
					UNSYNC_ERROR(L"Sync failed from '%ls' to '%ls'. Status: %ls.",
								 Item.ResolvedSourceFilePath.wstring().c_str(),
								 Item.TargetFilePath.wstring().c_str(),
								 ToString(SyncResult.Status));
				}

				NumFailedTasks++;
			}
		};

		std::atomic<uint64> NumBackgroundTasks = {};
		std::atomic<uint64> NumForegroundTasks = {};

		FTaskGroup BackgroundTaskGroup = GScheduler->CreateTaskGroup();
		FTaskGroup ForegroundTaskGroup = GScheduler->CreateTaskGroup();

		std::atomic<uint64> BackgroundTaskMemory	   = {};
		std::atomic<uint64> RemainingSourceBytes	   = EstimatedNeedBytesFromSource;

		std::mutex				SchedulerMutex;
		std::condition_variable SchedulerEvent;

		while (!SyncTaskList.empty())
		{
			if (NumForegroundTasks == 0)
			{
				FFileSyncTaskBatch LocalTaskBatch = std::move(SyncTaskList.back());

				SyncTaskList.pop_back();
				++NumForegroundTasks;

				RemainingSourceBytes -= LocalTaskBatch.NeedBytesFromSource;

				ForegroundTaskGroup.run([TaskBatch = std::move(LocalTaskBatch),
										 &SchedulerEvent,
										 &NumForegroundTasks,
										 &FileSyncTaskBody,
										 &ProxyPool,
										 Algorithm,
										 LogVerbose = GLogVerbose]() {
					FLogVerbosityScope VerbosityScope(LogVerbose);

					std::unique_ptr<FBlockCache> BlockCache;
					if (TaskBatch.FileTasks.size() > 1 && ProxyPool.IsValid())
					{
						BlockCache = TaskBatch.CreateBlockCache(ProxyPool, Algorithm.StrongHashAlgorithmId);
					}

					for (const FFileSyncTask* Task : TaskBatch.FileTasks)
					{
						FileSyncTaskBody(*Task, BlockCache.get(), false);
					}

					--NumForegroundTasks;
					SchedulerEvent.notify_one();
				});
				continue;
			}

			const uint32 MaxBackgroundTasks = std::min<uint32>(8, GMaxThreads - 1);

			if (NumBackgroundTasks < MaxBackgroundTasks && (SyncTaskList.front().NeedBytesFromSource < RemainingSourceBytes / 4) &&
				(BackgroundTaskMemory + SyncTaskList.front().TotalSizeBytes < BackgroundTaskMemoryBudget))
			{
				FFileSyncTaskBatch LocalTaskBatch = std::move(SyncTaskList.front());
				SyncTaskList.pop_front();

				BackgroundTaskMemory += LocalTaskBatch.TotalSizeBytes;
				++NumBackgroundTasks;

				RemainingSourceBytes -= LocalTaskBatch.NeedBytesFromSource;

				BackgroundTaskGroup.run([TaskBatch = std::move(LocalTaskBatch),
										 &SchedulerEvent,
										 &NumBackgroundTasks,
										 &FileSyncTaskBody,
										 &BackgroundTaskMemory,
										 &ProxyPool,
										 Algorithm]() {
					FLogVerbosityScope VerbosityScope(false);  // turn off logging from background threads

					std::unique_ptr<FBlockCache> BlockCache;
					if (TaskBatch.FileTasks.size() > 1 && ProxyPool.IsValid())
					{
						BlockCache = TaskBatch.CreateBlockCache(ProxyPool, Algorithm.StrongHashAlgorithmId);
					}

					for (const FFileSyncTask* Task : TaskBatch.FileTasks)
					{
						FileSyncTaskBody(*Task, BlockCache.get(), true);
					}
					BackgroundTaskMemory -= TaskBatch.TotalSizeBytes;
					--NumBackgroundTasks;
					SchedulerEvent.notify_one();
				});

				continue;
			}

			if (GScheduler->ExecuteTasksUntilIdle())
			{
				continue;
			}

			std::unique_lock<std::mutex> SchedulerLock(SchedulerMutex);
			SchedulerEvent.wait(SchedulerLock);
		}

		ForegroundTaskGroup.wait();

		if (NumBackgroundTasks != 0)
		{
			UNSYNC_LOG(L"Waiting for background tasks to complete");
		}
		BackgroundTaskGroup.wait();

		UNSYNC_ASSERT(RemainingSourceBytes == 0);

		bool   bAllBackgroundTasksSucceeded = true;
		uint32 NumBackgroundSyncFiles		= 0;
		uint64 DownloadedBackgroundBytes	= 0;
		for (const FBackgroundTaskResult& Item : BackgroundTaskResults)
		{
			if (Item.SyncResult.Succeeded())
			{
				UNSYNC_VERBOSE2(L"Copied '%ls' (%ls, background)",
								Item.TargetFilePath.wstring().c_str(),
								Item.bIsPartialCopy ? L"partial" : L"full");
				++NumBackgroundSyncFiles;
				DownloadedBackgroundBytes += Item.SyncResult.SourceBytes;
			}
			else
			{
				bAllBackgroundTasksSucceeded = false;
			}
		}

		if (NumBackgroundSyncFiles)
		{
			UNSYNC_VERBOSE(L"Background file copies: %d (%.2f MB)", NumBackgroundSyncFiles, SizeMb(DownloadedBackgroundBytes));
		}

		if (!bAllBackgroundTasksSucceeded)
		{
			for (const FBackgroundTaskResult& Item : BackgroundTaskResults)
			{
				if (!Item.SyncResult.Succeeded())
				{
					UNSYNC_ERROR(L"Failed to copy file '%ls' on background task. Status: %ls, system error code: %d %hs",
								 Item.TargetFilePath.wstring().c_str(),
								 ToString(Item.SyncResult.Status),
								 Item.SyncResult.SystemErrorCode.value(),
								 Item.SyncResult.SystemErrorCode.message().c_str());
				}
			}
			UNSYNC_ERROR(L"Background file copy process failed!");
		}
	}

	const bool bSyncSucceeded = NumFailedTasks.load() == 0;

	if (bSyncSucceeded && SyncOptions.bCleanup)
	{
		UNSYNC_LOG(L"Deleting unnecessary files");
		UNSYNC_LOG_INDENT;
		DeleteUnnecessaryFiles(TargetPath, TargetDirectoryManifest, SourceDirectoryManifest, SyncFilter);
	}

	// Save the source directory manifest on success.
	// It can be used to speed up the diffing process during next sync.
	if (bSyncSucceeded && !GDryRun)
	{
		bool bSaveOk = SaveDirectoryManifest(SourceDirectoryManifest, TargetManifestPath);

		if (!bSaveOk)
		{
			UNSYNC_ERROR(L"Failed to save manifest after sync");
		}
	}

	UNSYNC_LOG(L"Skipped files: %d, full copies: %d, partial copies: %d", StatSkipped, StatFullCopy, StatPartialCopy);
	UNSYNC_LOG(L"Copied from source: %.2f MB, copied from base: %.2f MB (including %.2f MB in situ)", SizeMb(StatSourceBytes), SizeMb(StatBaseBytes), SizeMb(StatInSituBytes));
	UNSYNC_LOG(L"Sync completed %ls", bSyncSucceeded ? L"successfully" : L"with errors (see log for details)");

	double ElapsedSeconds = DurationSec(TimeBegin, TimePointNow());
	UNSYNC_VERBOSE2(L"Sync time: %.2f seconds", ElapsedSeconds);

	if (ProxyPool.IsValid() && ProxyPool.GetFeatures().bTelemetry)
	{
		FTelemetryEventSyncComplete Event;

		Event.ClientVersion		 = GetVersionString();
		Event.Session			 = ProxyPool.GetSessionId();
		Event.Source			 = ConvertWideToUtf8(SourcePath.wstring());
		Event.ClientHostNameHash = GetAnonymizedMachineIdString();
		Event.TotalBytes		 = TotalSourceSize;
		Event.SourceBytes		 = StatSourceBytes;
		Event.BaseBytes			 = StatBaseBytes;
		Event.SkippedFiles		 = StatSkipped;
		Event.FullCopyFiles		 = StatFullCopy;
		Event.PartialCopyFiles	 = StatPartialCopy;
		Event.Elapsed			 = ElapsedSeconds;
		Event.bSuccess			 = bSyncSucceeded;

		ProxyPool.SendTelemetryEvent(Event);
	}

	return bSyncSucceeded;
}

FNeedListSize
ComputeNeedListSize(const FNeedList& NeedList)
{
	FNeedListSize Result = {};

	for (const FNeedBlock& Block : NeedList.Base)
	{
		Result.TotalBytes += Block.Size;
		Result.BaseBytes += Block.Size;
	}
	for (const FNeedBlock& Block : NeedList.Source)
	{
		Result.TotalBytes += Block.Size;
		Result.SourceBytes += Block.Size;
	}

	return Result;
}

const wchar_t*
ToString(EFileSyncStatus Status)
{
	switch (Status)
	{
		default:
			return L"UNKNOWN";
		case EFileSyncStatus::Ok:
			return L"Ok";
		case EFileSyncStatus::ErrorUnknown:
			return L"Unknown error";
		case EFileSyncStatus::ErrorFullCopy:
			return L"Full file copy failed";
		case EFileSyncStatus::ErrorValidation:
			return L"Patched file validation failed";
		case EFileSyncStatus::ErrorFinalRename:
			return L"Final file rename failed";
		case EFileSyncStatus::ErrorTargetFileCreate:
			return L"Target file creation failed";
		case EFileSyncStatus::ErrorBuildTargetFailed:
			return L"Failed to build target";
	}
}

}  // namespace unsync
