// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncBuffer.h"
#include "UnsyncCommon.h"
#include "UnsyncHash.h"
#include "UnsyncProtocol.h"
#include "UnsyncSocket.h"
#include "UnsyncUtil.h"
#include "UnsyncManifest.h"
#include "UnsyncHashTable.h"

#include <stdint.h>
#include <deque>
#include <functional>
#include <unordered_map>
#include <unordered_set>
#include <vector>

namespace unsync {

extern bool GDryRun;
extern bool GExperimental;
extern bool GExperimentalStreaming;

class FProxy;
class FProxyPool;
class FBlockCache;
class FScavengeDatabase;
struct FBuildTargetResult;
struct FRemoteDesc;
struct FIOReader;
struct FIOWriter;
struct FIOReaderWriter;
struct FPackWriteContext;
struct FSyncFilter;

enum class ESourceType : uint8;

struct FIdentityHash32
{
	uint32 operator()(uint32 X) const { return X; }
};

struct FBlockStrongHash
{
	uint32 operator()(const FBlock128& Block) const
	{
		FHash128::Hasher Hasher;
		return Hasher(Block.HashStrong);
	}

	uint32 operator()(const FGenericBlock& Block) const
	{
		FGenericHash::Hasher Hasher;
		return Hasher(Block.HashStrong);
	}
};

struct FBlockStrongHashEq
{
	bool operator()(const FBlock128& A, const FBlock128& B) const { return A.HashStrong == B.HashStrong; }

	bool operator()(const FGenericBlock& A, const FGenericBlock& B) const { return A.HashStrong == B.HashStrong; }
};

struct FCopyCommand
{
	uint64 Size			= 0;
	uint64 SourceOffset = 0;
	uint64 TargetOffset = 0;

	struct FCompareBySourceOffset
	{
		bool operator()(const FCopyCommand& A, const FCopyCommand& B) const { return A.SourceOffset < B.SourceOffset; }
	};
};

struct FNeedBlock : FCopyCommand
{
	FGenericHash Hash = {};
};

struct FReadSchedule
{
	std::vector<FCopyCommand> Blocks;
	std::deque<uint64>		  Requests;	 // unique block request indices sorted small to large
};

FReadSchedule BuildReadSchedule(const std::vector<FNeedBlock>& Blocks);

class FBlockCache
{
public:
	FBuffer							BlockData;
	THashMap<FHash128, FBufferView> BlockMap;  // Decompressed block data by hash
	FTimeDuration					InitDuration; // How long it took to create the cache (i.e. download / read all the blocks)
};

inline uint64
ComputeSize(const std::vector<FNeedBlock>& NeedBlocks)
{
	uint64 Result = 0;
	for (const FNeedBlock& It : NeedBlocks)
	{
		Result += It.Size;
	}
	return Result;
}

struct FNeedList
{
	std::vector<FNeedBlock> Source;
	std::vector<FNeedBlock> Base;
	std::vector<FHash128>	Sequence;
};

struct FPatchCommandList
{
	std::vector<FCopyCommand> Source;
	std::vector<FCopyCommand> Base;
	std::vector<FHash128>	  Sequence;
};

struct FNeedListSize
{
	uint64 SourceBytes = 0;
	uint64 BaseBytes   = 0;
	uint64 TotalBytes  = 0;
};

FNeedListSize ComputeNeedListSize(const FNeedList& NeedList);

struct FBlockSourceInfo
{
	// FPath  FilePath; // TODO: pass this in to allow filtering by name, etc.
	uint64 TotalSize = 0;  // Total size of the input file/buffer from which the block was generated
};

using FOnBlockGenerated = std::function<void(const FGenericBlock& Block, const FBlockSourceInfo& Source, FBufferView Data)>;

struct FComputeBlocksParams
{
	bool			  bNeedBlocks = true;
	uint32			  BlockSize	  = uint32(64_KB);
	FAlgorithmOptions Algorithm;

	bool   bNeedMacroBlocks		= false;
	uint64 MacroBlockTargetSize = 3_MB;
	uint64 MacroBlockMaxSize	= 5_MB;	 // Maximum allowed by Jupiter

	// Callbacks may be called from worker threads
	FOnBlockGenerated OnBlockGenerated;

	bool bAllowThreading = true;

	bool bAllowStreaming = GExperimentalStreaming;
};


std::vector<FCopyCommand> OptimizeNeedList(const std::vector<FNeedBlock>& Input, uint64 MaxMergedBlockSize = 8_MB);

bool IsSynchronized(const FNeedList& NeedList, const FGenericBlockArray& SourceBlocks);

enum class EFileSyncStatus
{
	Ok,
	ErrorUnknown,
	ErrorFullCopy,
	ErrorValidation,
	ErrorFinalRename,
	ErrorTargetFileCreate,
	ErrorBuildTargetFailed,
	ErrorInvalidParameters,
};

const wchar_t* ToString(EFileSyncStatus Status);

struct FFileSyncTask
{
	const FFileManifest* SourceManifest = nullptr;
	const FFileManifest* BaseManifest	= nullptr;
	FPath				 OriginalSourceFilePath;
	FPath				 ResolvedSourceFilePath;
	FPath				 BaseFilePath;
	FPath				 TargetFilePath;
	FPath				 RelativeFilePath;
	FNeedList			 NeedList;

	uint64 NeedBytesFromSource = 0;
	uint64 NeedBytesFromBase   = 0;
	uint64 TotalSizeBytes	   = 0;

	uint32 SourceId = 0;

	bool IsBaseValid() const { return !BaseFilePath.empty(); }
};

struct FFileSyncResult
{
	EFileSyncStatus Status			= EFileSyncStatus::ErrorUnknown;
	std::error_code SystemErrorCode = {};

	bool Succeeded() const { return (uint32)Status < (uint32)EFileSyncStatus::ErrorUnknown; }

	FFileSyncResult& operator+=(const FBuildTargetResult& BuildTargetResult);

	uint64 SourceBytes = 0;
	uint64 BaseBytes   = 0;
	uint64 InSituBytes = 0;
};

struct FSyncFileOptions
{
	FAlgorithmOptions Algorithm;
	uint32			  BlockSize = uint32(64_KB);

	FProxyPool* ProxyPool = nullptr;
	FBlockCache* BlockCache = nullptr;
	FScavengeDatabase* ScavengeDatabase = nullptr;

	bool bValidateTargetFiles = true;  // WARNING: turning this off is intended only for testing/profiling

	ESourceType SourceType = (ESourceType)0;
};

FFileSyncResult SyncFile(const FPath&			   SourceFilePath,
						 const FGenericBlockArray& SourceBlocks,
						 FIOReader&				   BaseDataReader,
						 const FPath&			   TargetFilePath,
						 const FSyncFileOptions&   Options);

FFileSyncResult SyncFile(const FPath&			 SourceFilePath,
						 const FPath&			 BaseFilePath,
						 const FPath&			 TargetFilePath,
						 const FSyncFileOptions& Options);

struct FSyncDirectoryOptions
{
	ESourceType		   SourceType = (ESourceType)0;
	FPath			   Source;			   // remote data location
	FPath			   Target;			   // output target location
	FPath			   Base;			   // base data location, which typically is the same as sync target
	FPath			   ScavengeRoot;	   // base directory where we may want to find reusable blocks
	uint32			   ScavengeDepth = 5;  // how deep to look for unsync manifests
	std::vector<FPath> Overlays;		   // extra source directories to overlay over primary (add extra files, replace existing files)
	FPath			   SourceManifestOverride;	// force the manifest to be read from a specified file instead of source directory
	FSyncFilter*	   SyncFilter = nullptr;	// filter callback for partial sync support
	FProxyPool*		   ProxyPool  = nullptr;
	bool			   bCleanup	  = false;	// whether to cleanup any files in the target directory that are not in the source manifest file
	bool			   bValidateSourceFiles = true;	 // whether to check that all source files declared in the manifest are present/valid
	bool			   bValidateTargetFiles = true;	 // WARNING: turning this off is intended only for testing/profiling
	bool			   bFullDifference = true;	// whether to run full file difference algorithm, even when there is an existing manifest
	bool			   bCheckAvailableSpace		  = true;  // whether to abort the sync if target path does not have enough available space
	uint64			   BackgroundTaskMemoryBudget = 2_GB;
	uint64			   MaxFilesPerTask			  = 1000;
};

bool SyncDirectory(const FSyncDirectoryOptions& SyncOptions);

// #wip-widehash -- temporary helper functions
FBlock128			   ToBlock128(const FGenericBlock& GenericBlock);
std::vector<FBlock128> ToBlock128(FGenericBlockArray& GenericBlocks);

template<typename BlockType>
bool
ValidateBlockListT(const std::vector<BlockType>& Blocks)
{
	uint64 CurrentOffset = 0;
	for (const BlockType& Block : Blocks)
	{
		if (CurrentOffset != Block.Offset)
		{
			UNSYNC_ERROR(L"Found block at unexpected offset. Blocks are expected to be ordered by offset and contiguous.");
			return false;
		}

		CurrentOffset += Block.Size;
	}

	return true;
}

bool ValidateTarget(FIOReader& Reader, const FNeedList& NeedList, EStrongHashAlgorithmID StrongHasher);
bool ValidateTarget(FIOReader& Reader, const FGenericBlockArray& ValidationBlocks, EStrongHashAlgorithmID StrongHasher);

}  // namespace unsync
