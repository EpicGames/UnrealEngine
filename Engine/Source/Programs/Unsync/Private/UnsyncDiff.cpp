// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncDiff.h"
#include "UnsyncChunking.h"
#include "UnsyncCompression.h"
#include "UnsyncFile.h"
#include "UnsyncScan.h"
#include "UnsyncScheduler.h"

namespace unsync {

FNeedList
DiffBlocksVariable(FIOReader&				 BaseDataReader,
				   uint32					 BlockSize,
				   EWeakHashAlgorithmID		 WeakHasher,
				   EStrongHashAlgorithmID	 StrongHasher,
				   const FGenericBlockArray& SourceBlocks)
{
	FGenericBlockArray BaseBlocks = ComputeBlocksVariable(BaseDataReader, BlockSize, WeakHasher, StrongHasher);
	if (!ValidateBlockListT(BaseBlocks))
	{
		UNSYNC_FATAL(L"Base block list validation failed");
	}

	return DiffManifestBlocks(SourceBlocks, BaseBlocks);
}

template<typename BlockType>
FNeedList
DiffManifestBlocksT(const std::vector<BlockType>& SourceBlocks, const std::vector<BlockType>& BaseBlocks)
{
	FNeedList NeedList;

	struct BlockIndexAndCount
	{
		uint64 Index = 0;
		uint64 Count = 0;
	};

	THashMap<typename BlockType::StrongHashType, BlockIndexAndCount> BaseBlockMap;
	THashMap<uint64, uint64>										 BaseBlockByOffset;

	for (uint64 I = 0; I < BaseBlocks.size(); ++I)
	{
		const BlockType& Block			= BaseBlocks[I];
		BaseBlockByOffset[Block.Offset] = I;

		auto Existing = BaseBlockMap.find(Block.HashStrong);
		if (Existing == BaseBlockMap.end())
		{
			BlockIndexAndCount Item;
			Item.Index = I;
			Item.Count = 1;
			BaseBlockMap.insert(std::make_pair(Block.HashStrong, Item));
		}
		else
		{
			Existing->second.Count += 1;
		}
	}

	for (uint64 I = 0; I < SourceBlocks.size(); ++I)
	{
		const BlockType& SourceBlock = SourceBlocks[I];

		auto BaseBlockIt = BaseBlockMap.find(SourceBlock.HashStrong);
		if (BaseBlockIt == BaseBlockMap.end())
		{
			FNeedBlock NeedBlock;
			NeedBlock.Hash		   = SourceBlock.HashStrong;
			NeedBlock.Size		   = SourceBlock.Size;
			NeedBlock.SourceOffset = SourceBlock.Offset;
			NeedBlock.TargetOffset = SourceBlock.Offset;
			NeedList.Source.push_back(NeedBlock);
		}
		else
		{
			BlockIndexAndCount IndexAndCount = BaseBlockIt->second;
			const BlockType&   BaseBlock	 = BaseBlocks[IndexAndCount.Index];

			UNSYNC_ASSERT(BaseBlock.Size == SourceBlock.Size);

			FNeedBlock NeedBlock;
			NeedBlock.Hash		   = BaseBlock.HashStrong;
			NeedBlock.Size		   = BaseBlock.Size;
			NeedBlock.SourceOffset = BaseBlock.Offset;
			NeedBlock.TargetOffset = SourceBlock.Offset;

			const FNeedBlock* LastBaseNeedBlock = NeedList.Base.empty() ? nullptr : &(NeedList.Base.back());

			// Try to preserve contiguous base data reads
			if (LastBaseNeedBlock)
			{
				uint64 LastBlockEnd			  = LastBaseNeedBlock->SourceOffset + LastBaseNeedBlock->Size;
				auto   ConsecutiveBaseBlockIt = BaseBlockByOffset.find(LastBlockEnd);
				if (ConsecutiveBaseBlockIt != BaseBlockByOffset.end())
				{
					const BlockType& ConsecutiveBaseBlock = BaseBlocks[ConsecutiveBaseBlockIt->second];
					if (ConsecutiveBaseBlock.HashStrong == NeedBlock.Hash)
					{
						UNSYNC_ASSERT(NeedBlock.Size == ConsecutiveBaseBlock.Size);
						NeedBlock.SourceOffset = ConsecutiveBaseBlock.Offset;
					}
				}
			}

			NeedList.Base.push_back(NeedBlock);
		}

		NeedList.Sequence.push_back(ToHash128(SourceBlock.HashStrong));	 // #wip-widehash
	}

	return NeedList;
}

FNeedList
DiffManifestBlocks(const FGenericBlockArray& SourceBlocks, const FGenericBlockArray& BaseBlocks)
{
	return DiffManifestBlocksT(SourceBlocks, BaseBlocks);
}

template<typename WeakHasher>
FNeedList
DiffBlocksParallelT(FIOReader&				  BaseDataReader,
					uint32					  BlockSize,
					EStrongHashAlgorithmID	  StrongHasher,
					const FGenericBlockArray& SourceBlocks,
					uint64					  BytesPerTask)
{
	auto TimeBegin = TimePointNow();

	const uint64 BaseDataSize = BaseDataReader.GetSize();

	THashSet<uint32, FIdentityHash32>							  SourceWeakHashSet;
	THashSet<FGenericBlock, FBlockStrongHash, FBlockStrongHashEq> SourceStrongHashSet;

	for (uint32 I = 0; I < uint32(SourceBlocks.size()); ++I)
	{
		SourceWeakHashSet.insert(SourceBlocks[I].HashWeak);
		SourceStrongHashSet.insert(SourceBlocks[I]);
	}

	FNeedList NeedList;

	struct FTask
	{
		uint64														  Offset = 0;
		uint64														  Size	 = 0;
		std::vector<FHash128>										  Sequence;
		THashSet<FGenericBlock, FBlockStrongHash, FBlockStrongHashEq> BaseStrongHashSet;
	};

	BytesPerTask = std::max<uint64>(BlockSize, BytesPerTask);

	std::vector<FTask> Tasks;
	const uint64	   NumTasks = DivUp(BaseDataSize, BytesPerTask);
	Tasks.resize(NumTasks);

	FSchedulerSemaphore IoSemaphore(*GScheduler, 16);
	FTaskGroup			TaskGroup = GScheduler->CreateTaskGroup(&IoSemaphore);

	std::unique_ptr<FAsyncReader> AsyncReader = BaseDataReader.CreateAsyncReader();

	for (uint64 I = 0; I < NumTasks; ++I)
	{
		FTask& Task		 = Tasks[I];
		uint64 TaskBegin = I * BytesPerTask;
		uint64 TaskEnd	 = std::min(TaskBegin + BytesPerTask, BaseDataSize);

		Task.Offset = TaskBegin;
		Task.Size	= TaskEnd - TaskBegin;

		auto ReadCallback =
			[&SourceStrongHashSet, &SourceWeakHashSet, &Tasks, &TaskGroup, BaseDataSize, StrongHasher, BlockSize](FIOBuffer CmdBuffer,
																												  uint64	CmdOffset,
																												  uint64	CmdReadSize,
																												  uint64	CmdUserData)
		{
			TaskGroup.run(
				[&SourceStrongHashSet,
				 &SourceWeakHashSet,
				 &Tasks,
				 CmdBuffer = std::make_shared<FIOBuffer>(std::move(CmdBuffer)),
				 CmdReadSize,
				 CmdUserData,
				 BaseDataSize,
				 StrongHasher,
				 BlockSize]()
				{
					UNSYNC_ASSERT(CmdBuffer->GetSize() == CmdReadSize);

					uint8* TaskBuffer = CmdBuffer->GetData();
					uint64 TaskIndex  = CmdUserData;
					FTask& Task		  = Tasks[TaskIndex];

					UNSYNC_ASSERT(Task.Size == CmdReadSize);

					const uint8* TaskEnd = TaskBuffer + Task.Size;

					const uint32							  MaxWeakHashFalsePositives = 8;
					THashMap<uint32, uint32, FIdentityHash32> WeakHashFalsePositives;
					THashSet<uint32, FIdentityHash32>		  WeakHashBanList;

					auto ScanFn = [&SourceWeakHashSet,
								   &WeakHashBanList,
								   BlockSize,
								   &Task,
								   TaskBuffer,
								   StrongHasher,
								   &SourceStrongHashSet,
								   &WeakHashFalsePositives,
								   TaskEnd,
								   BaseDataSize](const uint8* WindowBegin, const uint8* WindowEnd, uint32 WindowHash)
					{
						uint64 ThisBlockSize = WindowEnd - WindowBegin;

						if (SourceWeakHashSet.find(WindowHash) != SourceWeakHashSet.end() &&
							WeakHashBanList.find(WindowHash) == WeakHashBanList.end())
						{
							UNSYNC_ASSERT(ThisBlockSize <= BlockSize);

							FGenericBlock BaseBlock;
							BaseBlock.Offset	 = Task.Offset + (WindowBegin - TaskBuffer);
							BaseBlock.Size		 = uint32(ThisBlockSize);
							BaseBlock.HashWeak	 = WindowHash;
							BaseBlock.HashStrong = ComputeHash(WindowBegin, ThisBlockSize, StrongHasher);

							auto SourceBlockIt = SourceStrongHashSet.find(BaseBlock);
							if (SourceBlockIt != SourceStrongHashSet.end())
							{
								const FGenericBlock& SourceBlock = *SourceBlockIt;

								Task.BaseStrongHashSet.insert(BaseBlock);
								Task.Sequence.push_back(SourceBlock.HashStrong.ToHash128());  // #wip-widehash

								return true;
							}

							uint32 FalsePositives = WeakHashFalsePositives[WindowHash]++;
							if (FalsePositives >= MaxWeakHashFalsePositives)
							{
								WeakHashBanList.insert(WindowHash);
							}
						}

						return WindowEnd == TaskEnd && (Task.Offset + Task.Size) != BaseDataSize;
					};

					HashScan<WeakHasher>(TaskBuffer, Task.Size, BlockSize, ScanFn);
				});
		};

		AsyncReader->EnqueueRead(Task.Offset, Task.Size, I, ReadCallback);
	}

	AsyncReader->Flush();
	TaskGroup.wait();

	THashSet<FGenericBlock, FBlockStrongHash, FBlockStrongHashEq> BaseStrongHashSet;

	for (FTask& Task : Tasks)
	{
		NeedList.Sequence.insert(NeedList.Sequence.end(), Task.Sequence.begin(), Task.Sequence.end());

		for (const FGenericBlock& Block : Task.BaseStrongHashSet)
		{
			BaseStrongHashSet.insert(Block);
		}
	}

	uint64 NeedBaseBytes   = 0;
	uint64 NeedSourceBytes = 0;

	for (const FGenericBlock& SourceBlock : SourceBlocks)
	{
		FNeedBlock NeedBlock;
		NeedBlock.Size		   = SourceBlock.Size;
		NeedBlock.TargetOffset = SourceBlock.Offset;
		NeedBlock.Hash		   = SourceBlock.HashStrong;

		auto BaseBlockIt = BaseStrongHashSet.find(SourceBlock);
		if (BaseBlockIt != BaseStrongHashSet.end())
		{
			NeedBlock.SourceOffset = BaseBlockIt->Offset;
			NeedList.Base.push_back(NeedBlock);
			NeedBaseBytes += BaseBlockIt->Size;
		}
		else
		{
			NeedBlock.SourceOffset = SourceBlock.Offset;
			NeedList.Source.push_back(NeedBlock);
			NeedSourceBytes += SourceBlock.Size;
		}
	}

	double Duration = DurationSec(TimeBegin, TimePointNow());
	UNSYNC_VERBOSE(L"Done in %.3f sec (%.3f MB / sec)", Duration, SizeMb(double(BaseDataSize) / Duration));

	return NeedList;
}

FNeedList
DiffBlocksParallel(FIOReader&				 BaseDataReader,
				   uint32					 BlockSize,
				   EWeakHashAlgorithmID		 WeakHasher,
				   EStrongHashAlgorithmID	 StrongHasher,
				   const FGenericBlockArray& SourceBlocks,
				   uint64					 BytesPerTask)
{
	switch (WeakHasher)
	{
		case EWeakHashAlgorithmID::Naive:
			return DiffBlocksParallelT<FRollingChecksum>(BaseDataReader, BlockSize, StrongHasher, SourceBlocks, BytesPerTask);
		case EWeakHashAlgorithmID::BuzHash:
			return DiffBlocksParallelT<FBuzHash>(BaseDataReader, BlockSize, StrongHasher, SourceBlocks, BytesPerTask);
		default:
			UNSYNC_FATAL(L"Unexpected weak hash algorithm id");
			return {};
	}
}

FNeedList
DiffBlocks(FIOReader&				 BaseDataReader,
		   uint32					 BlockSize,
		   EWeakHashAlgorithmID		 WeakHasher,
		   EStrongHashAlgorithmID	 StrongHasher,
		   const FGenericBlockArray& SourceBlocks)
{
	const uint64 BytesPerTask = 32_MB;	// <-- reasonably OK balance between accuracy and speed
	// const uint64 bytes_per_task = base_data_size; // <-- run single-threaded
	return DiffBlocksParallel(BaseDataReader, BlockSize, WeakHasher, StrongHasher, SourceBlocks, BytesPerTask);
}

FNeedList
DiffBlocks(const uint8*				 BaseData,
		   uint64					 BaseDataSize,
		   uint32					 BlockSize,
		   EWeakHashAlgorithmID		 WeakHasher,
		   EStrongHashAlgorithmID	 StrongHasher,
		   const FGenericBlockArray& SourceBlocks)
{
	FMemReader BaseReader(BaseData, BaseDataSize);
	return DiffBlocks(BaseReader, BlockSize, WeakHasher, StrongHasher, SourceBlocks);
}

FNeedList
DiffBlocksParallel(const uint8*				 BaseData,
				   uint64					 BaseDataSize,
				   uint32					 BlockSize,
				   EWeakHashAlgorithmID		 WeakHasher,
				   EStrongHashAlgorithmID	 StrongHasher,
				   const FGenericBlockArray& SourceBlocks,
				   uint64					 BytesPerTask)
{
	FMemReader BaseReader(BaseData, BaseDataSize);
	return DiffBlocksParallel(BaseReader, BlockSize, WeakHasher, StrongHasher, SourceBlocks, BytesPerTask);
}

FBuffer
GeneratePatch(const uint8*			 BaseData,
			  uint64				 BaseDataSize,
			  const uint8*			 SourceData,
			  uint64				 SourceDataSize,
			  uint32				 BlockSize,
			  EWeakHashAlgorithmID	 WeakHasher,
			  EStrongHashAlgorithmID StrongHasher,
			  int32					 CompressionLevel)
{
	FBuffer Result;

	FAlgorithmOptions Algorithm;
	Algorithm.ChunkingAlgorithmId	= EChunkingAlgorithmID::FixedBlocks;
	Algorithm.WeakHashAlgorithmId	= WeakHasher;
	Algorithm.StrongHashAlgorithmId = StrongHasher;

	UNSYNC_VERBOSE(L"Computing blocks for source (%.2f MB)", SizeMb(SourceDataSize));
	FGenericBlockArray SourceBlocks = ComputeBlocks(SourceData, SourceDataSize, BlockSize, Algorithm);

	FGenericBlockArray SourceValidation, BaseValidation;

	{
		FLogVerbosityScope VerbosityScope(false);
		SourceValidation = ComputeBlocks(SourceData, SourceDataSize, FPatchHeader::VALIDATION_BLOCK_SIZE, Algorithm);
		BaseValidation	 = ComputeBlocks(BaseData, BaseDataSize, FPatchHeader::VALIDATION_BLOCK_SIZE, Algorithm);
	}

	UNSYNC_VERBOSE(L"Computing difference for base (%.2f MB)", SizeMb(BaseDataSize));
	FNeedList NeedList = DiffBlocks(BaseData, BaseDataSize, BlockSize, WeakHasher, StrongHasher, SourceBlocks);

	if (IsSynchronized(NeedList, SourceBlocks))
	{
		return Result;
	}

	FPatchCommandList PatchCommands;
	PatchCommands.Source = OptimizeNeedList(NeedList.Source);
	PatchCommands.Base	 = OptimizeNeedList(NeedList.Base);

	uint64 NeedFromSource = ComputeSize(NeedList.Source);
	uint64 NeedFromBase	  = ComputeSize(NeedList.Base);
	UNSYNC_VERBOSE(L"Need from source %.2f MB, from base: %.2f MB", SizeMb(NeedFromSource), SizeMb(NeedFromBase));

	FVectorStreamOut Stream(Result);

	FPatchHeader Header;
	Header.SourceSize				 = SourceDataSize;
	Header.BaseSize					 = BaseDataSize;
	Header.NumSourceValidationBlocks = SourceValidation.size();
	Header.NumBaseValidationBlocks	 = BaseValidation.size();
	Header.NumSourceBlocks			 = PatchCommands.Source.size();
	Header.NumBaseBlocks			 = PatchCommands.Base.size();
	Header.BlockSize				 = BlockSize;
	Header.WeakHashAlgorithmId		 = WeakHasher;
	Header.StrongHashAlgorithmId	 = StrongHasher;
	Stream.Write(&Header, sizeof(Header));

	FHash128 HeaderHash = HashBlake3Bytes<FHash128>(Result.Data(), Result.Size());
	Stream.Write(&HeaderHash, sizeof(HeaderHash));

	for (const FGenericBlock& Block : SourceValidation)
	{
		Stream.Write(&Block, sizeof(Block));
	}
	for (const FGenericBlock& Block : BaseValidation)
	{
		Stream.Write(&Block, sizeof(Block));
	}
	for (FCopyCommand& Cmd : PatchCommands.Source)
	{
		Stream.Write(&Cmd, sizeof(Cmd));
	}
	for (FCopyCommand& Cmd : PatchCommands.Base)
	{
		Stream.Write(&Cmd, sizeof(Cmd));
	}

	FHash128 BlockHash = HashBlake3Bytes<FHash128>(Result.Data(), Result.Size());
	Stream.Write(&BlockHash, sizeof(BlockHash));

	for (const FCopyCommand& Cmd : PatchCommands.Source)
	{
		Stream.Write(SourceData + Cmd.SourceOffset, Cmd.Size);
	}

	const uint64 RawPatchSize = Result.Size();
	UNSYNC_VERBOSE(L"Compressing patch (%.2f MB raw)", SizeMb(RawPatchSize));

	Result = Compress(Result.Data(), Result.Size(), CompressionLevel);

	UNSYNC_VERBOSE(L"Compressed patch size: %.2f MB", SizeMb(Result.Size()));

	return Result;
}

}  // namespace unsync
