// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"

namespace unsync {

FNeedList DiffBlocks(const uint8*			   BaseData,
					 uint64					   BaseDataSize,
					 uint32					   BlockSize,
					 EWeakHashAlgorithmID	   WeakHasher,
					 EStrongHashAlgorithmID	   StrongHasher,
					 const FGenericBlockArray& SourceBlocks);

FNeedList DiffBlocksParallel(const uint8*			   BaseData,
							 uint64					   BaseDataSize,
							 uint32					   BlockSize,
							 EWeakHashAlgorithmID	   WeakHasher,
							 EStrongHashAlgorithmID	   StrongHasher,
							 const FGenericBlockArray& SourceBlocks,
							 uint64					   BytesPerTask);

FNeedList DiffBlocks(FIOReader&				   BaseDataReader,
					 uint32					   BlockSize,
					 EWeakHashAlgorithmID	   WeakHasher,
					 EStrongHashAlgorithmID	   StrongHasher,
					 const FGenericBlockArray& SourceBlocks);

FNeedList DiffBlocksParallel(FIOReader&				   BaseDataReader,
							 uint32					   BlockSize,
							 EWeakHashAlgorithmID	   WeakHasher,
							 EStrongHashAlgorithmID	   StrongHasher,
							 const FGenericBlockArray& SourceBlocks,
							 uint64					   BytesPerTask);

FNeedList DiffBlocksVariable(FIOReader&				   BaseDataReader,
							 uint32					   BlockSize,
							 EWeakHashAlgorithmID	   WeakHasher,
							 EStrongHashAlgorithmID	   StrongHasher,
							 const FGenericBlockArray& SourceBlocks);

FNeedList DiffManifestBlocks(const FGenericBlockArray& SourceBlocks, const FGenericBlockArray& BaseBlocks);

FBuffer GeneratePatch(const uint8*			 BaseData,
					  uint64				 BaseDataSize,
					  const uint8*			 SourceData,
					  uint64				 SourceDataSize,
					  uint32				 BlockSize,
					  EWeakHashAlgorithmID	 WeakHasher,
					  EStrongHashAlgorithmID StrongHasher,
					  int32					 CompressionLevel = 3);

}  // namespace unsync
