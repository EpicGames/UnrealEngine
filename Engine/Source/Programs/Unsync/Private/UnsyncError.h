// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncLog.h"

#include <string>
#include <type_traits>
#include <variant>
#include <optional>
#include <atomic>

namespace unsync {

enum class EErrorKind
{
	Unknown,
	App,
	System,
	Http,
};

struct FError
{
	int32		 Code = 0;
	EErrorKind	 Kind = EErrorKind::Unknown;
	std::wstring Context;

	// TODO: add support for chained errors for rich error context reporting
};

inline FError
SystemError(std::wstring&& Context, int32 Code = -1)
{
	return FError{.Code = Code, .Kind = EErrorKind::System, .Context = Context};
}

inline FError
AppError(std::wstring&& Context, int32 Code = -1)
{
	return FError{.Code = Code, .Kind = EErrorKind::App, .Context = Context};
}

inline FError
AppError(std::string&& Context, int32 Code = -1)
{
	extern std::wstring ConvertUtf8ToWide(std::string_view StringUtf8);
	return FError{.Code = Code, .Kind = EErrorKind::App, .Context = ConvertUtf8ToWide(Context)};
}

inline FError
HttpError(std::wstring&& Context, int32 Code)
{
	return FError{.Code = Code, .Kind = EErrorKind::Http, .Context = Context};
}

inline FError
HttpError(std::string&& Context, int32 Code)
{
	extern std::wstring ConvertUtf8ToWide(std::string_view StringUtf8);
	return HttpError(ConvertUtf8ToWide(Context), Code);
}

inline FError
HttpError(int32 Code)
{
	return FError{.Code = Code, .Kind = EErrorKind::Http};
}

struct FEmpty
{
};

template<typename T = FEmpty, typename Et = FError>
struct [[nodiscard]] TResult
{
	// Implicit construction from Error type for convenience
	TResult(Et&& E) : DataOrError(std::forward<Et>(E)) {}

	explicit TResult(T&& Ok) : DataOrError(std::forward<T>(Ok)) {}

	bool IsOk() const { return std::holds_alternative<T>(DataOrError); }

	bool IsError() const { return std::holds_alternative<Et>(DataOrError); }

	const T*  TryData() const { return std::get_if<T>(&DataOrError); }
	const Et* TryError() const { return std::get_if<Et>(&DataOrError); }

	T*	TryData() { return std::get_if<T>(&DataOrError); }
	Et* TryError() { return std::get_if<Et>(&DataOrError); }

	const T& GetData() const
	{
		UNSYNC_ASSERT(IsOk());
		return *TryData();
	}

	const Et& GetError() const
	{
		UNSYNC_ASSERT(IsError());
		return *TryError();
	}

	T& GetData()
	{
		UNSYNC_ASSERT(IsOk());
		return *TryData();
	}

	Et& GetError()
	{
		UNSYNC_ASSERT(IsError());
		return *TryError();
	}

	const T* operator->() const
	{
		UNSYNC_ASSERT(IsOk());
		return TryData();
	}

	const T& operator*() const
	{
		UNSYNC_ASSERT(IsOk());
		return *TryData();
	}

private:
	std::variant<T, Et> DataOrError;
};

template<typename T, typename Te, typename T2>
inline TResult<T, Te>
MoveError(TResult<T2, Te>& X)
{
	return TResult<T, Te>(std::move(X.GetError()));
}

template<typename Tx, typename E = FError>
static TResult<Tx, E>
ResultOk(Tx Ok)
{
	return TResult<Tx, E>(std::forward<Tx>(Ok));
}

template<typename E = FError>
static TResult<FEmpty, E>
ResultOk()
{
	return TResult<FEmpty, E>(FEmpty());
}

struct FAtomicError
{
	std::atomic_flag	  Flag;
	std::optional<FError> Data;

	operator bool() const { return Test(); }

	bool Test() const { return Flag.test(); }
	bool Set(FError&& InData)
	{
		if (Flag.test_and_set() == false)
		{
			Data = std::move(InData);
			return true;
		}
		else
		{
			return false;
		}
	}
};

#define UNSYNC_RETURN_ON_ERROR(x)           \
	{                                       \
		if (x.IsError())                    \
		{                                   \
			return std::move(x.GetError()); \
		}                                   \
	}

}  // namespace unsync
