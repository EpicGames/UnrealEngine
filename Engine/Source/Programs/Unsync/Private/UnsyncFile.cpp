// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncFile.h"
#include "UnsyncCore.h"
#include "UnsyncFilter.h"
#include "UnsyncHash.h"
#include "UnsyncMemory.h"
#include "UnsyncScheduler.h"
#include "UnsyncThread.h"

#include <mutex>

#if UNSYNC_PLATFORM_UNIX
#	include <errno.h>
#	include <fcntl.h>
#	include <sys/stat.h>
#	include <sys/types.h>
#	include <unistd.h>
#endif	// UNSYNC_PLATFORM_UNIX

#if UNSYNC_PLATFORM_WINDOWS
UNSYNC_THIRD_PARTY_INCLUDES_START
#	include <winioctl.h>
UNSYNC_THIRD_PARTY_INCLUDES_END
#endif	// UNSYNC_PLATFORM_WINDOWS

namespace unsync {

bool GForceBufferedFiles = false;

// Windows epoch : 1601-01-01T00:00:00Z
// Unix epoch    : 1970-01-01T00:00:00Z
static constexpr uint64 SECONDS_BETWEEN_WINDOWS_AND_UNIX = 11'644'473'600ull;
static constexpr uint64 NANOS_PER_WINDOWS_TICK			 = 100ull;
static constexpr uint64 WINDOWS_TICKS_PER_SECOND		 = 1'000'000'000ull / NANOS_PER_WINDOWS_TICK;  // each tick is 100ns

struct FCloneCommand
{
	// Number of bytes preceding the aligned range
	uint64 PreBytes = 0;

	// Number of bytes in the aligned range.
	uint64 CloneBytes = 0;

	// Number of bytes after the aligned range.
	uint64 PostBytes = 0;

	uint64 SourcePreBytesOffset = 0;
	uint64 DestPreBytesOffset = 0;

	uint64 SourceCloneOffset = 0;
	uint64 DestCloneOffset = 0;

	uint64 SourcePostBytesOffset = 0;
	uint64 DestPostBytesOffset = 0;

	static std::optional<FCloneCommand> TryCreate(const FCopyCommand& CopyCommand, uint64 AlignBytes);
};

// Construct an FCloneCommand for the given alignment, or nullopt if the clone cannot be aligned.
std::optional<FCloneCommand> FCloneCommand::TryCreate(const FCopyCommand& CopyCommand, uint64 AlignBytes)
{
	if (CopyCommand.SourceOffset % AlignBytes != CopyCommand.TargetOffset % AlignBytes)
	{
		return std::nullopt;
	}

	const uint64 PreBytes = (AlignBytes - (CopyCommand.SourceOffset % AlignBytes)) % AlignBytes;
	const uint64 PostBytes = (CopyCommand.SourceOffset + CopyCommand.Size) % AlignBytes;

	// Check for writes that are too small to clone any blocks.
	if (PreBytes >= CopyCommand.Size || PostBytes >= CopyCommand.Size || CopyCommand.Size < PreBytes + PostBytes + AlignBytes)
	{
		return std::nullopt;
	}

	const uint64 CloneBytes = (CopyCommand.Size - PreBytes - PostBytes);

	UNSYNC_ASSERT(CloneBytes % AlignBytes == 0);
	UNSYNC_ASSERT(PreBytes + CloneBytes + PostBytes == CopyCommand.Size);

	FCloneCommand CloneCommand{
		.PreBytes = PreBytes,
		.CloneBytes = CloneBytes,
		.PostBytes = PostBytes,

		.SourcePreBytesOffset = CopyCommand.SourceOffset,
		.DestPreBytesOffset = CopyCommand.TargetOffset,

		.SourceCloneOffset = CopyCommand.SourceOffset + PreBytes,
		.DestCloneOffset = CopyCommand.TargetOffset + PreBytes,

		.SourcePostBytesOffset = CopyCommand.SourceOffset + PreBytes + CloneBytes,
		.DestPostBytesOffset = CopyCommand.TargetOffset + PreBytes + CloneBytes,
	};

	UNSYNC_ASSERT(CloneCommand.SourceCloneOffset % AlignBytes == 0);
	UNSYNC_ASSERT(CloneCommand.DestCloneOffset % AlignBytes == 0);
	UNSYNC_ASSERT(CloneCommand.SourcePostBytesOffset % AlignBytes == 0);
	UNSYNC_ASSERT(CloneCommand.DestPostBytesOffset % AlignBytes == 0);

	return CloneCommand;
}

// Returns extended absolute path of a form \\?\D:\verylongpath or \\?\UNC\servername\verylongpath
// Expects an absolute path input. Returns original path on non-Windows.
// https://docs.microsoft.com/en-us/windows/win32/fileio/maximum-file-path-limitation
FPath
MakeExtendedAbsolutePath(const FPath& InAbsolutePath)
{
	if (InAbsolutePath.empty())
	{
		return FPath();
	}

#if UNSYNC_PLATFORM_WINDOWS
	UNSYNC_ASSERTF(InAbsolutePath.is_absolute(), L"Input path '%ls' must be absolute", InAbsolutePath.wstring().c_str());
	const std::wstring& InFilenameString = InAbsolutePath.native();
	if (InFilenameString.starts_with(L"\\\\?\\"))
	{
		return InAbsolutePath;
	}
	else if (InFilenameString.starts_with(L"\\\\"))
	{
		return std::wstring(L"\\\\?\\UNC\\") + InFilenameString.substr(2);
	}
	else
	{
		return std::wstring(L"\\\\?\\") + InFilenameString;
	}
#else	// UNSYNC_PLATFORM_WINDOWS
	return InAbsolutePath;
#endif	// UNSYNC_PLATFORM_WINDOWS
}

// Removes \\, \\?\UNC\, \\.\UNC\, \\.\ or \\?\ prefix from a path.
//	\\server\foo\bar -> server\foo\bar
//	\\?\d:\foo\bar -> d:\foo\bar
//	d:\foo\bar -> d:\foo\bar
// Returns original path on non-Windows.
static inline FPathStringView
RemoveUNCPrefix(const FPath& InPath)
{
	FPathStringView InPathString = InPath.native();

#if UNSYNC_PLATFORM_WINDOWS
	if (InPathString.starts_with(L"\\\\?\\UNC\\"))
	{
		return InPathString.substr(8);
	}
	else if (InPathString.starts_with(L"\\\\?\\"))
	{
		return InPathString.substr(4);
	}
	else if (InPathString.starts_with(L"\\\\"))
	{
		return InPathString.substr(2);
	}
	else
#endif
	{
		return InPathString;
	}
}

FPath
RemoveExtendedPathPrefix(const FPath& InPath)
{
	FPathStringView InPathString = InPath.native();
#if UNSYNC_PLATFORM_WINDOWS
	if (InPathString.starts_with(L"\\\\?\\UNC\\"))
	{
		FPathStringView Remainder = InPathString.substr(8);
		std::wstring	Result;
		Result.reserve(Remainder.length() + 2);
		Result += L"\\\\";
		Result += Remainder;
		return FPath(Result);
	}
	else if (InPathString.starts_with(L"\\\\?\\"))
	{
		return InPathString.substr(4);
	}
	else
	{
		return InPathString;
	}
#else	// UNSYNC_PLATFORM_WINDOWS
	return InPathString;
#endif	// UNSYNC_PLATFORM_WINDOWS
}

std::filesystem::file_time_type
FromWindowsFileTime(uint64 Ticks)
{
	using FileTimeDuration = std::filesystem::file_time_type::duration;

	uint64 RawSeconds		 = Ticks / WINDOWS_TICKS_PER_SECOND;
	uint64 RawSubsecondTicks = Ticks - (RawSeconds * WINDOWS_TICKS_PER_SECOND);
	uint64 RawSubsecondNanos = RawSubsecondTicks * NANOS_PER_WINDOWS_TICK;

#if UNSYNC_PLATFORM_WINDOWS
	FileTimeDuration Seconds = std::chrono::duration_cast<FileTimeDuration>(std::chrono::seconds(RawSeconds));
#else	// UNSYNC_PLATFORM_WINDOWS
	FileTimeDuration Seconds = std::chrono::seconds(RawSeconds - SECONDS_BETWEEN_WINDOWS_AND_UNIX);
#endif	// UNSYNC_PLATFORM_WINDOWS

	FileTimeDuration SubsecondNanos = std::chrono::duration_cast<FileTimeDuration>(std::chrono::nanoseconds(RawSubsecondNanos));

	FileTimeDuration DurationFromNativeEpoch = Seconds + SubsecondNanos;

	std::filesystem::file_time_type Result(DurationFromNativeEpoch);

	return Result;
}

FPath
GetRelativePath(const FPath& Path, const FPath& Base)
{
	FPathStringView ResultView = GetRelativePathView(Path, Base);
	return ResultView;
}

FPathStringView
GetRelativePathView(const FPath& Path, const FPath& Base)
{
	// Try a trivial case first, without touching the filesystem
	FPathStringView PathView = RemoveUNCPrefix(Path);
	FPathStringView BaseView = RemoveUNCPrefix(Base);

	if (PathView.starts_with(BaseView))
	{
		FPathStringView PathViewRemainder = PathView.substr(BaseView.length());
		if (PathViewRemainder.starts_with(FPath::preferred_separator))
		{
			FPathStringView RelativePath = PathView.substr(BaseView.length());
			while (RelativePath.starts_with(FPath::preferred_separator))
			{
				RelativePath = RelativePath.substr(1);
			}
			return RelativePath;
		}
	}

	return {};
}

void
ConvertDirectorySeparatorsToNative(std::string& Path)
{
	std::replace_if(Path.begin(), Path.end(), [](char C) { return C == '/' || C == '\\'; }, PATH_SEPARATOR);
}

void
ConvertDirectorySeparatorsToUnix(std::string& Path)
{
	std::replace_if(Path.begin(), Path.end(), [](char C) { return C == '\\'; }, char('/'));
}

void
ConvertDirectorySeparatorsToNative(std::wstring& Path)
{
	std::replace_if(Path.begin(), Path.end(), [](wchar_t C) { return C == '/' || C == '\\'; }, wchar_t(FPath::preferred_separator));
}

void
ConvertDirectorySeparatorsToUnix(std::wstring& Path)
{
	std::replace_if(Path.begin(), Path.end(), [](wchar_t C) { return C == '\\'; }, wchar_t('/'));
}

std::error_code
CopyFileIfNewer(const FPath& Source, const FPath& Target)
{
	FFileAttributes SourceAttr = GetFileAttrib(Source);
	FFileAttributes TargetAttr = GetFileAttrib(Target);
	std::error_code Ec;
	if (SourceAttr.Size != TargetAttr.Size || SourceAttr.Mtime != TargetAttr.Mtime)
	{
		FileCopyOverwrite(Source, Target, Ec);
	}
	return Ec;
}

bool
IsNonCaseSensitiveFileSystem(const FPath& ExistingPath)
{
	UNSYNC_ASSERTF(PathExists(ExistingPath), L"IsCaseSensitiveFileSystem must be called with a path that exists on disk");

	// Assume file system is case-sensitive if all-upper and all-lower versions of the path exist and resolve to the same FS entry.
	// This is not 100% robust due to symlinks, but is good enough for most practical purposes.

	FPath PathUpper = StringToUpper(ExistingPath.wstring());
	FPath PathLower = StringToLower(ExistingPath.wstring());

	if (PathExists(PathUpper) && PathExists(PathLower))
	{
		return std::filesystem::equivalent(ExistingPath, PathUpper) && std::filesystem::equivalent(PathLower, PathUpper);
	}
	else
	{
		return false;
	}
}

bool
IsCaseSensitiveFileSystem(const FPath& ExistingPath)
{
	return !IsNonCaseSensitiveFileSystem(ExistingPath);
}

FFileAttributes
GetCachedFileAttrib(const FPath& Path, FFileAttributeCache& AttribCache)
{
	FFileAttributes Result;

	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);

	auto It = AttribCache.Map.find(ExtendedPath);
	if (It != AttribCache.Map.end())
	{
		Result = It->second;
	}

	return Result;
}

int64
FIOWriter::WriteFrom(FIOReader& Reader, const FCopyCommand& CopyCommand, FBufferView FileData, uint64* NumBytesClonedPtr)
{
	if (FileData.Size != CopyCommand.Size)
	{
		return -1;
	}

	return Write(FileData.Data, CopyCommand.TargetOffset, FileData.Size);
}

#if UNSYNC_PLATFORM_WINDOWS
inline uint64
MakeU64(FILETIME Ft)
{
	return MakeU64(Ft.dwHighDateTime, Ft.dwLowDateTime);
}

struct FCreateFileInfo
{
	DWORD FileAccess  = 0;
	DWORD Share		  = 0;
	DWORD Disposition = 0;
	DWORD Protection  = 0;
	DWORD MapAccess	  = 0;
	DWORD FileFlags	  = FILE_ATTRIBUTE_NORMAL;

	FCreateFileInfo(EFileMode Mode)
	{
		switch (Mode & EFileMode::CommonModeMask)
		{
			default:
			case EFileMode::ReadOnly:
			case EFileMode::ReadOnlyUnbuffered:
				FileAccess	= GENERIC_READ;
				Share		= FILE_SHARE_READ;
				Disposition = OPEN_EXISTING;
				Protection	= PAGE_READONLY;
				MapAccess	= FILE_MAP_READ;
				break;
			case EFileMode::CreateReadWrite:
			case EFileMode::CreateWriteOnly:
				UNSYNC_ASSERT(!GDryRun || EnumHasAnyFlags(Mode, EFileMode::IgnoreDryRun));
				FileAccess	= GENERIC_READ | GENERIC_WRITE;
				Share		= FILE_SHARE_WRITE;
				Disposition = CREATE_ALWAYS;
				Protection	= PAGE_READWRITE;
				MapAccess	= FILE_MAP_ALL_ACCESS;
				break;
		}
	}
};

struct FWindowsAsyncFileReader final : FAsyncReader
{
	UNSYNC_DISALLOW_COPY_ASSIGN(FWindowsAsyncFileReader)

	struct FOverlappedCommand
	{
		OVERLAPPED Overlapped	   = {};
		uint64	   RequestedOffset = 0;
		uint64	   RequestedSize   = 0;
		uint64	   AlignedOffset   = 0;
		uint64	   AlignedSize	   = 0;
		uint64	   Transferred	   = 0;
		uint64	   UserData		   = 0;
		uint32	   ErrorCode	   = 0;
		bool	   bIoActive	   = false;
		bool	   bComplete	   = true;
		FIOBuffer  Buffer;
		IOCallback Callback = {};
	};

	FWindowsAsyncFileReader(FWindowsFile& InReader, uint32 InMaxPipelineDepth);

	virtual ~FWindowsAsyncFileReader();
	virtual uint64 GetSize() override { return FileSize; }
	virtual bool   IsValid() override { return !ErrorCode && !bClosed; }
	virtual bool   EnqueueRead(uint64 SourceOffset, uint64 Size, uint64 UserData, IOCallback Callback);
	virtual void   Flush() override;

	bool BeginReadingNextSegment(FOverlappedCommand& Cmd);
	bool FinishReadingSegment(FOverlappedCommand& Cmd);

	void CompleteReadCommand(FOverlappedCommand& Cmd);

	FWindowsFile& Inner;
	const uint32  MaxQueueDepth;

	static constexpr uint32 MAX_OVERLAPPED_COMMANDS = MAX_IO_PIPELINE_DEPTH;

	HANDLE			   OverlappedEvents[MAX_OVERLAPPED_COMMANDS] = {};
	FOverlappedCommand Commands[MAX_OVERLAPPED_COMMANDS]		 = {};
	uint64			   NumCommandsIssued						 = 0;

	HANDLE			 FileHandle = INVALID_HANDLE_VALUE;
	uint64			 FileSize	= 0;
	FAtomicError	 ErrorCode;
	std::atomic_bool bClosed;
};

FWindowsAsyncFileReader::FWindowsAsyncFileReader(FWindowsFile& InReader, uint32 InMaxPipelineDepth)
: Inner(InReader)
, FileHandle(InReader.FileHandle)
, FileSize(InReader.GetSize())
, MaxQueueDepth(InMaxPipelineDepth)
{
	UNSYNC_ASSERT(IsReadOnly(InReader.Mode));

	if (!InReader.IsValid())
	{
		ErrorCode.Set(SystemError(L"FWindowsAsyncFileReader source file is invalid", InReader.GetError()));
		InReader.GetError();
	}

	for (uint32 I = 0; I < MaxQueueDepth; ++I)
	{
		OverlappedEvents[I]			  = CreateEvent(nullptr, true, true, nullptr);
		Commands[I].Overlapped.hEvent = OverlappedEvents[I];
	}

	Inner.AddAsyncReader(this);
}

FWindowsAsyncFileReader::~FWindowsAsyncFileReader()
{
	Flush();

	for (HANDLE EventHandle : OverlappedEvents)
	{
		if (EventHandle)
		{
			CloseHandle(EventHandle);
		}
	}

	Inner.RemoveAsyncReader(this);
}

FWindowsFile::FWindowsFile(const FPath& InFilename, EFileMode InMode, uint64 InSize) : Mode(InMode)
{
	Filename = MakeExtendedAbsolutePath(InFilename);

	bool bOpenedOk = OpenFileHandle(InMode);

	if (bOpenedOk)
	{
		if (IsReadOnly(InMode))
		{
			LARGE_INTEGER LiFileSize = {};
			bool		  bSizeOk	 = GetFileSizeEx(FileHandle, &LiFileSize);
			if (!bSizeOk)
			{
				LastError = GetLastError();
				return;
			}

			FileSize = LiFileSize.QuadPart;
		}
		else if (IsWritable(InMode) && InSize)
		{
			DWORD BytesReturned = 0;
			BOOL  SparseFileOk	= DeviceIoControl(FileHandle, FSCTL_SET_SPARSE, nullptr, 0, nullptr, 0, &BytesReturned, nullptr);
			if (!SparseFileOk)
			{
				UNSYNC_WARNING(L"Failed to mark file '%ls' as sparse.", Filename.wstring().c_str());
			}

			LARGE_INTEGER LiFileSize = {};
			LiFileSize.QuadPart		 = InSize;
			BOOL SizeOk				 = SetFilePointerEx(FileHandle, LiFileSize, nullptr, FILE_BEGIN);
			if (!SizeOk)
			{
				CloseHandle(FileHandle);
				LastError = GetLastError();
				return;
			}

			BOOL EndOfFileOk = SetEndOfFile(FileHandle);
			if (!EndOfFileOk)
			{
				CloseHandle(FileHandle);
				LastError = GetLastError();
				return;
			}

			FileSize = LiFileSize.QuadPart;
		}
		else if (IsWritable(InMode) && (InSize == 0))
		{
			// nothing to do when creating an empty file
		}
		else
		{
			UNSYNC_ERROR(L"Unexpected file mode %d", (int)InMode);
		}
	}
}
FWindowsFile::~FWindowsFile()
{
	Close();

	std::lock_guard<std::mutex> LockGuard(Mutex);
	UNSYNC_ASSERT(AsyncReaders.empty());
}

void
FWindowsFile::AddAsyncReader(FWindowsAsyncFileReader* Reader)
{
	std::lock_guard<std::mutex> LockGuard(Mutex);
	AsyncReaders.push_back(Reader);
}

void
FWindowsFile::RemoveAsyncReader(FWindowsAsyncFileReader* Reader)
{
	std::lock_guard<std::mutex> LockGuard(Mutex);
	AsyncReaders.erase(std::remove(AsyncReaders.begin(), AsyncReaders.end(), Reader), AsyncReaders.end());
}

bool
FWindowsFile::OpenFileHandle(EFileMode InMode)
{
	FCreateFileInfo Info(InMode);
	Info.FileFlags |= FILE_FLAG_OVERLAPPED;
	if (EnumHasAnyFlags(InMode, EFileMode::Unbuffered) && !GForceBufferedFiles)
	{
		Info.FileFlags |= FILE_FLAG_NO_BUFFERING;
	}

	FileHandle = CreateFileW(Filename.c_str(), Info.FileAccess, Info.Share, nullptr, Info.Disposition, Info.FileFlags, nullptr);

	if (FileHandle == INVALID_HANDLE_VALUE)
	{
		LastError = GetLastError();
		return false;
	}
	else
	{
		return true;
	}
}

bool
FWindowsFile::IsValid()
{
	std::lock_guard<std::mutex> LockGuard(Mutex);
	return FileHandle != INVALID_HANDLE_VALUE;
}

void
FWindowsFile::Close()
{
	std::lock_guard<std::mutex> LockGuard(Mutex);

	FlushAsyncReaders();

	if (FileHandle != INVALID_HANDLE_VALUE)
	{
		CloseHandle(FileHandle);
		FileHandle = INVALID_HANDLE_VALUE;
	}

	for (FWindowsAsyncFileReader* It : AsyncReaders)
	{
		It->bClosed = true;
	}
}

inline void
SetOverlappedOffset(OVERLAPPED& Overlapped, uint64 Offset)
{
	LARGE_INTEGER Pos;
	Pos.QuadPart = Offset;

	Overlapped.Offset	  = Pos.LowPart;
	Overlapped.OffsetHigh = Pos.HighPart;
}

bool
FWindowsAsyncFileReader::FinishReadingSegment(FOverlappedCommand& Cmd)
{
	UNSYNC_ASSERT(Cmd.bIoActive);

	DWORD ReadBytes = 0;

	const BOOL OverlappedResultOk = GetOverlappedResult(FileHandle, &Cmd.Overlapped, &ReadBytes, true);

	Cmd.bIoActive = false;
	Cmd.Transferred += ReadBytes;

	if (OverlappedResultOk)
	{
		return true;
	}
	else
	{
		Cmd.ErrorCode = GetLastError();
		ErrorCode.Set(SystemError(L"GetOverlappedResult failed", Cmd.ErrorCode));
		return false;
	}
}

bool
FWindowsAsyncFileReader::BeginReadingNextSegment(FOverlappedCommand& Cmd)
{
	UNSYNC_ASSERT(!Cmd.bIoActive);

	if (Cmd.Transferred >= Cmd.RequestedSize)
	{
		return false;
	}

	if (EnumHasAnyFlags(Inner.Mode, EFileMode::Unbuffered))
	{
		Cmd.Transferred = AlignDownToMultiplePow2(Cmd.Transferred, FWindowsFile::UNBUFFERED_READ_ALIGNMENT);
	}

	const uint64 NextReadSize = Cmd.AlignedSize - Cmd.Transferred;

	uint8* BufferMemory = reinterpret_cast<uint8*>(Cmd.Buffer.GetMemory());
	UNSYNC_ASSERT(Cmd.Transferred + NextReadSize <= Cmd.Buffer.GetMemorySize());

	ResetEvent(Cmd.Overlapped.hEvent);

	SetOverlappedOffset(Cmd.Overlapped, Cmd.AlignedOffset + Cmd.Transferred);

	if (!ReadFile(FileHandle, BufferMemory + Cmd.Transferred, CheckedNarrow(NextReadSize), nullptr, &Cmd.Overlapped))
	{
		const DWORD LastError = GetLastError();
		if (LastError != ERROR_IO_PENDING)
		{
			Cmd.ErrorCode = LastError;
			ErrorCode.Set(SystemError(L"ReadFile failed", LastError));
			return false;
		}
	}

	Cmd.bIoActive = true;

	return true;
}

void
FWindowsAsyncFileReader::CompleteReadCommand(FOverlappedCommand& Cmd)
{
	UNSYNC_ASSERT(!Cmd.bComplete);

	while (Cmd.bIoActive)
	{
		if (FinishReadingSegment(Cmd))
		{
			BeginReadingNextSegment(Cmd);
		}
	}

	UNSYNC_ASSERTF(
		Cmd.RequestedSize <= Cmd.Transferred,
		L"Expected to read at least %llu bytes, but read %llu [FileSize=%llu, Cmd.AlignedOffset=%llu, Cmd.AlignedSize=%llu, Cmd.ErrorCode=%u]",
		llu(Cmd.RequestedSize),
		llu(Cmd.Transferred),
		llu(FileSize),
		llu(Cmd.AlignedOffset),
		llu(Cmd.AlignedSize),
		Cmd.ErrorCode);

	const uint64 ReadBytesClamped = std::min<uint64>(Cmd.Buffer.GetSize(), Cmd.Transferred);

	if (Cmd.Callback)
	{
		Cmd.Callback(std::move(Cmd.Buffer), Cmd.RequestedOffset, ReadBytesClamped, Cmd.UserData);
	}

	Cmd.bComplete = true;
}

uint64
FWindowsFile::Write(const void* Data, uint64 DestOffset, uint64 TotalSize)
{
	// TODO: !!!!! fire-and-forget asynchronous writes !!!!!

	std::lock_guard<std::mutex> LockGuard(Mutex);

	UNSYNC_ASSERT(IsWritable(Mode));

	if (!IsWriteOnly(Mode))
	{
		FlushAsyncReaders();  // flush any outstanding read requests before writing
	}

	LARGE_INTEGER Pos;
	Pos.QuadPart = DestOffset;

	uint64					WrittenBytes = 0;
	static constexpr uint64 ChunkSize	 = 128_MB;
	uint64					NumChunks	 = DivUp(TotalSize, ChunkSize);

	uint64 SourceOffset = 0;

	for (uint64 I = 0; I < NumChunks; ++I)
	{
		int32	   ThisChunkSize = CheckedNarrow(CalcChunkSize(I, ChunkSize, TotalSize));
		OVERLAPPED Overlapped	 = {};

		Overlapped.Offset	  = Pos.LowPart;
		Overlapped.OffsetHigh = Pos.HighPart;

		BOOL WriteOk = WriteFile(FileHandle, reinterpret_cast<const uint8*>(Data) + SourceOffset, ThisChunkSize, nullptr, &Overlapped);
		if (!WriteOk && GetLastError() != ERROR_IO_PENDING)
		{
			LastError = GetLastError();
			return 0;
		}

		DWORD ChunkWrittenBytes	 = 0;
		BOOL  OverlappedResultOk = TRUE;

		uint32 MaxAttempts = 100000;
		uint32 Attempt	   = 0;
		for (Attempt = 0; Attempt < MaxAttempts; ++Attempt)
		{
			OverlappedResultOk = GetOverlappedResult(FileHandle, &Overlapped, &ChunkWrittenBytes, true);
			if (!OverlappedResultOk || ChunkWrittenBytes != 0)
			{
				break;
			}
			if (ChunkWrittenBytes == 0)
			{
				SchedulerSleep(1);
			}
		}
		if (Attempt == MaxAttempts)
		{
			UNSYNC_ERROR(L"Overlapped file write timed out");
		}

		if (!OverlappedResultOk)
		{
			LastError = GetLastError();
			break;
		}

		WrittenBytes += ChunkWrittenBytes;
		Pos.QuadPart += ChunkWrittenBytes;
		SourceOffset += ChunkWrittenBytes;
	}

	return WrittenBytes;
}

uint64
FWindowsFile::Read(void* Dest, uint64 SourceOffset, uint64 ReadSize)
{
	std::lock_guard<std::mutex> LockGuard(Mutex);

	UNSYNC_ASSERTF(((Mode & EFileMode::Unbuffered) == 0) ||
					   ((SourceOffset % UNBUFFERED_READ_ALIGNMENT == 0) && (ReadSize % UNBUFFERED_READ_ALIGNMENT == 0)),
				   L"Unbuffered files only support Read when offset and size are aligned to 4KB");

	UNSYNC_ASSERT(IsReadable(Mode));

	LARGE_INTEGER Pos;
	Pos.QuadPart = SourceOffset;

	uint64					ReadBytes = 0;
	static constexpr uint64 ChunkSize = 128_MB;
	uint64					NumChunks = DivUp(ReadSize, ChunkSize);

	uint64 DestOffset = 0;

	for (uint64 I = 0; I < NumChunks; ++I)
	{
		uint32	   ThisChunkSize = CheckedNarrow(CalcChunkSize(I, ChunkSize, ReadSize));
		OVERLAPPED Overlapped	 = {};

		Overlapped.Offset	  = Pos.LowPart;
		Overlapped.OffsetHigh = Pos.HighPart;

		BOOL ReadOk =
			ReadFile(FileHandle, reinterpret_cast<uint8*>(Dest) + DestOffset + I * ChunkSize, ThisChunkSize, nullptr, &Overlapped);
		if (!ReadOk && GetLastError() != ERROR_IO_PENDING)
		{
			LastError = GetLastError();
			return 0;
		}

		DWORD ChunkReadBytes	 = 0;
		BOOL  OverlappedResultOk = GetOverlappedResult(FileHandle, &Overlapped, &ChunkReadBytes, true);
		if (!OverlappedResultOk)
		{
			LastError = GetLastError();
			break;
		}

		ReadBytes += ChunkReadBytes;
		Pos.QuadPart += ChunkReadBytes;
		SourceOffset += ChunkReadBytes;
	}

	return ReadBytes;
}

bool
FWindowsAsyncFileReader::EnqueueRead(uint64 SourceOffset, uint64 Size, uint64 UserData, IOCallback Callback)
{
	if (!IsValid())
	{
		return false;
	}

	// Clamp size to the end of the file
	Size = std::min(FileSize, SourceOffset + Size) - SourceOffset;

	const EFileMode FileMode = Inner.Mode;

	UNSYNC_ASSERT(IsReadable(FileMode));

	uint32 CmdIdx = ~0u;

	// Async commands are always strictly ordered
	const uint32 WaitSlotIndex = uint32(NumCommandsIssued % MaxQueueDepth);
	const DWORD	 WaitResult	   = WaitForSingleObject(OverlappedEvents[WaitSlotIndex], INFINITE);
	UNSYNC_ASSERT(WaitResult == WAIT_OBJECT_0);
	CmdIdx = WaitSlotIndex;

	UNSYNC_ASSERT(CmdIdx < MaxQueueDepth);

	FOverlappedCommand& Cmd = Commands[CmdIdx];

	if (!Cmd.bComplete)
	{
		CompleteReadCommand(Cmd);
	}

	Cmd.RequestedOffset = SourceOffset;
	Cmd.RequestedSize	= Size;
	Cmd.UserData		= UserData;
	Cmd.Callback		= Callback;
	Cmd.Transferred		= 0;
	Cmd.ErrorCode		= 0;
	Cmd.bComplete		= false;

	if (EnumHasAnyFlags(FileMode, EFileMode::Unbuffered))
	{
		uint64 OriginalSize	 = Size;
		uint64 OriginalBegin = SourceOffset;
		uint64 OriginalEnd	 = SourceOffset + Size;

		uint64 AlignedBegin = AlignDownToMultiplePow2(OriginalBegin, FWindowsFile::UNBUFFERED_READ_ALIGNMENT);
		uint64 AlignedEnd	= AlignUpToMultiplePow2(OriginalEnd, FWindowsFile::UNBUFFERED_READ_ALIGNMENT);

		uint64 AlignedSize = AlignedEnd - AlignedBegin;

		Cmd.Buffer = FIOBuffer::Alloc(AlignedSize, L"WindowsFile::ReadAsync_aligned");

		Cmd.Buffer.SetDataRange(OriginalBegin - AlignedBegin, OriginalSize);

		Cmd.AlignedOffset = AlignedBegin;
		Cmd.AlignedSize	  = AlignedSize;
	}
	else
	{
		Cmd.Buffer = FIOBuffer::Alloc(Size, L"WindowsFile::ReadAsync");

		Cmd.AlignedOffset = SourceOffset;
		Cmd.AlignedSize	  = Size;
	}

	if (BeginReadingNextSegment(Cmd))
	{
		++NumCommandsIssued;
		return true;
	}
	else
	{
		return false;
	}
}

void
FWindowsAsyncFileReader::Flush()
{
	for (uint32 I = 0; I < MaxQueueDepth; ++I)
	{
		const uint64 CommandIndex = (NumCommandsIssued + I) % MaxQueueDepth;
		if (!Commands[CommandIndex].bComplete)
		{
			WaitForSingleObject(OverlappedEvents[CommandIndex], INFINITE);
			CompleteReadCommand(Commands[CommandIndex]);
		}
	}
}

void
FWindowsFile::FlushAsyncReaders()
{
	for (FWindowsAsyncFileReader* It : AsyncReaders)
	{
		It->Flush();
	}
}

std::unique_ptr<FAsyncReader>
FWindowsFile::CreateAsyncReader(uint32 MaxPipelineDepth)
{
	UNSYNC_ASSERT(IsValid());

	MaxPipelineDepth = std::min(MaxPipelineDepth, FWindowsAsyncFileReader::MAX_OVERLAPPED_COMMANDS);
	return std::unique_ptr<FAsyncReader>(new FWindowsAsyncFileReader(*this, MaxPipelineDepth));
}

bool
CloneFileRange(FWindowsFile& SourceFile, FWindowsFile& DestFile, const FCopyCommand& CopyCommand, FBufferView FileData, uint64* ReflinkedBytesPtr)
{
#if true
	// Disabled due to reports of slowness on NTFS. UE-354599
	return false;
#else // false
	// ReFS has a number of preconditions for block cloning. See
	// https://learn.microsoft.com/en-us/windows-server/storage/refs/block-cloning#functionality-restrictions-and-remarks

	// Block clones must be aligned on cluster boundaries, typically 4 kiB.
	// TODO: Query the filesystem cluster size, since ReFS also supports 64 kiB clusters.
	constexpr uint64 AlignBytes = 4096;
	std::optional<FCloneCommand> OptCloneCommand = FCloneCommand::TryCreate(CopyCommand, AlignBytes);
	if (!OptCloneCommand)
	{
		return false;
	}
	FCloneCommand& CloneCommand = *OptCloneCommand;

	// Check whether the files are on the same filesystem, and the filesystem supports block cloning.
	{
		DWORD SourceVolumeSerialNumber = 0;
		DWORD DestVolumeSerialNumber   = 0;
		DWORD DestVolumeFlags		   = 0;

		if (GetVolumeInformationByHandleW(SourceFile.FileHandle,
										  nullptr,
										  0,
										  &SourceVolumeSerialNumber,
										  nullptr,
										  nullptr,
										  nullptr,
										  0) != TRUE ||
			GetVolumeInformationByHandleW(DestFile.FileHandle,
										  nullptr,
										  0,
										  &DestVolumeSerialNumber,
										  nullptr,
										  &DestVolumeFlags,
										  nullptr,
										  0) != TRUE)
		{
			return false;
		}

		if (SourceVolumeSerialNumber != DestVolumeSerialNumber || (DestVolumeFlags & FILE_SUPPORTS_BLOCK_REFCOUNTING) == 0)
		{
			return false;
		}
	}

	// Ensure target file is sparse if input file is.
	{
		FILE_BASIC_INFO SourceFileInfo{};
		if (GetFileInformationByHandleEx(SourceFile.FileHandle, FileBasicInfo, &SourceFileInfo, sizeof(SourceFileInfo)))
		{
			FILE_BASIC_INFO DestFileInfo{};
			if (GetFileInformationByHandleEx(DestFile.FileHandle, FileBasicInfo, &DestFileInfo, sizeof(DestFileInfo)))
			{
				if ((SourceFileInfo.FileAttributes & FILE_ATTRIBUTE_SPARSE_FILE) !=
					(DestFileInfo.FileAttributes & FILE_ATTRIBUTE_SPARSE_FILE))
				{
					FILE_SET_SPARSE_BUFFER Sparse{};
					Sparse.SetSparse = (SourceFileInfo.FileAttributes & FILE_ATTRIBUTE_SPARSE_FILE) ? TRUE : FALSE;
					DWORD BytesReturned{};
					if (!DeviceIoControl(DestFile.FileHandle,
										 FSCTL_SET_SPARSE,
										 &Sparse,
										 sizeof(Sparse),
										 nullptr,
										 0,
										 &BytesReturned,
										 nullptr))
					{
						UNSYNC_VERBOSE("Unable to set target file sparseness to match source file: %s",
									   std::system_category().message(GetLastError()).c_str());
					}
				}
			}
		}
	}

	// Ensure the output file is big enough
	{
		LARGE_INTEGER ExistingFileSize{};
		if (!GetFileSizeEx(DestFile.FileHandle, &ExistingFileSize))
		{
			return false;
		}

		const int64 NewMinSize = CopyCommand.TargetOffset + CopyCommand.Size;
		if (ExistingFileSize.QuadPart < NewMinSize)
		{
			FILE_END_OF_FILE_INFO EOFInfo{};
			EOFInfo.EndOfFile.QuadPart = NewMinSize;
			if (!SetFileInformationByHandle(DestFile.FileHandle, FileEndOfFileInfo, &EOFInfo, sizeof(EOFInfo)))
			{
				return false;
			}
		}
	}

	DUPLICATE_EXTENTS_DATA Extent{};
	Extent.FileHandle				 = SourceFile.FileHandle;
	Extent.SourceFileOffset.QuadPart = CloneCommand.SourceCloneOffset;
	Extent.TargetFileOffset.QuadPart = CloneCommand.DestCloneOffset;
	Extent.ByteCount.QuadPart		 = CloneCommand.CloneBytes;
	DWORD	BytesReturned{};
	if (DeviceIoControl(DestFile.FileHandle, FSCTL_DUPLICATE_EXTENTS_TO_FILE, &Extent, sizeof(Extent), NULL, 0, &BytesReturned, NULL) != TRUE)
	{
		return false;
	}

	// Write unaligned pre/post data
	if (FileData.Size == CopyCommand.Size)
	{
		// Use file data that was already read into memory.
		if (DestFile.Write(FileData.Data, CloneCommand.DestPreBytesOffset, CloneCommand.PreBytes) != CloneCommand.PreBytes ||
			DestFile.Write(FileData.Data + CloneCommand.PreBytes + CloneCommand.CloneBytes, CloneCommand.DestPostBytesOffset, CloneCommand.PostBytes) != CloneCommand.PostBytes)
		{
			return false;
		}
	}
	else
	{
		// It's preferable for callers to pass in the already-in-memory data via `FileData` rather than re-reading it here.
		uint8 Buffer[AlignBytes];
		if (CloneCommand.PreBytes > 0)
		{
			if (SourceFile.Read(Buffer, CloneCommand.SourcePreBytesOffset, CloneCommand.PreBytes) != CloneCommand.PreBytes ||
				DestFile.Write(Buffer, CloneCommand.DestPreBytesOffset, CloneCommand.PreBytes) != CloneCommand.PreBytes)
			{
				return false;
			}
		}
		if (CloneCommand.PostBytes > 0)
		{
			if (SourceFile.Read(Buffer, CloneCommand.SourcePostBytesOffset, CloneCommand.PostBytes) != CloneCommand.PostBytes ||
				DestFile.Write(Buffer, CloneCommand.DestPostBytesOffset, CloneCommand.PostBytes) != CloneCommand.PostBytes)
			{
				return false;
			}
		}
	}

	if (ReflinkedBytesPtr)
	{
		*ReflinkedBytesPtr += CloneCommand.CloneBytes;
	}

	return true;
#endif // false
}

FFileAttributes
GetFileAttrib(const FPath& Path, FFileAttributeCache* AttribCache)
{
	FFileAttributes Result;

	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);

	if (AttribCache)
	{
		auto It = AttribCache->Map.find(ExtendedPath);
		if (It != AttribCache->Map.end())
		{
			Result = It->second;
			return Result;
		}
	}

	WIN32_FILE_ATTRIBUTE_DATA AttributeData;
	BOOL					  Ok = GetFileAttributesExW(ExtendedPath.c_str(), GetFileExInfoStandard, &AttributeData);
	if (Ok)
	{
		Result.bDirectory = !!(AttributeData.dwFileAttributes & FILE_ATTRIBUTE_DIRECTORY);
		Result.Size		  = MakeU64(AttributeData.nFileSizeHigh, AttributeData.nFileSizeLow);
		Result.Mtime	  = MakeU64(AttributeData.ftLastWriteTime);
		Result.bReadOnly  = (AttributeData.dwFileAttributes & FILE_ATTRIBUTE_READONLY);
		Result.bValid	  = true;
	}

	return Result;
}

uint64
ToWindowsFileTime(const std::filesystem::file_time_type& T)
{
	return T.time_since_epoch().count();
}

uint64
GetAvailableDiskSpace(const FPath& Path)
{
	ULARGE_INTEGER AvailableBytes = {};
	ULARGE_INTEGER TotalBytes	  = {};
	ULARGE_INTEGER FreeBytes	  = {};

	BOOL bOk = GetDiskFreeSpaceExW(Path.native().c_str(), &AvailableBytes, &TotalBytes, &FreeBytes);

	if (bOk)
	{
		return AvailableBytes.QuadPart;
	}
	else
	{
		return ~0ull;
	}
}

#endif	// UNSYNC_PLATFORM_WINDOWS

#if UNSYNC_PLATFORM_UNIX
FUnixFile::FUnixFile(const FPath& InFilename, EFileMode InMode, uint64 in_size) : Filename(InFilename), Mode(InMode)
{
	FileDescriptor = open(
		Filename.native().c_str(),
		IsReadOnly(Mode) ? O_RDONLY : (O_RDWR | O_CREAT | O_TRUNC),
		S_IRUSR | S_IWUSR | S_IRGRP | S_IWGRP | S_IROTH | S_IWOTH);

	if (FileDescriptor == -1)
	{
		LastError = errno;
		return;
	}

	if (IsReadOnly(Mode))
	{
		struct stat stat_buf = {};
		if (fstat(FileDescriptor, &stat_buf) == 0)
		{
			FileSize = stat_buf.st_size;
		}
		else
		{
			LastError = errno;
		}
	}
	else
	{
		// Note: Do not ftruncate() here as it inhibits copy-on-write reflinking.
		// Delayed allocation will minimize fragmentation.
		FileSize = in_size;
	}
}

FUnixFile::~FUnixFile()
{
	Close();
}

void
FUnixFile::Close()
{
	if (FileDescriptor != -1)
	{
		if (close(FileDescriptor) != 0)
		{
			LastError = errno;
			// Notify the user as this could mean file contents are not as expected. 
			UNSYNC_WARNING(L"Failed to close %hs: %hs", Filename.string().c_str(), strerror(LastError));
		}
		FileDescriptor = -1;
	}
}

uint64
FUnixFile::Read(void* dest, uint64 SourceOffset, uint64 ReadSize)
{
	// TODO: handle partial reads (pread returning 0 < x < ReadSize)
	uint64 read_bytes = pread(FileDescriptor, dest, ReadSize, SourceOffset);
	if (read_bytes != ReadSize)
	{
		LastError = errno;
	}
	return read_bytes;
}

uint64
FUnixFile::Write(const void* data, uint64 DestOffset, uint64 WriteSize)
{
	UNSYNC_ASSERT(IsWritable(Mode));

	// TODO: handle partial writes (pwrite returning 0 < x < WriteSize)
	uint64 wrote_bytes = pwrite(FileDescriptor, data, WriteSize, DestOffset);
	if (wrote_bytes != WriteSize)
	{
		LastError = errno;
	}
	return wrote_bytes;
}

bool
CloneFileRange(FUnixFile& SourceFile, FUnixFile& DestFile, const FCopyCommand& CopyCommand, FBufferView FileData, uint64* NumReflinkedBytesPtr)
{
#if UNSYNC_PLATFORM_LINUX
	if (const std::optional<FCloneCommand> OptCloneCommand = FCloneCommand::TryCreate(CopyCommand, 4096))
	{
		const FCloneCommand& CloneCommand = *OptCloneCommand;

		// Most filesystems, including btrfs, will only reflink if the copy is block-aligned. This is also true of the FICLONERANGE ioctl.
		// Unaligned copy_file_range() will succeed, but all file extents are duplicated, not reused.
		// However by copying the aligned range separately, those blocks will be reflinked.

		off_t SourcePreBytesOffset = CloneCommand.SourcePreBytesOffset;
		off_t DestPreBytesOffset = CloneCommand.DestPreBytesOffset;
		off_t SourceCloneOffset = CloneCommand.SourceCloneOffset;
		off_t DestCloneOffset = CloneCommand.DestCloneOffset;
		off_t SourcePostBytesOffset = CloneCommand.SourcePostBytesOffset;
		off_t DestPostBytesOffset = CloneCommand.DestPostBytesOffset;

		ssize_t PreBytesWritten = 0;
		ssize_t CloneBytesWritten = 0;
		ssize_t PostBytesWritten = 0;
		if (
			(PreBytesWritten = copy_file_range(
				SourceFile.FileDescriptor, &SourcePreBytesOffset,
				DestFile.FileDescriptor, &DestPreBytesOffset,
				CloneCommand.PreBytes, 0)) == CloneCommand.PreBytes
			&& 
			(CloneBytesWritten = copy_file_range(
				SourceFile.FileDescriptor, &SourceCloneOffset,
				DestFile.FileDescriptor, &DestCloneOffset,
				CloneCommand.CloneBytes, 0)) == CloneCommand.CloneBytes
			&&
			(PostBytesWritten = copy_file_range(
				SourceFile.FileDescriptor, &SourcePostBytesOffset,
				DestFile.FileDescriptor, &DestPostBytesOffset,
				CloneCommand.PostBytes, 0)) == CloneCommand.PostBytes)
		{
			UNSYNC_ASSERT(PreBytesWritten + CloneBytesWritten + PostBytesWritten == CopyCommand.Size);

			if (NumReflinkedBytesPtr)
			{
				*NumReflinkedBytesPtr += CloneBytesWritten;
			}

			return true;
		}
	}

	off_t SourceOffset = CopyCommand.SourceOffset;
	off_t DestOffset = CopyCommand.TargetOffset;

	ssize_t Written = copy_file_range(
		SourceFile.FileDescriptor,
		&SourceOffset,
		DestFile.FileDescriptor,
		&DestOffset,
		CopyCommand.Size,
		0);

	if (Written == -1)
	{
		UNSYNC_VERBOSE2("copy_file_range failed: %s", strerror(errno));
		return false;
	}

	return Written == CopyCommand.Size;
#else // !UNSYNC_PLATFORM_LINUX
	return false;
#endif // !UNSYNC_PLATFORM_LINUX
}

FFileAttributes
GetFileAttrib(const FPath& Path, FFileAttributeCache* AttribCache)
{
	FFileAttributes Result;

	if (AttribCache)
	{
		auto it = AttribCache->Map.find(Path);
		if (it != AttribCache->Map.end())
		{
			Result = it->second;
			return Result;
		}
	}

	// TODO: could potentially use std::filesystem::directory_entry for this on all platforms

	std::error_code ErrorCode = {};
	auto			Entry	  = std::filesystem::directory_entry(Path, ErrorCode);

	if (!ErrorCode)
	{
		std::filesystem::file_status Status = Entry.status(ErrorCode);
		if (ErrorCode)
		{
			return Result;
		}

		std::filesystem::perms Perms = Status.permissions();

		Result.bDirectory	 = Entry.is_directory();
		Result.Size			 = Result.bDirectory ? 0 : Entry.file_size();
		Result.Mtime		 = ToWindowsFileTime(Entry.last_write_time());
		Result.bReadOnly	 = IsReadOnly(Perms);
		Result.bIsExecutable = IsExecutable(Perms);
		Result.bValid		 = true;
	}

	return Result;
}

uint64
ToWindowsFileTime(const std::filesystem::file_time_type& FileTime)
{
	std::chrono::duration FullDuration = FileTime.time_since_epoch();

	uint64 FullSeconds = std::chrono::floor<std::chrono::seconds>(FullDuration).count();

	auto SubsecondDuration = FullDuration - std::chrono::seconds(FullSeconds);
	auto SubsecondNanos	   = std::chrono::duration_cast<std::chrono::nanoseconds>(SubsecondDuration).count();

	uint64 Ticks = (FullSeconds + SECONDS_BETWEEN_WINDOWS_AND_UNIX) * WINDOWS_TICKS_PER_SECOND + (SubsecondNanos / NANOS_PER_WINDOWS_TICK);

	return Ticks;
}

uint64
GetAvailableDiskSpace(const FPath& Path)
{
	return ~0ull;  // TODO: query available space via statvfs()
}

#endif	// UNSYNC_PLATFORM_UNIX

bool
SetFileMtime(const FPath& Path, uint64 Mtime, bool bAllowInDryRun)
{
	UNSYNC_ASSERT(!GDryRun || bAllowInDryRun);
	UNSYNC_ASSERT(Mtime != 0);

	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);

	std::filesystem::file_time_type FileTime = FromWindowsFileTime(Mtime);

	std::error_code ErrorCode;
	std::filesystem::last_write_time(ExtendedPath, FileTime, ErrorCode);

	return !ErrorCode;
}

bool
SetFileReadOnly(const FPath& Path, bool bReadOnly)
{
	UNSYNC_ASSERT(!GDryRun);

	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);

	std::error_code ErrorCode;

	if (bReadOnly)
	{
		std::filesystem::permissions(
			ExtendedPath,
			std::filesystem::perms::owner_write | std::filesystem::perms::group_write | std::filesystem::perms::others_write,
			std::filesystem::perm_options::remove,
			ErrorCode);
	}
	else
	{
		std::filesystem::permissions(ExtendedPath, std::filesystem::perms::owner_write, std::filesystem::perm_options::add, ErrorCode);
	}

	return !ErrorCode;
}

bool
SetFileExecutable(const FPath& Path, bool Executable)
{
	UNSYNC_ASSERT(!GDryRun);

	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);

	std::error_code ErrorCode;
	// TODO: In general if the +x bit is set, it set for all, this isn't always true though
	// so we should at some point respect what the original file had set. For now though,
	// this is sufficient.
	if (Executable)
	{
		std::filesystem::permissions(
			ExtendedPath,
			std::filesystem::perms::owner_exec | std::filesystem::perms::group_exec | std::filesystem::perms::others_exec,
			std::filesystem::perm_options::add,
			ErrorCode);
	}
	else
	{
		std::filesystem::permissions(
			ExtendedPath,
			std::filesystem::perms::owner_exec | std::filesystem::perms::group_exec | std::filesystem::perms::others_exec,
			std::filesystem::perm_options::remove,
			ErrorCode);
	}

	return !ErrorCode;
}

FBuffer
ReadFileToBuffer(const FPath& Filename)
{
	FBuffer		Result;
	FNativeFile File(Filename, EFileMode::ReadOnly);
	if (File.IsValid())
	{
		Result.Resize(File.GetSize());
		uint64 ReadBytes = File.Read(Result.Data(), 0, Result.Size());
		Result.Resize(ReadBytes);
	}
	return Result;
}

bool
WriteBufferToFile(const FPath& Filename, const uint8* Data, uint64 Size, EFileMode FileMode)
{
	UNSYNC_LOG_INDENT;

	if (Data == nullptr)
	{
		UNSYNC_ERROR(L"WriteBufferToFile called with null buffer");
		return false;
	}
	if (Size == 0)
	{
		UNSYNC_ERROR(L"WriteBufferToFile called with zero size buffer");
		return false;
	}
	if (GDryRun && !EnumHasAnyFlags(FileMode, EFileMode::IgnoreDryRun))
	{
		UNSYNC_ERROR(L"WriteBufferToFile called in dry run mode");
		return false;
	}

	FNativeFile File(Filename, FileMode, Size);

	if (File.IsValid())
	{
		uint64 WroteBytes = File.Write(Data, 0, Size);
		if (WroteBytes != Size)
		{
			UNSYNC_ERROR(L"Failed to write complete file '%ls'. Expected to write %llu bytes, actually written %llu bytes",
						 Filename.wstring().c_str(),
						 llu(Size),
						 llu(WroteBytes));
		}
		return WroteBytes == Size;
	}
	else
	{
		UNSYNC_ERROR(L"Failed to open file '%ls' for writing. %hs",
					 Filename.wstring().c_str(),
					 FormatSystemErrorMessage(File.GetError()).c_str());
		return false;
	}
}

bool
WriteBufferToFile(const FPath& Filename, const FBuffer& Buffer, EFileMode FileMode)
{
	return WriteBufferToFile(Filename, Buffer.Data(), Buffer.Size(), FileMode);
}

bool
WriteBufferToFile(const FPath& Filename, const std::string& Buffer, EFileMode FileMode)
{
	return WriteBufferToFile(Filename, (const uint8*)Buffer.data(), Buffer.length(), FileMode);
}

struct FIOBufferCache
{
	std::mutex Mutex;

	struct FAllocation
	{
		const wchar_t* DebugName = nullptr;
		uint8*		   Memory;
		uint64		   Size;
	};

	std::vector<FAllocation> AllocatedBlocks;  // todo: hash table (though current numbers are very low, so...)
	std::vector<FAllocation> AvailableBlocks;
	uint64					 CurrentCacheSize	  = 0;
	uint64					 CurrentAllocatedSize = 0;

	static constexpr uint64 MAX_CACHED_ALLOC_SIZE = 32_MB;
	static constexpr uint64 MAX_TOTAL_CACHE_SIZE  = 4_GB;

	~FIOBufferCache()
	{
		for (FAllocation& X : AllocatedBlocks)
		{
			UnsyncFree(X.Memory);
		}
		for (FAllocation& X : AvailableBlocks)
		{
			UnsyncFree(X.Memory);
		}
	}

	uint8* Alloc(uint64 Size, const wchar_t* DebugName)
	{
		std::lock_guard<std::mutex> LockGuard(Mutex);

		uint64 BestBlockIndex = ~0ull;
		uint64 BestBlockSize  = ~0ull;

		if (Size <= MAX_CACHED_ALLOC_SIZE)
		{
			Size = NextPow2(CheckedNarrow(Size));
			for (uint64 I = 0; I < AvailableBlocks.size(); ++I)
			{
				FAllocation& Candidate = AvailableBlocks[I];
				if (Candidate.Size < BestBlockSize && Candidate.Size >= Size)
				{
					BestBlockSize  = Candidate.Size;
					BestBlockIndex = I;
				}
			}
		}

		FAllocation Allocation = {};

		if (BestBlockSize != ~0ull)
		{
			FAllocation Candidate = AvailableBlocks[BestBlockIndex];
			AllocatedBlocks.push_back(Candidate);
			AvailableBlocks[BestBlockIndex] = AvailableBlocks.back();
			AvailableBlocks.pop_back();

			Allocation = Candidate;
		}
		else
		{
			CurrentAllocatedSize += Size;

			Allocation.Memory = (uint8*)UnsyncMalloc(Size);
			UNSYNC_ASSERT(Allocation.Memory);

			Allocation.Size = Size;
			AllocatedBlocks.push_back(Allocation);

			if (Size <= MAX_CACHED_ALLOC_SIZE)
			{
				CurrentCacheSize += Allocation.Size;
			}
		}

		return Allocation.Memory;
	}

	void Free(uint8* Ptr)
	{
		std::lock_guard<std::mutex> LockGuard(Mutex);

		uint64 AllocationIndex = ~0u;
		for (uint64 I = 0; I < AllocatedBlocks.size(); ++I)
		{
			if (AllocatedBlocks[I].Memory == Ptr)
			{
				AllocationIndex = I;
				break;
			}
		}

		UNSYNC_ASSERTF(AllocationIndex != ~0u, L"Trying to free an unknown IOBuffer.");

		FAllocation FreedBlock = AllocatedBlocks[AllocationIndex];

		if (FreedBlock.Size <= MAX_CACHED_ALLOC_SIZE)
		{
			AvailableBlocks.push_back(FreedBlock);
		}
		else
		{
			UnsyncFree(FreedBlock.Memory);
			CurrentAllocatedSize -= FreedBlock.Size;
		}

		while (CurrentCacheSize > MAX_TOTAL_CACHE_SIZE && !AvailableBlocks.empty())
		{
			FAllocation& LastBlock = AvailableBlocks.back();
			UnsyncFree(LastBlock.Memory);

			UNSYNC_ASSERT(CurrentCacheSize >= LastBlock.Size);
			CurrentCacheSize -= LastBlock.Size;

			CurrentAllocatedSize -= LastBlock.Size;
			AvailableBlocks.pop_back();
		}

		AllocatedBlocks[AllocationIndex] = AllocatedBlocks.back();
		AllocatedBlocks.pop_back();
	}

	uint64 GetCurrentSize()
	{
		std::lock_guard<std::mutex> LockGuard(Mutex);
		return CurrentCacheSize;
	}
};

static FIOBufferCache GIoBufferCache;

uint8*
AllocIoBuffer(uint64 Size, const wchar_t* DebugName)
{
	return GIoBufferCache.Alloc(Size, DebugName);
}

void
FreeIoBuffer(uint8* Ptr)
{
	GIoBufferCache.Free(Ptr);
}

uint64
GetCurrentIoCacheSize()
{
	return GIoBufferCache.GetCurrentSize();
}

FFileAttributeCache
CreateFileAttributeCache(const FPath& Root, const FSyncFilter* SyncFilter)
{
	FFileAttributeCache Result;

	FTimePoint NextProgressLogTime = TimePointNow() + std::chrono::seconds(1);

	auto ReportProgress = [&NextProgressLogTime, &Result]()
	{
		FTimePoint TimeNow = TimePointNow();
		if (TimeNow >= NextProgressLogTime)
		{
			LogPrintf(ELogLevel::Debug, L"Found files: %d\r", (int)Result.Map.size());
			NextProgressLogTime = TimeNow + std::chrono::seconds(1);
		}
	};

	FPath ResolvedRoot = SyncFilter ? SyncFilter->Resolve(Root) : Root;

	for (const std::filesystem::directory_entry& Dir : RecursiveDirectoryScan(ResolvedRoot))
	{
		if (Dir.is_directory())
		{
			continue;
		}

		if (SyncFilter && !SyncFilter->ShouldSync(Dir.path().native()))
		{
			continue;
		}

		FFileAttributes		   Attr	 = {};
		std::filesystem::perms Perms = Dir.status().permissions();

		Attr.Mtime		   = ToWindowsFileTime(Dir.last_write_time());
		Attr.Size		   = Dir.file_size();
		Attr.bValid		   = true;
		Attr.bReadOnly	   = IsReadOnly(Perms);
		Attr.bIsExecutable = IsExecutable(Perms);

		Result.Map[Dir.path().native()] = Attr;

		ReportProgress();
	}

	ReportProgress();

	return Result;
}

bool
IsDirectory(const FPath& Path)
{
	FFileAttributes Attr = GetFileAttrib(Path);
	return Attr.bValid && Attr.bDirectory;
}

bool
PathExists(const FPath& Path)
{
	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);
	return std::filesystem::exists(ExtendedPath);
}

bool
PathExists(const FPath& Path, std::error_code& OutErrorCode)
{
	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);
	return std::filesystem::exists(ExtendedPath, OutErrorCode);
}

bool
CreateDirectories(const FPath& Path)
{
	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);
	return std::filesystem::create_directories(ExtendedPath);
}

bool
EnsureDirectoryExists(const FPath& Path)
{
	return (PathExists(Path) && IsDirectory(Path)) || CreateDirectories(Path);
}

bool
FileRename(const FPath& From, const FPath& To, std::error_code& OutErrorCode)
{
	FPath ExtendedFrom = MakeExtendedAbsolutePath(From);
	FPath ExtendedTo   = MakeExtendedAbsolutePath(To);
	std::filesystem::rename(ExtendedFrom, ExtendedTo, OutErrorCode);
	return OutErrorCode.value() == 0;
}

bool
FileCopy(const FPath& From, const FPath& To, std::error_code& OutErrorCode)
{
	FPath ExtendedFrom = MakeExtendedAbsolutePath(From);
	FPath ExtendedTo   = MakeExtendedAbsolutePath(To);
	return std::filesystem::copy_file(ExtendedFrom, ExtendedTo, OutErrorCode);
}

bool
FileCopyOverwrite(const FPath& From, const FPath& To, std::error_code& OutErrorCode)
{
	FPath ExtendedFrom = MakeExtendedAbsolutePath(From);
	FPath ExtendedTo   = MakeExtendedAbsolutePath(To);
	return std::filesystem::copy_file(ExtendedFrom, ExtendedTo, std::filesystem::copy_options::overwrite_existing, OutErrorCode);
}

bool
FileRemove(const FPath& Path, std::error_code& OutErrorCode)
{
	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);
	return std::filesystem::remove(ExtendedPath, OutErrorCode);
}

std::filesystem::recursive_directory_iterator
RecursiveDirectoryScan(const FPath& Path)
{
	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);
	return std::filesystem::recursive_directory_iterator(ExtendedPath);
}

std::filesystem::directory_iterator
DirectoryScan(const FPath& Path)
{
	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);
	return std::filesystem::directory_iterator(ExtendedPath);
}

FMemReader::FMemReader(const uint8* InData, uint64 InDataSize) : Data(InData), Size(InDataSize)
{
}

uint64
FMemReader::Read(void* Dest, uint64 SourceOffset, uint64 ReadSize)
{
	uint64 ReadEndOffset   = std::min(SourceOffset + ReadSize, Size);
	uint64 ClampedReadSize = ReadEndOffset - SourceOffset;
	memcpy(Dest, Data + SourceOffset, ClampedReadSize);
	return ClampedReadSize;
}

FMemReaderWriter::FMemReaderWriter(uint8* InData, uint64 InDataSize) : FMemReader(InData, InDataSize), DataRw(InData)
{
}

uint64
FMemReaderWriter::Write(const void* InData, uint64 DestOffset, uint64 WriteSize)
{
	uint64 WriteEndOffset	= std::min(DestOffset + WriteSize, Size);
	uint64 ClampedWriteSize = WriteEndOffset - DestOffset;
	if (ClampedWriteSize && DataRw)
	{
		memcpy(DataRw + DestOffset, InData, ClampedWriteSize);
	}
	return ClampedWriteSize;
}

FIOBuffer
FIOBuffer::Alloc(uint64 Size, const wchar_t* DebugName)
{
	UNSYNC_ASSERT(Size);

	FIOBuffer Result;

	Result.MemoryPtr  = AllocIoBuffer(Size, DebugName);
	Result.MemorySize = Size;

	Result.DataPtr	= Result.MemoryPtr;
	Result.DataSize = Size;

	Result.DebugName = DebugName;

	return Result;
}

FIOBuffer::~FIOBuffer()
{
	Clear();
	UNSYNC_CLOBBER(Canary);
}

FIOBuffer::FIOBuffer(FIOBuffer&& Rhs)
{
	UNSYNC_ASSERT(Rhs.Canary == CANARY);

	std::swap(MemoryPtr, Rhs.MemoryPtr);
	std::swap(MemorySize, Rhs.MemorySize);

	std::swap(DataPtr, Rhs.DataPtr);
	std::swap(DataSize, Rhs.DataSize);
}

void
FIOBuffer::Clear()
{
	UNSYNC_ASSERT(Canary == CANARY);
	if (MemoryPtr)
	{
		FreeIoBuffer(MemoryPtr);

		MemoryPtr  = nullptr;
		MemorySize = 0;

		DataPtr	 = nullptr;
		DataSize = 0;
	}
}

FIOBuffer&
FIOBuffer::operator=(FIOBuffer&& Rhs)
{
	UNSYNC_ASSERT(Canary == CANARY);
	UNSYNC_ASSERT(Rhs.Canary == CANARY);
	if (this != &Rhs)
	{
		std::swap(MemoryPtr, Rhs.MemoryPtr);
		std::swap(MemorySize, Rhs.MemorySize);

		std::swap(DataPtr, Rhs.DataPtr);
		std::swap(DataSize, Rhs.DataSize);

		Rhs.Clear();
	}
	return *this;
}

void
TestFileTime()
{
	UNSYNC_LOG(L"TestFileTime()");
	UNSYNC_LOG_INDENT;

	// 20231024004826Z - 2023 October 24 12:48:26
	// unix 1698108506
	// windows 133425821060000000
	const uint64 BaseExpectedWindowsTime = 133425821060000000ull;

	// Check basic conversion functionality at maximum
	{
		UNSYNC_LOG(L"File time precision estimate:");
		UNSYNC_LOG_INDENT;

		uint64 ExpectedWindowsTime = BaseExpectedWindowsTime + 9999999;

		std::filesystem::file_time_type FileTime = FromWindowsFileTime(ExpectedWindowsTime);

		uint64 RoundTripWindowsTime = ToWindowsFileTime(FileTime);
		uint64 NativeCount			= FileTime.time_since_epoch().count();

		uint64 Delta = ExpectedWindowsTime > RoundTripWindowsTime ? ExpectedWindowsTime - RoundTripWindowsTime
																  : RoundTripWindowsTime - ExpectedWindowsTime;

		UNSYNC_LOG(L"ExpectedWindowsTime  = %llu", llu(ExpectedWindowsTime));
		UNSYNC_LOG(L"RoundTripWindowsTime = %llu", llu(RoundTripWindowsTime));
		UNSYNC_LOG(L"NativeCount = %llu, Delta = %llu", llu(NativeCount), llu(Delta));
	}

	// Check basic conversion functionality at 1 second precision
	{
		uint64 ExpectedWindowsTime = BaseExpectedWindowsTime;

		std::filesystem::file_time_type FileTime = FromWindowsFileTime(ExpectedWindowsTime);

		uint64 RoundTripWindowsTime = ToWindowsFileTime(FileTime);
		uint64 NativeCount			= FileTime.time_since_epoch().count();

		UNSYNC_ASSERTF(RoundTripWindowsTime == ExpectedWindowsTime,
					   L"RoundTripWindowsTime is %llu, but expected to be %llu. Native count: %llu",
					   llu(RoundTripWindowsTime),
					   llu(ExpectedWindowsTime),
					   llu(NativeCount));
	}
}

uint64
BlockingReadLarge(FIOReader& InReader, uint64 Offset, uint64 Size, uint8* OutputBuffer, uint64 OutputBufferSize)
{
	const uint64 BytesPerRead = 2_MB;
	const uint64 ReadEnd	  = std::min(Offset + Size, InReader.GetSize());
	const uint64 ClampedSize  = ReadEnd - Offset;

	std::unique_ptr<FAsyncReader> AsyncReader = InReader.CreateAsyncReader();

	std::atomic<uint64> TotalReadSize = 0;

	if (ClampedSize == 0)
	{
		return TotalReadSize;
	}

	FSchedulerSemaphore IoSemaphore(*GScheduler, 16);
	FTaskGroup			CopyTasks = GScheduler->CreateTaskGroup(&IoSemaphore);

	uint64 NumReads = DivUp(ClampedSize, BytesPerRead);
	for (uint64 ReadIndex = 0; ReadIndex < NumReads; ++ReadIndex)
	{
		const uint64 ThisBatchSize	= CalcChunkSize(ReadIndex, BytesPerRead, ClampedSize);
		const uint64 OutputOffset	= BytesPerRead * ReadIndex;
		const uint64 ThisReadOffset = Offset + OutputOffset;

		auto ReadCallback = [OutputBuffer, OutputBufferSize, &TotalReadSize, &CopyTasks](FIOBuffer CmdBuffer,
																						 uint64	   CmdSourceOffset,
																						 uint64	   CmdReadSize,
																						 uint64	   OutputOffset)
		{
			UNSYNC_ASSERT(OutputOffset + CmdReadSize <= OutputBufferSize);

			CopyTasks.run(
				[OutputBuffer, OutputOffset, CmdReadSize, CmdBuffer = MakeShared(std::move(CmdBuffer)), &TotalReadSize]()
				{
					memcpy(OutputBuffer + OutputOffset, CmdBuffer->GetData(), CmdReadSize);
					TotalReadSize += CmdReadSize;
				});
		};

		AsyncReader->EnqueueRead(ThisReadOffset, ThisBatchSize, OutputOffset, ReadCallback);
	}

	AsyncReader->Flush();
	CopyTasks.wait();

	return TotalReadSize;
}

void
TestFileAttrib()
{
	UNSYNC_LOG(L"TestFileAttrib()");
	UNSYNC_LOG_INDENT;

	FPath TempDirPath = std::filesystem::temp_directory_path() / "unsync_test";
	CreateDirectories(TempDirPath);

	const bool bDirectoryExists = PathExists(TempDirPath) && IsDirectory(TempDirPath);
	UNSYNC_ASSERT(bDirectoryExists);

	const FPath TestFilename = TempDirPath / "attrib.txt";
	UNSYNC_LOG(L"Test file name: %ls", TestFilename.wstring().c_str());

	if (PathExists(TestFilename))
	{
		SetFileReadOnly(TestFilename, false);
	}

	const bool bFileWritten = WriteBufferToFile(TestFilename, "unsync test file");
	UNSYNC_ASSERT(bFileWritten);

	const uint64 ExpectedFileTime = 133425821060000000ull;

	const bool bMtimeSet = SetFileMtime(TestFilename, ExpectedFileTime);
	UNSYNC_ASSERT(bMtimeSet);

	const FFileAttributes FileAttrib = GetFileAttrib(TestFilename);
	UNSYNC_ASSERT(!FileAttrib.bReadOnly);

	UNSYNC_ASSERT(FileAttrib.Mtime == ExpectedFileTime);

	const bool bReadOnlySet = SetFileReadOnly(TestFilename, true);
	UNSYNC_ASSERT(bReadOnlySet);

	const FFileAttributes FileAttribReadOnly = GetFileAttrib(TestFilename);
	UNSYNC_ASSERT(FileAttribReadOnly.bReadOnly);

	const bool bReadOnlyReset = SetFileReadOnly(TestFilename, false);
	UNSYNC_ASSERT(bReadOnlyReset);

	const FFileAttributes FileAttribNonReadOnly = GetFileAttrib(TestFilename);
	UNSYNC_ASSERT(!FileAttribNonReadOnly.bReadOnly);

#if UNSYNC_PLATFORM_UNIX
	const bool bIsExecutable = SetFileExecutable(TestFilename, true);
	UNSYNC_ASSERT(bIsExecutable);

	const FFileAttributes FileAttribExecutable = GetFileAttrib(TestFilename);
	UNSYNC_ASSERT(FileAttribExecutable.bIsExecutable);

	const bool bIsNotExecutable = SetFileExecutable(TestFilename, false);
	UNSYNC_ASSERT(bIsNotExecutable);

	// This part of the test will fail on Windows platforms as the +x bit
	// means nothing there, so bIsExecutable will never be set to true.
	const FFileAttributes FileAttribNotExecutable = GetFileAttrib(TestFilename);
	UNSYNC_ASSERT(!FileAttribNotExecutable.bIsExecutable);
#endif	// UNSYNC_PLATFORM_UNIX

	std::error_code ErrorCode;
	const bool		bFileDeleted = FileRemove(TestFilename, ErrorCode);
	UNSYNC_ASSERT(bFileDeleted);
}

void
TestPathUtil()
{
#if UNSYNC_PLATFORM_WINDOWS

	UNSYNC_LOG(L"TestPathUtil()");
	UNSYNC_LOG_INDENT;

	// Test path manipulation helpers

	{
		FPath Simple   = FPath("\\\\?\\UNC\\server\\subdir\\a\\b\\c");
		FPath Extended = MakeExtendedAbsolutePath(Simple);
		UNSYNC_ASSERT(Simple == Extended);
	}

	{
		FPath Simple   = FPath("\\\\?\\d:\\local\\subdir\\a\\b\\c");
		FPath Extended = MakeExtendedAbsolutePath(Simple);
		UNSYNC_ASSERT(Simple == Extended);
	}

	{
		FPath Simple   = FPath("d:\\local\\subdir\\a\\b\\c");
		FPath Extended = MakeExtendedAbsolutePath(Simple);
		FPath Stripped = RemoveExtendedPathPrefix(Extended);
		UNSYNC_ASSERT(Stripped == Simple);
	}

	{
		FPath Simple   = FPath("\\\\server\\local\\subdir\\a\\b\\c");
		FPath Extended = MakeExtendedAbsolutePath(Simple);
		FPath Stripped = RemoveExtendedPathPrefix(Extended);
		UNSYNC_ASSERT(Stripped == Simple);
	}

	{
		FPath Base	   = FPath("d:\\local\\subdir");
		FPath Full	   = FPath("d:\\local\\subdir\\a\\b\\c");
		FPath Relative = GetRelativePath(Full, Base);
		UNSYNC_ASSERT(Relative == FPath("a\\b\\c"));
	}

	{
		FPath Base	   = FPath("\\\\server\\subdir");
		FPath Full	   = FPath("\\\\server\\subdir\\a\\b\\c");
		FPath Relative = GetRelativePath(Full, Base);
		UNSYNC_ASSERT(Relative == FPath("a\\b\\c"));
	}

	{
		FPath Base	   = FPath("\\\\?\\d:\\local\\subdir");
		FPath Full	   = FPath("\\\\?\\d:\\local\\subdir\\a\\b\\c");
		FPath Relative = GetRelativePath(Full, Base);
		UNSYNC_ASSERT(Relative == FPath("a\\b\\c"));
	}

	{
		FPath Base	   = FPath("\\\\?\\d:\\local\\subdir");
		FPath Full	   = FPath("d:\\local\\subdir\\a\\b\\c");
		FPath Relative = GetRelativePath(Full, Base);
		UNSYNC_ASSERT(Relative == FPath("a\\b\\c"));
	}

	{
		FPath Base	   = FPath("d:\\local\\subdir");
		FPath Full	   = FPath("\\\\?\\d:\\local\\subdir\\a\\b\\c");
		FPath Relative = GetRelativePath(Full, Base);
		UNSYNC_ASSERT(Relative == FPath("a\\b\\c"));
	}

	{
		FPath Base	   = FPath("d:\\local\\subdir");
		FPath Full	   = FPath("\\\\?\\e:\\local\\subdir\\a\\b\\c");
		FPath Relative = GetRelativePath(Full, Base);
		UNSYNC_ASSERT(Relative.empty());
	}

	{
		FPath Base	   = FPath("d:\\local\\subdir");
		FPath Full	   = FPath("d:\\local\\a\\b\\c");
		FPath Relative = GetRelativePath(Full, Base);
		UNSYNC_ASSERT(Relative.empty());
	}

#endif	// UNSYNC_PLATFORM_WINDOWS
}

void
DeleteOldFilesInDirectory(const FPath& Path, uint32 MaxFilesToKeep, bool bAllowInDryRun, const FPathFilterCallback& Filter)
{
	struct FEntry
	{
		FPath  Path;
		uint64 Mtime;
	};

	std::vector<FEntry> Entries;

	FPath ExtendedPath = MakeExtendedAbsolutePath(Path);
	for (const std::filesystem::directory_entry& It : std::filesystem::directory_iterator(ExtendedPath))
	{
		if (Filter && !Filter(It.path()))
		{
			continue;
		}

		if (It.is_regular_file())
		{
			FEntry Entry;
			Entry.Mtime = ToWindowsFileTime(It.last_write_time());
			Entry.Path	= It.path();
			Entries.push_back(Entry);
		}
	}

	// reverse sort
	std::sort(Entries.begin(), Entries.end(), [](const FEntry& A, const FEntry& B) { return A.Mtime > B.Mtime; });

	while (Entries.size() > MaxFilesToKeep)
	{
		const FEntry& Oldest = Entries.back();

		std::wstring PathStr = RemoveExtendedPathPrefix(Oldest.Path).wstring();

		if (GDryRun && !bAllowInDryRun)
		{
			UNSYNC_VERBOSE(L"Deleting '%ls'(skipped due to dry run mode)", PathStr.c_str());
		}
		else
		{
			UNSYNC_VERBOSE(L"Deleting '%ls'", PathStr.c_str());
			std::error_code ErrorCode = {};
			FileRemove(Oldest.Path, ErrorCode);
		}

		Entries.pop_back();
	}
}

std::chrono::system_clock::time_point
SystemTimeFromFileTime(std::filesystem::file_time_type FileTime)
{
#if 0  // TODO: use clock_cast() when it's supported in all target compilers

	return std::chrono::clock_cast<std::chrono::system_clock>(FileTime);

#else

#	if UNSYNC_PLATFORM_WINDOWS
	std::chrono::seconds Offset(SECONDS_BETWEEN_WINDOWS_AND_UNIX);
#	else	// UNSYNC_PLATFORM_WINDOWS
	std::chrono::seconds Offset(0);
#	endif	// UNSYNC_PLATFORM_WINDOWS

	std::chrono::duration SinceEpoch = FileTime.time_since_epoch() - Offset;
	return std::chrono::system_clock::time_point(std::chrono::duration_cast<std::chrono::system_clock::duration>(SinceEpoch));

#endif	// 0
}

FDummyAsyncReader::FDummyAsyncReader(FIOReader& InReader) : Inner(InReader)
{
}

bool
FDummyAsyncReader::EnqueueRead(uint64 SourceOffset, uint64 Size, uint64 UserData, IOCallback Callback)
{
	if (Size != 0)
	{
		FIOBuffer Buffer   = FIOBuffer::Alloc(Size, L"FDummyAsyncReader::ReadAsync");
		uint64	  ReadSize = Inner.Read(Buffer.GetData(), SourceOffset, Size);
		Callback(std::move(Buffer), SourceOffset, ReadSize, UserData);
		return true;
	}
	else
	{
		return false;
	}
}

std::unique_ptr<FAsyncReader>
FIOReader::CreateAsyncReader(uint32 MaxPipelineDepth)
{
	return std::unique_ptr<FDummyAsyncReader>(new FDummyAsyncReader(*this));
}

void
TestFileAsyncRead()
{
	UNSYNC_LOG(L"TestFileAsyncRead()");
	UNSYNC_LOG_INDENT;

	UNSYNC_LOG(L"Initializing test data");

	const FPath TempDirPath = std::filesystem::temp_directory_path() / "unsync_test";
	CreateDirectories(TempDirPath);

	const bool bDirectoryExists = PathExists(TempDirPath) && IsDirectory(TempDirPath);
	UNSYNC_ASSERT(bDirectoryExists);

	const FPath		 TestFilename = TempDirPath / "ordered_integers.bin";
	constexpr uint64 TestFileSize = 1_GB;

	FBuffer TempBuffer;
	TempBuffer.Resize(TestFileSize);
	uint32* TempBufferData = reinterpret_cast<uint32*>(TempBuffer.Data());

	const uint32 NumElements = uint32(TestFileSize / sizeof(uint32));

	for (uint32 i = 0; i < NumElements; ++i)
	{
		TempBufferData[i] = i;
	}

	const FHash256 ExpectedHash = HashBlake3Bytes<FHash256>(reinterpret_cast<const uint8*>(TempBufferData), TestFileSize);

	if (!PathExists(TestFilename))
	{
		UNSYNC_LOG(L"Writing test file '%ls'", TestFilename.wstring().c_str());
		const bool bFileWritten = WriteBufferToFile(TestFilename, TempBuffer);
		UNSYNC_ASSERT(bFileWritten);
	}

	UNSYNC_LOG(L"ReadOnlyUnbuffered");

	{
		UNSYNC_LOG_INDENT;

		memset(TempBufferData, 0, TestFileSize);

		FNativeFile					  TestFile(TestFilename, EFileMode::ReadOnlyUnbuffered);
		std::unique_ptr<FAsyncReader> TestFileReader = TestFile.CreateAsyncReader();

		constexpr uint64 ChunkSize = 8_MB;
		static_assert(TestFileSize % ChunkSize == 0);

		UNSYNC_LOG(L"Reading test data");

		const FTimePoint ReadStartTime = TimePointNow();

		const uint64 NumChunks = TestFileSize / ChunkSize;

		auto IoCallback = [TempBufferData](FIOBuffer Buffer, uint64 SourceOffset, uint64 ReadSize, uint64 UserData)
		{ memcpy(reinterpret_cast<uint8*>(TempBufferData) + SourceOffset, Buffer.GetData(), Buffer.GetSize()); };

		for (uint32 ChunkIndex = 0; ChunkIndex < NumChunks; ++ChunkIndex)
		{
			uint64 ChunkOffset = ChunkIndex * ChunkSize;

			TestFileReader->EnqueueRead(ChunkOffset, ChunkSize, 0, IoCallback);
		}

		TestFileReader->Flush();

		const FTimePoint ReadDoneTime = TimePointNow();

		UNSYNC_LOG(L"Hashing test data");

		const FHash256 ActualHash = HashBlake3Bytes<FHash256>(reinterpret_cast<const uint8*>(TempBufferData), TestFileSize);

		const FTimePoint HashDoneTime = TimePointNow();

		const double ReadDuration  = DurationSec(ReadStartTime, ReadDoneTime);
		const double HashDuration  = DurationSec(ReadDoneTime, HashDoneTime);
		const double TotalDuration = DurationSec(ReadStartTime, HashDoneTime);

		UNSYNC_LOG(L"Read rate: %.2f MB / sec", SizeMb(TestFileSize) / ReadDuration);
		UNSYNC_LOG(L"Hash rate: %.2f MB / sec", SizeMb(TestFileSize) / HashDuration);
		UNSYNC_LOG(L"Total rate: %.2f MB / sec", SizeMb(TestFileSize) / TotalDuration);
		UNSYNC_LOG(L"Total time: %.3f sec", TotalDuration);

		UNSYNC_ASSERT(ActualHash == ExpectedHash);
	}

	UNSYNC_LOG(L"ReadOnlyUnbufferedStreaming");

	{
		UNSYNC_LOG_INDENT;

		memset(TempBufferData, 0, TestFileSize);

		FNativeFile					  TestFile(TestFilename, EFileMode::ReadOnlyUnbuffered);
		std::unique_ptr<FAsyncReader> TestFileReader = TestFile.CreateAsyncReader();

		constexpr uint64 ChunkSize = 1_MB;
		static_assert(TestFileSize % ChunkSize == 0);

		UNSYNC_LOG(L"Reading test data");

		const FTimePoint ReadStartTime = TimePointNow();

		const uint64 NumChunks = TestFileSize / ChunkSize;

		FBlake3Hasher Hasher;

		uint64 CurrentOffset = 0;

		auto IoCallback = [TempBufferData, &Hasher, &CurrentOffset](FIOBuffer Buffer, uint64 SourceOffset, uint64 ReadSize, uint64 UserData)
		{
			UNSYNC_ASSERT(CurrentOffset == SourceOffset);
			UNSYNC_ASSERT(ReadSize == Buffer.GetSize());
			CurrentOffset += ReadSize;
			Hasher.Update(Buffer.GetData(), Buffer.GetSize());
		};

		for (uint32 ChunkIndex = 0; ChunkIndex < NumChunks; ++ChunkIndex)
		{
			uint64 ChunkOffset = ChunkIndex * ChunkSize;
			TestFileReader->EnqueueRead(ChunkOffset, ChunkSize, 0, IoCallback);
		}

		TestFileReader->Flush();
		const FTimePoint ReadDoneTime = TimePointNow();

		FHash256 ActualHash = Hasher.Finalize();

		const FTimePoint HashDoneTime = TimePointNow();

		const double TotalDuration = DurationSec(ReadStartTime, HashDoneTime);

		UNSYNC_LOG(L"Read + Hash rate: %.2f MB / sec", SizeMb(TestFileSize) / TotalDuration);
		UNSYNC_LOG(L"Total time: %.3f sec", TotalDuration);

		UNSYNC_ASSERT(ActualHash == ExpectedHash);
	}
}

}  // namespace unsync
