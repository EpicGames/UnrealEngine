// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncBuffer.h"
#include "UnsyncCore.h"
#include "UnsyncUtil.h"

#include <algorithm>
#include <atomic>
#include <functional>
#include <memory>
#include <mutex>

namespace unsync {

extern bool GForceBufferedFiles;

static constexpr uint32 MAX_IO_PIPELINE_DEPTH = 16;

enum class EFileMode : uint32 {
	None	   = 0,

	Read	   = 1 << 0,
	Write	   = 1 << 1,
	Create	   = 1 << 2,
	Unbuffered = 1 << 3,

	// Extended modes
	IgnoreDryRun = 1 << 4,	// allow write operations even in dry run mode

	// Commonly used mode combinations
	ReadOnly				  = Read,
	ReadOnlyUnbuffered		  = Read | Unbuffered,
	CreateReadWrite			  = Read | Write | Create,
	CreateWriteOnly			  = Write | Create,

	// Masks
	CommonModeMask	 = Create | Read | Write | Unbuffered,
	ExtendedModeMask = ~CommonModeMask,
};
UNSYNC_ENUM_CLASS_FLAGS(EFileMode, uint32)

inline bool
IsReadOnly(EFileMode Mode)
{
	switch (Mode & EFileMode::CommonModeMask)
	{
		case EFileMode::ReadOnly:
		case EFileMode::ReadOnlyUnbuffered:
			return true;
		default:
			return false;
	}
}

inline bool
IsWriteOnly(EFileMode Mode)
{
	return (Mode & EFileMode::Read) == 0;
}

inline bool
IsReadable(EFileMode Mode)
{
	return (Mode & EFileMode::Read) != 0;
}

inline bool
IsWritable(EFileMode Mode)
{
	return (Mode & EFileMode::Write) != 0;
}

struct FIOBuffer
{
	static FIOBuffer Alloc(uint64 Size, const wchar_t* DebugName);
	FIOBuffer() = default;
	FIOBuffer(FIOBuffer&& Rhs);
	FIOBuffer(const FIOBuffer& Rhs) = delete;
	FIOBuffer& operator=(const FIOBuffer& Rhs) = delete;
	FIOBuffer& operator						   =(FIOBuffer&& Rhs);
	~FIOBuffer();

	uint8* GetData() const
	{
		UNSYNC_ASSERT(Canary == CANARY);
		return DataPtr;
	}
	uint64 GetSize() const
	{
		UNSYNC_ASSERT(Canary == CANARY);
		return DataSize;
	}

	uint64 GetMemorySize() const
	{
		UNSYNC_ASSERT(Canary == CANARY);
		return MemorySize;
	}

	void Clear();

	void SetDataRange(uint64 Offset, uint64 Size)
	{
		UNSYNC_ASSERT(Offset + Size <= MemorySize);
		DataPtr	 = MemoryPtr + Offset;
		DataSize = Size;
	}

	void* GetMemory() const
	{
		UNSYNC_ASSERT(Canary == CANARY);
		return MemoryPtr;
	}

	FBufferView GetBufferView() const { return FBufferView{GetData(), GetSize()}; }
	FMutBufferView GetMutBufferView() { return FMutBufferView{GetData(), GetSize()}; }

private:
	static constexpr uint64 CANARY = 0x67aced0423000de5ull;

	uint64		   Canary	  = CANARY;
	uint8*		   MemoryPtr  = nullptr;
	uint64		   MemorySize = 0;
	uint8*		   DataPtr	  = nullptr;
	uint64		   DataSize	  = 0;
	const wchar_t* DebugName  = nullptr;
};

inline std::shared_ptr<FIOBuffer>
MakeShared(FIOBuffer&& Buffer)
{
	return std::make_shared<FIOBuffer>(std::forward<FIOBuffer>(Buffer));
}

using IOCallback = std::function<void(FIOBuffer Buffer, uint64 SourceOffset, uint64 ReadSize, uint64 UserData)>;

struct FIOBase
{
	virtual ~FIOBase()		  = default;
	virtual uint64 GetSize()  = 0;
	virtual bool   IsValid()  = 0;
	virtual void   Close()	  = 0;
	virtual int32  GetError() = 0;
};

struct FAsyncReader
{
	virtual ~FAsyncReader()																			   = default;
	virtual uint64 GetSize()																		   = 0;
	virtual bool   IsValid()																		   = 0;
	virtual bool   EnqueueRead(uint64 SourceOffset, uint64 Size, uint64 UserData, IOCallback Callback) = 0; // NOT thread-safe
	virtual void   Flush()																			   = 0; // NOT thread-safe
};

struct FIOReader : virtual FIOBase
{
	virtual uint64 Read(void* Dest, uint64 SourceOffset, uint64 Size) = 0;

	virtual std::unique_ptr<FAsyncReader> CreateAsyncReader(uint32 MaxPipelineDepth = MAX_IO_PIPELINE_DEPTH);

	// Get the actual reader, eg. for a deferred reader that has an inner file reader.
	virtual FIOReader& ActualReader() { return *this; }
};

struct FDummyAsyncReader final : FAsyncReader
{
	FDummyAsyncReader(FIOReader& InReader);

	virtual uint64 GetSize() override { return Inner.GetSize(); }
	virtual bool   IsValid() override { return Inner.IsValid(); }
	virtual bool   EnqueueRead(uint64 SourceOffset, uint64 Size, uint64 UserData, IOCallback Callback) override;
	virtual void   Flush() override {};

private:
	FIOReader& Inner;
};

struct FIOWriter : virtual FIOBase
{
	virtual uint64 Write(const void* Data, uint64 DestOffset, uint64 Size) = 0;

	// Write file data from Reader. File-backed writers may use block cloning to copy the data by reference.
	// FileData.Size must equal CopyCommand.Size.
	// NumBytesClonedPtr may be null. Otherwise the number of bytes cloned is added to the target variable.
	virtual int64 WriteFrom(FIOReader& Reader, const FCopyCommand& CopyCommand, FBufferView FileData, uint64* NumBytesClonedPtr = nullptr);
};

struct FIOReaderWriter : FIOReader, FIOWriter
{
};

// File reader/writer that uses CRTP to implement block-cloned copies.
template <typename InFileType>
class FFileReaderWriter : public FIOReaderWriter
{
public:
	using FileType = InFileType;

	virtual int64 WriteFrom(FIOReader& Reader, const FCopyCommand& CopyCommand, FBufferView FileData, uint64* NumBytesClonedPtr) override
	{
		FileType* FileReader = dynamic_cast<FileType*>(&Reader.ActualReader());
		if (FileReader && FileReader->IsValid())
		{
			// Attempt block cloning
			FileType* FileWriter = dynamic_cast<FileType*>(this);
			UNSYNC_ASSERT(FileWriter);

			if (CloneFileRange(*FileReader, *FileWriter, CopyCommand, FileData, NumBytesClonedPtr))
			{
				return CopyCommand.Size;
			}
		}

		if (FileData.Size != CopyCommand.Size)
		{
			return -1;
		}

		return Write(FileData.Data, CopyCommand.TargetOffset, FileData.Size);
	}
};

#if UNSYNC_PLATFORM_WINDOWS
struct FWindowsFile : FFileReaderWriter<FWindowsFile>
{
	friend struct FWindowsAsyncFileReader;

	explicit FWindowsFile(const FPath& Filename, EFileMode Mode = EFileMode::ReadOnly, uint64 InSize = 0);
	~FWindowsFile();

	// IOBase
	virtual uint64 GetSize() override { return FileSize; }
	virtual bool   IsValid() override;
	virtual void   Close() override;
	virtual int32  GetError() override { return LastError; }

	// IORead
	virtual uint64						  Read(void* Dest, uint64 SourceOffset, uint64 ReadSize) override;
	virtual std::unique_ptr<FAsyncReader> CreateAsyncReader(uint32 MaxPipelineDepth = MAX_IO_PIPELINE_DEPTH) override;

	// IOWrite
	virtual uint64 Write(const void* InData, uint64 DestOffset, uint64 WriteSize) override;

	uint64 FileSize	  = 0;
	HANDLE FileHandle = INVALID_HANDLE_VALUE;
	int32  LastError  = 0;

	FPath Filename;

	static constexpr uint32 UNBUFFERED_READ_ALIGNMENT = 4096;

private:

	// All internal methods expect the Mutex to be locked
	bool   OpenFileHandle(EFileMode InMode);

private:
	EFileMode Mode;

	std::mutex Mutex;

	void								  FlushAsyncReaders();
	void								  AddAsyncReader(FWindowsAsyncFileReader* Reader);
	void								  RemoveAsyncReader(FWindowsAsyncFileReader* Reader);
	std::vector<FWindowsAsyncFileReader*> AsyncReaders;
};

// Clone file blocks via ref counting if possible.
// FileData optionally supplies an in-memory view of the file contents that's used for writing unaligned bytes at the beginning and end of the cloned range.
bool CloneFileRange(FWindowsFile& SourceFile, FWindowsFile& DestFile, const FCopyCommand& CopyCommand, FBufferView FileData = FBufferView(), uint64* NumBytesClonedPtr = nullptr);

using FNativeFile = FWindowsFile;
#endif	// UNSYNC_PLATFORM_WINDOWS

#if UNSYNC_PLATFORM_UNIX
struct FUnixFile;

// Copy file data using in-kernel copy, which also provides block cloning (reflinking) on filesystems that support it.
// The `FileData` parameter is not used; it's only for API compatiblity with the Windows CloneFileRange() function.
bool CloneFileRange(FUnixFile& SourceFile, FUnixFile& DestFile, const FCopyCommand& CopyCommand, FBufferView FileData = FBufferView(), uint64* NumReflinkedBytesPtr = nullptr);

struct FUnixFile : FFileReaderWriter<FUnixFile>
{
	explicit FUnixFile(const FPath& InFilename, EFileMode InMode = EFileMode::ReadOnly, uint64 InSize = 0);
	~FUnixFile();

	// IOBase
	virtual uint64 GetSize() override { return FileSize; }
	virtual bool   IsValid() override { return FileDescriptor != -1 && LastError == 0; }
	virtual void   Close() override;
	virtual int32  GetError() override { return LastError; }

	// IORead
	virtual uint64 Read(void* Dest, uint64 SourceOffset, uint64 ReadSize) override;

	// IOWrite
	virtual uint64 Write(const void* Indata, uint64 DestOffset, uint64 WriteSize) override;

	uint64 FileSize	 = 0;
	int32  LastError = 0;

	FPath Filename;

	static constexpr uint32 UNBUFFERED_READ_ALIGNMENT = 4096;

	friend bool CloneFileRange(FUnixFile& SourceFile, FUnixFile& DestFile, const FCopyCommand& CopyCommand, FBufferView FileData, uint64* NumReflinkedBytesPtr);

private:
	EFileMode Mode			 = EFileMode::None;
	int		  FileDescriptor = -1;
};
using FNativeFile = FUnixFile;
#endif	// UNSYNC_PLATFORM_UNIX

struct FVectorStreamOut
{
	FVectorStreamOut(FBuffer& Output) : Output(Output) {}

	void Write(const void* Data, uint64 Size)
	{
		const uint8* DataBytes = reinterpret_cast<const uint8*>(Data);
		Output.Append(DataBytes, Size);
	}

	template<typename T>
	void WriteT(const T& Data)
	{
		Write(&Data, sizeof(Data));
	}

	void WriteString(const std::string& S)
	{
		uint32 Len = uint32(S.length());
		WriteT(Len);
		Write(S.c_str(), Len);
	}

	FBuffer& Output;
};

struct FMemReader : FIOReader
{
	FMemReader(const FBuffer& Buffer) : FMemReader(Buffer.Data(), Buffer.Size()) {}
	FMemReader(const uint8* InData, uint64 InDataSize);

	// IOBase
	virtual uint64 GetSize() override { return Size; }
	virtual bool   IsValid() override { return Data != nullptr; }
	virtual void   Close() override
	{
		Size = 0;
		Data = nullptr;
	}
	virtual int32 GetError() override { return 0; }

	// IORead
	virtual uint64 Read(void* Dest, uint64 SourceOffset, uint64 ReadSize) override;

	const uint8* Data = nullptr;
	uint64		 Size = 0;
};

#if UNSYNC_COMPILER_MSVC
#pragma warning(push)
#pragma warning(disable : 4250)	 // 'FMemReaderWriter': inherits 'FMemReader::FMemReader::flush_one' via dominance
#endif // UNSYNC_COMPILER_MSVC
struct FMemReaderWriter : FMemReader, FIOReaderWriter
{
	FMemReaderWriter(uint8* InData, uint64 InDataSize);
	FMemReaderWriter(FMutBufferView Buffer) : FMemReaderWriter(Buffer.Data, Buffer.Size) {}

	// IOBase
	virtual void Close() override
	{
		FMemReader::Close();
		DataRw = nullptr;
	}

	// IOWrite
	virtual uint64 Write(const void* InData, uint64 DestOffset, uint64 WriteSize) override;

	// IORead
	virtual uint64 Read(void* Dest, uint64 SourceOffset, uint64 ReadSize) override
	{
		return FMemReader::Read(Dest, SourceOffset, ReadSize);
	}

	uint8* DataRw = nullptr;
};
#if UNSYNC_COMPILER_MSVC
#pragma warning(pop)
#endif // UNSYNC_COMPILER_MSVC

struct FNullReaderWriter : FIOReaderWriter
{
	struct FInvalid
	{
	};

	explicit FNullReaderWriter(uint64 InDataSize) : DataSize(InDataSize) {}
	explicit FNullReaderWriter(FInvalid) : DataSize(0), bValid(false) {}

	// IOBase
	virtual uint64 GetSize() override { return DataSize; }
	virtual bool   IsValid() override { return bValid; }
	virtual void   Close() override {};
	virtual int32  GetError() override { return 0; }

	// IORead
	virtual uint64 Read(void* Dest, uint64 SourceOffset, uint64 ReadSize) override
	{
		memset(Dest, 0, ReadSize);
		return ReadSize;
	}

	// IOWrite
	virtual uint64 Write(const void* InData, uint64 DestOffset, uint64 WriteSize) override { return WriteSize; }

	uint64 DataSize;
	bool   bValid = true;
};

struct FDeferredOpenReader : FIOReader
{
	using FOpenCallback = std::function<std::unique_ptr<FIOReader>()>;

	FDeferredOpenReader(FOpenCallback InOpenCallback) : OpenCallback(InOpenCallback) {}

	// IOBase
	virtual uint64 GetSize() override { return GetOrOpenInner()->GetSize(); }
	virtual bool   IsValid() override { return GetOrOpenInner()->IsValid(); }
	virtual void   Close() override { if (Inner) { Inner->Close(); } }
	virtual int32  GetError() override { return GetOrOpenInner()->GetError(); }

	// IORead
	virtual uint64 Read(void* Dest, uint64 SourceOffset, uint64 ReadSize) override
	{
		return GetOrOpenInner()->Read(Dest, SourceOffset, ReadSize);
	}
	virtual std::unique_ptr<FAsyncReader> CreateAsyncReader(uint32 MaxPipelineDepth = MAX_IO_PIPELINE_DEPTH) override
	{
		return GetOrOpenInner()->CreateAsyncReader(MaxPipelineDepth);
	}
	virtual FIOReader& ActualReader() override { return *GetOrOpenInner(); }

	FIOReader* GetOrOpenInner()
	{
		if (!Inner)
		{
			Inner = OpenCallback();
		}
		return Inner.get();
	}

	FOpenCallback			   OpenCallback;
	std::unique_ptr<FIOReader> Inner;
};

struct FIOReaderStream
{
	FIOReaderStream(FIOReader& InInner) : Inner(InInner) {}

	uint64 Read(void* Dest, uint64 Size)
	{
		uint64 ReadBytes = Inner.Read(Dest, Offset, Size);
		Offset += ReadBytes;
		return ReadBytes;
	}

	void Seek(uint64 InOffset)
	{
		UNSYNC_ASSERT(InOffset <= Inner.GetSize());
		Offset = InOffset;
	}

	uint64 Tell() const { return Offset; }
	void   Skip(uint64 NumBytes) { Seek(Tell() + NumBytes); }

	bool IsValid() const { return Inner.IsValid(); }

	template <typename T>
	inline uint64 ReadInto(T& Output)
	{
		return Read(&Output, sizeof(T));
	}

	uint64 RemainingSize() const { return std::max(Offset, Inner.GetSize()) - Offset; }

	FIOReader& Inner;
	uint64	   Offset = 0;
};

FBuffer ReadFileToBuffer(const FPath& Filename);
bool	WriteBufferToFile(const FPath& Filename, const uint8* Data, uint64 Size, EFileMode FileMode = EFileMode::CreateWriteOnly);
bool	WriteBufferToFile(const FPath& Filename, const FBuffer& Buffer, EFileMode FileMode = EFileMode::CreateWriteOnly);
bool	WriteBufferToFile(const FPath& Filename, const std::string& Buffer, EFileMode FileMode = EFileMode::CreateWriteOnly);

struct FFileAttributes
{
	uint64 Mtime	     = 0;	// Windows file time (100ns ticks since 1601-01-01T00:00:00Z)
	uint64 Size		     = 0;
	bool   bDirectory    = false;
	bool   bValid	     = false;
	bool   bReadOnly     = false;
	bool   bIsExecutable = false;
};

struct FFileAttributeCache
{
	std::unordered_map<FPath::string_type, FFileAttributes> Map;

	const bool Exists(const FPath& Path) const;
};

inline bool
IsReadOnly(std::filesystem::perms Perms)
{
	return ((Perms & std::filesystem::perms::owner_write)  == std::filesystem::perms::none) &&
		   ((Perms & std::filesystem::perms::group_write)  == std::filesystem::perms::none) &&
		   ((Perms & std::filesystem::perms::others_write) == std::filesystem::perms::none);
}

inline bool
IsExecutable(std::filesystem::perms Perms)
{
	return ((Perms & std::filesystem::perms::owner_exec)  != std::filesystem::perms::none) ||
           ((Perms & std::filesystem::perms::group_exec)  != std::filesystem::perms::none) ||
           ((Perms & std::filesystem::perms::others_exec) != std::filesystem::perms::none);
}

uint64 BlockingReadLarge(FIOReader& Reader, uint64 Offset, uint64 Size, uint8* OutputBuffer, uint64 OutputBufferSize);

FFileAttributes GetFileAttrib(const FPath& Path, FFileAttributeCache* AttribCache = nullptr);
FFileAttributes GetCachedFileAttrib(const FPath& Path, FFileAttributeCache& AttribCache);

bool			SetFileMtime(const FPath& Path, uint64 Mtime, bool bAllowInDryRun = false);
bool			SetFileReadOnly(const FPath& Path, bool ReadOnly);
bool            SetFileExecutable(const FPath& Path,  bool Executable);
bool			IsDirectory(const FPath& Path);
bool			PathExists(const FPath& Path);
bool			PathExists(const FPath& Path, std::error_code& OutErrorCode);
bool			CreateDirectories(const FPath& Path);
bool			EnsureDirectoryExists(const FPath& Path);
bool			FileRename(const FPath& From, const FPath& To, std::error_code& OutErrorCode);
bool			FileCopy(const FPath& From, const FPath& To, std::error_code& OutErrorCode);
bool			FileCopyOverwrite(const FPath& From, const FPath& To, std::error_code& OutErrorCode);
bool			FileRemove(const FPath& Path, std::error_code& OutErrorCode);
FPath			GetRelativePath(const FPath& Path, const FPath& Base);
FPathStringView GetRelativePathView(const FPath& Path, const FPath& Base);
std::error_code CopyFileIfNewer(const FPath& Source, const FPath& Target);
bool			IsNonCaseSensitiveFileSystem(const FPath& ExistingPath);
bool			IsCaseSensitiveFileSystem(const FPath& ExistingPath);

void ConvertDirectorySeparatorsToNative(std::string& Path);
void ConvertDirectorySeparatorsToNative(std::wstring& Path);
void ConvertDirectorySeparatorsToUnix(std::string& Path);
void ConvertDirectorySeparatorsToUnix(std::wstring& Path);

// Returns number of bytes that can be written to the given path.
// Returns ~0ull if the available space could not be determined.
uint64 GetAvailableDiskSpace(const FPath& Path);

std::filesystem::recursive_directory_iterator RecursiveDirectoryScan(const FPath& Path);
std::filesystem::directory_iterator DirectoryScan(const FPath& Path);

uint64 ToWindowsFileTime(const std::filesystem::file_time_type& T);
std::filesystem::file_time_type FromWindowsFileTime(uint64 Ticks);
std::chrono::system_clock::time_point SystemTimeFromFileTime(std::filesystem::file_time_type FileTime);

struct FSyncFilter;
FFileAttributeCache CreateFileAttributeCache(const FPath& Root, const FSyncFilter* SyncFilter = nullptr);

// Returns extended absolute path of a form \\?\D:\verylongpath or \\?\UNC\servername\verylongpath
// Expects an absolute path input. Returns original path on non-Windows.
// https://docs.microsoft.com/en-us/windows/win32/fileio/maximum-file-path-limitation
FPath MakeExtendedAbsolutePath(const FPath& InAbsolutePath);

// Removes `\\?\` or `\\?\UNC\` prefix from a given path.
// Returns original path on non-Windows.
FPath RemoveExtendedPathPrefix(const FPath& InPath);

using FPathFilterCallback = std::function<bool(const FPath& Path)>;
void DeleteOldFilesInDirectory(const FPath& Path, uint32 MaxFilesToKeep, bool bAllowInDryRun = false, const FPathFilterCallback& Filter = {});

}  // namespace unsync
