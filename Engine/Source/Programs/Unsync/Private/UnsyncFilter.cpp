// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncFilter.h"
#include "UnsyncUtil.h"

namespace unsync {

static void
AddCommaSeparatedWordsToList(const std::wstring& CommaSeparatedWords, std::vector<std::wstring>& Output)
{
	size_t Offset = 0;
	size_t Len	  = CommaSeparatedWords.length();
	while (Offset < Len)
	{
		size_t MatchOffset = CommaSeparatedWords.find(L',', Offset);
		if (MatchOffset == std::wstring::npos)
		{
			MatchOffset = Len;
		}

		std::wstring Word = CommaSeparatedWords.substr(Offset, MatchOffset - Offset);
		Output.push_back(Word);

		Offset = MatchOffset + 1;
	}
}

void
FSyncFilter::IncludeInSync(const std::wstring& CommaSeparatedWords)
{
	AddCommaSeparatedWordsToList(CommaSeparatedWords, SyncIncludedWords);
}

void
FSyncFilter::ExcludeFromSync(const std::wstring& CommaSeparatedWords)
{
	AddCommaSeparatedWordsToList(CommaSeparatedWords, SyncExcludedWords);
}

void
FSyncFilter::ExcludeFromCleanup(const std::wstring& CommaSeparatedWords)
{
	AddCommaSeparatedWordsToList(CommaSeparatedWords, CleanupExcludedWords);
}

bool
FSyncFilter::ShouldSync(const FPath& Filename) const
{
#if UNSYNC_PLATFORM_WINDOWS
	return ShouldSync(Filename.native());
#else
	return ShouldSync(Filename.wstring());
#endif
}

bool
FSyncFilter::ShouldSync(const std::wstring& Filename) const
{
	bool bInclude = SyncIncludedWords.empty();	// Include everything if there are no specific inclusions
	for (const std::wstring& Word : SyncIncludedWords)
	{
		if (Filename.find(Word) != std::wstring::npos)
		{
			bInclude = true;
			break;
		}
	}

	if (!bInclude)
	{
		return false;
	}

	for (const std::wstring& Word : SyncExcludedWords)
	{
		if (Filename.find(Word) != std::wstring::npos)
		{
			return false;
		}
	}

	return true;
}

bool
FSyncFilter::ShouldCleanup(const FPath& Filename) const
{
#if UNSYNC_PLATFORM_WINDOWS
	return ShouldCleanup(Filename.native());
#else
	return ShouldCleanup(Filename.wstring());
#endif
}

bool
FSyncFilter::ShouldCleanup(const std::wstring& Filename) const
{
	for (const std::wstring& Word : CleanupExcludedWords)
	{
		if (Filename.find(Word) != std::wstring::npos)
		{
			return false;
		}
	}

	return true;
}

FPath
FSyncFilter::Resolve(const FPath& Filename) const
{
	std::wstring FilenameLower = StringToLower(Filename.wstring());

	FPath Result;
	for (const auto& Alias : DfsAliases)
	{
		// TODO: add a case-insensitive find() helper
		size_t Pos = FilenameLower.find(StringToLower(Alias.Source.wstring()));
		if (Pos == 0)
		{
			auto Tail = (Filename.wstring().substr(Alias.Source.wstring().length() + 1));
			Result	  = Alias.Target / Tail;
			break;
		}
	}

	if (Result.empty())
	{
		Result = Filename;
	}

	return Result;
}

}
