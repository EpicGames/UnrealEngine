// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncSource.h"

namespace unsync {

struct FSyncFilter
{
	FSyncFilter() = default;

	// By default all files will be included, calling this will include only files containing these substrings
	void IncludeInSync(const std::wstring& CommaSeparatedWords);
	void ExcludeFromSync(const std::wstring& CommaSeparatedWords);
	void ExcludeFromCleanup(const std::wstring& CommaSeparatedWords);

	bool ShouldSync(const FPath& Filename) const;
	bool ShouldSync(const std::wstring& Filename) const;

	bool ShouldCleanup(const FPath& Filename) const;
	bool ShouldCleanup(const std::wstring& Filename) const;

	FPath Resolve(const FPath& Filename) const;

	std::vector<std::wstring> SyncIncludedWords;
	std::vector<std::wstring> SyncExcludedWords;	 // any paths that contain these words will not be synced
	std::vector<std::wstring> CleanupExcludedWords;	 // any paths that contain these words will not be deleted after sync

	std::vector<FDfsAlias> DfsAliases;
};

}
