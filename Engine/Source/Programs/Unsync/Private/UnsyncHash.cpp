// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncHash.h"
#include "UnsyncBuffer.h"
#include "UnsyncProtocol.h"

UNSYNC_THIRD_PARTY_INCLUDES_START
#include <blake3.h>
#include <md5-sse2.h>
UNSYNC_THIRD_PARTY_INCLUDES_END

namespace unsync {

template<typename HashResultType>
HashResultType
HashBlake3Bytes(const uint8* Data, uint64 Size)
{
	static_assert(sizeof(HashResultType) <= 32);
	HashResultType Result;
	blake3_hasher  Hasher;
	blake3_hasher_init(&Hasher);
	blake3_hasher_update(&Hasher, Data, Size);
	blake3_hasher_finalize(&Hasher, Result.Data, sizeof(Result.Data));
	return Result;
}

FBlake3Hasher::FBlake3Hasher()
{
	static_assert(sizeof(State) >= sizeof(blake3_hasher));
	static_assert(sizeof(State) == sizeof(FBlake3Hasher));
	static_assert(alignof(FBlake3Hasher) >= alignof(blake3_hasher));

	blake3_hasher* Hasher = reinterpret_cast<blake3_hasher*>(State);
	blake3_hasher_init(Hasher);
}

void
FBlake3Hasher::Update(const void* Data, size_t Size)
{
	blake3_hasher* Hasher = reinterpret_cast<blake3_hasher*>(State);
	blake3_hasher_update(Hasher, Data, Size);
}

FHash256
FBlake3Hasher::Finalize()
{
	FHash256 Result;
	blake3_hasher* Hasher = reinterpret_cast<blake3_hasher*>(State);
	blake3_hasher_finalize(Hasher, Result.Data, sizeof(Result.Data));
	return Result;
}

void
FBlake3Hasher::Reset()
{
	blake3_hasher* Hasher = reinterpret_cast<blake3_hasher*>(State);
	blake3_hasher_reset(Hasher);
}


FMD5Hasher::FMD5Hasher()
{
	static_assert(sizeof(State) >= sizeof(md5_context));
	static_assert(sizeof(State) == sizeof(FMD5Hasher));
	static_assert(alignof(FMD5Hasher) >= alignof(md5_context));

	md5_context* Hasher = reinterpret_cast<md5_context*>(State);
	md5_init(Hasher);
}

void
FMD5Hasher::Update(const void* Data, size_t Size)
{
	md5_context* Hasher = reinterpret_cast<md5_context*>(State);
	md5_update(Hasher, reinterpret_cast<const uint8*>(Data), Size);
}

FHash128
FMD5Hasher::Finalize()
{
	FHash128	 Result;
	md5_context* Hasher = reinterpret_cast<md5_context*>(State);
	md5_finish(Hasher, Result.Data);
	return Result;
}

void
FMD5Hasher::Reset()
{
	md5_context* Hasher = reinterpret_cast<md5_context*>(State);
	md5_init(Hasher);
}

template FHash128 HashBlake3Bytes<FHash128>(const uint8* data, uint64 size);
template FHash160 HashBlake3Bytes<FHash160>(const uint8* data, uint64 size);
template FHash256 HashBlake3Bytes<FHash256>(const uint8* data, uint64 size);

FHash128
HashMd5Bytes(const uint8* Data, uint64 Size)
{
	FHash128	Result;
	md5_context Hasher;
	md5_init(&Hasher);
	md5_update(&Hasher, Data, Size);
	md5_finish(&Hasher, Result.Data);
	return Result;
}

FGenericHash
HashBytes(const uint8* Data, uint64 Size, EHashType HashType)
{
	FGenericHash Result = {};
	switch (HashType)
	{
		default:
		case EHashType::Unknown:
			UNSYNC_ERROR(L"Unexpected hash type");
			break;
		case EHashType::MD5:
			Result = FGenericHash::FromMd5(HashMd5Bytes(Data, Size));
			break;
		case EHashType::Blake3_128:
			Result = FGenericHash::FromBlake3_128(HashBlake3Bytes<FHash128>(Data, Size));
			break;
		case EHashType::Blake3_160:
			Result = FGenericHash::FromBlake3_160(HashBlake3Bytes<FHash160>(Data, Size));
			break;
		case EHashType::Blake3_256:
			Result = FGenericHash::FromBlake3_256(HashBlake3Bytes<FHash256>(Data, Size));
			break;
	}
	return Result;
}

FGenericHash
ComputeHash(const uint8* Data, uint64 Size, EStrongHashAlgorithmID Algorithm)
{
	switch (Algorithm)
	{
		case EStrongHashAlgorithmID::MD5:
			return FGenericHash::FromMd5(HashMd5Bytes(Data, Size));
		case EStrongHashAlgorithmID::Blake3_128:
			return FGenericHash::FromBlake3_128(HashBlake3Bytes<FHash128>(Data, Size));
		case EStrongHashAlgorithmID::Blake3_160:
			return FGenericHash::FromBlake3_160(HashBlake3Bytes<FHash160>(Data, Size));
		case EStrongHashAlgorithmID::Blake3_256:
			return FGenericHash::FromBlake3_256(HashBlake3Bytes<FHash256>(Data, Size));
		default:
			UNSYNC_ERROR(L"Unexpected hash algorithm id");
			return FGenericHash{};
	}
}

FGenericHash
ComputeHash(const FBuffer& Buffer, EStrongHashAlgorithmID Algorithm)
{
	return ComputeHash(Buffer.Data(), Buffer.Size(), Algorithm);
}

FGenericHash
ComputeHash(const FBufferView& Buffer, EStrongHashAlgorithmID Algorithm)
{
	return ComputeHash(Buffer.Data, Buffer.Size, Algorithm);
}

EHashType
ToHashType(EStrongHashAlgorithmID StrongHasher)
{
	switch (StrongHasher)
	{
		case EStrongHashAlgorithmID::MD5:
			return EHashType::MD5;
		case EStrongHashAlgorithmID::Blake3_128:
			return EHashType::Blake3_128;
		case EStrongHashAlgorithmID::Blake3_160:
			return EHashType::Blake3_160;
		case EStrongHashAlgorithmID::Blake3_256:
			return EHashType::Blake3_256;
		default:
			UNSYNC_ERROR(L"Unexpected hash algorithm id");
			return EHashType::Unknown;
	}
}

const uint32 FBuzHash::TABLE[256] = {
	0x458be752, 0xc10748cc, 0xfbbcdbb8, 0x6ded5b68, 0xb10a82b5, 0x20d75648, 0xdfc5665f, 0xa8428801, 0x7ebf5191, 0x841135c7, 0x65cc53b3,
	0x280a597c, 0x16f60255, 0xc78cbc3e, 0x294415f5, 0xb938d494, 0xec85c4e6, 0xb7d33edc, 0xe549b544, 0xfdeda5aa, 0x882bf287, 0x3116737c,
	0x05569956, 0xe8cc1f68, 0x0806ac5e, 0x22a14443, 0x15297e10, 0x50d090e7, 0x4ba60f6f, 0xefd9f1a7, 0x5c5c885c, 0x82482f93, 0x9bfd7c64,
	0x0b3e7276, 0xf2688e77, 0x8fad8abc, 0xb0509568, 0xf1ada29f, 0xa53efdfe, 0xcb2b1d00, 0xf2a9e986, 0x6463432b, 0x95094051, 0x5a223ad2,
	0x9be8401b, 0x61e579cb, 0x1a556a14, 0x5840fdc2, 0x9261ddf6, 0xcde002bb, 0x52432bb0, 0xbf17373e, 0x7b7c222f, 0x2955ed16, 0x9f10ca59,
	0xe840c4c9, 0xccabd806, 0x14543f34, 0x1462417a, 0x0d4a1f9c, 0x087ed925, 0xd7f8f24c, 0x7338c425, 0xcf86c8f5, 0xb19165cd, 0x9891c393,
	0x325384ac, 0x0308459d, 0x86141d7e, 0xc922116a, 0xe2ffa6b6, 0x53f52aed, 0x2cd86197, 0xf5b9f498, 0xbf319c8f, 0xe0411fae, 0x977eb18c,
	0xd8770976, 0x9833466a, 0xc674df7f, 0x8c297d45, 0x8ca48d26, 0xc49ed8e2, 0x7344f874, 0x556f79c7, 0x6b25eaed, 0xa03e2b42, 0xf68f66a4,
	0x8e8b09a2, 0xf2e0e62a, 0x0d3a9806, 0x9729e493, 0x8c72b0fc, 0x160b94f6, 0x450e4d3d, 0x7a320e85, 0xbef8f0e1, 0x21d73653, 0x4e3d977a,
	0x1e7b3929, 0x1cc6c719, 0xbe478d53, 0x8d752809, 0xe6d8c2c6, 0x275f0892, 0xc8acc273, 0x4cc21580, 0xecc4a617, 0xf5f7be70, 0xe795248a,
	0x375a2fe9, 0x425570b6, 0x8898dcf8, 0xdc2d97c4, 0x0106114b, 0x364dc22f, 0x1e0cad1f, 0xbe63803c, 0x5f69fac2, 0x4d5afa6f, 0x1bc0dfb5,
	0xfb273589, 0x0ea47f7b, 0x3c1c2b50, 0x21b2a932, 0x6b1223fd, 0x2fe706a8, 0xf9bd6ce2, 0xa268e64e, 0xe987f486, 0x3eacf563, 0x1ca2018c,
	0x65e18228, 0x2207360a, 0x57cf1715, 0x34c37d2b, 0x1f8f3cde, 0x93b657cf, 0x31a019fd, 0xe69eb729, 0x8bca7b9b, 0x4c9d5bed, 0x277ebeaf,
	0xe0d8f8ae, 0xd150821c, 0x31381871, 0xafc3f1b0, 0x927db328, 0xe95effac, 0x305a47bd, 0x426ba35b, 0x1233af3f, 0x686a5b83, 0x50e072e5,
	0xd9d3bb2a, 0x8befc475, 0x487f0de6, 0xc88dff89, 0xbd664d5e, 0x971b5d18, 0x63b14847, 0xd7d3c1ce, 0x7f583cf3, 0x72cbcb09, 0xc0d0a81c,
	0x7fa3429b, 0xe9158a1b, 0x225ea19a, 0xd8ca9ea3, 0xc763b282, 0xbb0c6341, 0x020b8293, 0xd4cd299d, 0x58cfa7f8, 0x91b4ee53, 0x37e4d140,
	0x95ec764c, 0x30f76b06, 0x5ee68d24, 0x679c8661, 0xa41979c2, 0xf2b61284, 0x4fac1475, 0x0adb49f9, 0x19727a23, 0x15a7e374, 0xc43a18d5,
	0x3fb1aa73, 0x342fc615, 0x924c0793, 0xbee2d7f0, 0x8a279de9, 0x4aa2d70c, 0xe24dd37f, 0xbe862c0b, 0x177c22c2, 0x5388e5ee, 0xcd8a7510,
	0xf901b4fd, 0xdbc13dbc, 0x6c0bae5b, 0x64efe8c7, 0x48b02079, 0x80331a49, 0xca3d8ae6, 0xf3546190, 0xfed7108b, 0xc49b941b, 0x32baf4a9,
	0xeb833a4a, 0x88a3f1a5, 0x3a91ce0a, 0x3cc27da1, 0x7112e684, 0x4a3096b1, 0x3794574c, 0xa3c8b6f3, 0x1d213941, 0x6e0a2e00, 0x233479f1,
	0x0f4cd82f, 0x6093edd2, 0x5d7d209e, 0x464fe319, 0xd4dcac9e, 0x0db845cb, 0xfb5e4bc3, 0xe0256ce1, 0x09fb4ed1, 0x0914be1e, 0xa5bdb2c3,
	0xc6eb57bb, 0x30320350, 0x3f397e91, 0xa67791bc, 0x86bc0e2c, 0xefa0a7e2, 0xe9ff7543, 0xe733612c, 0xd185897b, 0x329e5388, 0x91dd236b,
	0x2ecb0d93, 0xf4d82a3d, 0x35b5c03f, 0xe4e606f0, 0x05b21843, 0x37b45964, 0x5eff22f4, 0x6027f4cc, 0x77178b3c, 0xae507131, 0x7bf7cabc,
	0xf9c18d66, 0x593ade65, 0xd95ddf11,
};

inline bool
HexCharToUint8(char C, uint8& Output)
{
	if (C >= '0' && C <= '9')
	{
		Output = C - '0';
	}
	else if (C >= 'a' && C <= 'f')
	{
		Output = 10 + (C - 'a');
	}
	else if (C >= 'A' && C <= 'F')
	{
		Output = 10 + (C - 'A');
	}
	else
	{
		return false;
	}

	return true;
}

// Decodes hex character pairs into bytes. HexString must have exactly NumBytes*2 characters.
// When bReverseOrder is true, bytes are written in reverse order (for big-endian hex into little-endian memory).
template<bool bReverseOrder = false>
inline bool
ParseBytesFromHexString(std::string_view HexString, uint8* OutputBytes, uint64 NumBytes)
{
	if (HexString.length() != NumBytes * 2)
	{
		return false;
	}

	for (uint64 Index = 0; Index < NumBytes; ++Index)
	{
		uint8 H = 0, L = 0;
		bool  bValid = HexCharToUint8(HexString[Index * 2 + 0], H) && HexCharToUint8(HexString[Index * 2 + 1], L);
		if (!bValid)
		{
			return false;
		}
		uint64 OutputIndex = bReverseOrder ? (NumBytes - 1 - Index) : Index;
		OutputBytes[OutputIndex] = (H << 4) | L;
	}

	return true;
}

bool
ParseHashFromHexString(EStrongHashAlgorithmID StrongHasher, std::string_view HexString, FGenericHash& Output)
{
	FGenericHash Result = {};
	Result.Type			= ToHashType(StrongHasher);

	if (!ParseBytesFromHexString(HexString, Result.Data, Result.Size()))
	{
		return false;
	}

	Output = Result;
	return true;
}

bool
ParseUint64FromHexString(std::string_view HexString, uint64& Output)
{
	uint64 Result = 0;
	if (!ParseBytesFromHexString</*bReverseOrder*/ true>(HexString, reinterpret_cast<uint8*>(&Result), 8))
	{
		return false;
	}

	Output = Result;
	return true;
}

void
FStrongHasher::Reset()
{
	switch (Algorithm)
	{
		case EStrongHashAlgorithmID::MD5:
			MD5.Reset();
			break;
		case EStrongHashAlgorithmID::Blake3_128:
		case EStrongHashAlgorithmID::Blake3_160:
		case EStrongHashAlgorithmID::Blake3_256:
			Blake3.Reset();
			break;
		default:
			UNSYNC_FATAL(L"Unsupported hash algorithm ID: %u", uint32(Algorithm));
	}
}

void
FStrongHasher::Update(const void* Data, uint64 Size)
{
	switch (Algorithm)
	{
		case EStrongHashAlgorithmID::MD5:
			MD5.Update(Data, Size);
			break;
		case EStrongHashAlgorithmID::Blake3_128:
		case EStrongHashAlgorithmID::Blake3_160:
		case EStrongHashAlgorithmID::Blake3_256:
			Blake3.Update(Data, Size);
			break;
		default:
			UNSYNC_FATAL(L"Unsupported hash algorithm ID: %u", uint32(Algorithm));
	}
}

FGenericHash
FStrongHasher::Finalize()
{
	FGenericHash Result = {};
	switch (Algorithm)
	{
		case EStrongHashAlgorithmID::MD5:
			Result = FGenericHash::FromMd5(MD5.Finalize());
			break;
		case EStrongHashAlgorithmID::Blake3_128:
			Result = FGenericHash::FromBlake3_128(ToHash128(Blake3.Finalize()));
			break;
		case EStrongHashAlgorithmID::Blake3_160:
			Result = FGenericHash::FromBlake3_160(ToHash160(Blake3.Finalize()));
			break;
		case EStrongHashAlgorithmID::Blake3_256:
			Result = FGenericHash::FromBlake3_256(Blake3.Finalize());
			break;
		default:
			UNSYNC_FATAL(L"Unsupported hash algorithm ID: %u", uint32(Algorithm));
	}
	return Result;
}

}  // namespace unsync
