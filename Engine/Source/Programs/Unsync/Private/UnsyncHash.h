// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncLog.h"

#include <stdint.h>
#include <string.h>
#include <algorithm>
#include <string>

namespace unsync {

class FBuffer;
struct FBufferView;

enum class EStrongHashAlgorithmID : uint64;

template<size_t SizeBytes>
struct THashValue
{
	alignas(uint32) uint8 Data[SizeBytes];

	bool operator==(const THashValue& Other) const { return !memcmp(Data, Other.Data, SizeBytes); }

	bool operator!=(const THashValue& Other) const { return !(*this == Other); }

	struct Hasher
	{
		uint32 operator()(const THashValue& Hash) const
		{
			uint32 Result;
			memcpy(&Result, Hash.Data, 4);
			return Result;
		}
	};

	static constexpr size_t Size() { return SizeBytes; }
};

using FHash128 = THashValue<16>;
using FHash160 = THashValue<20>;
using FHash256 = THashValue<32>;

enum class EHashType : uint8 {
	Unknown,
	Unknown_128,
	Unknown_160,
	Unknown_256,
	MD5,		 // Standard 128 bit MD5
	Blake3_128,	 // Blake3 hash, truncated to 128 bits
	Blake3_160,	 // Blake3 hash, truncated to 160 bits, AKA Unreal Engine IoHash
	Blake3_256,	 // Blake3 hash, full 256 bits
};

inline size_t
GetHashSize(EHashType Type)
{
	switch (Type)
	{
		default:
		case EHashType::Unknown:
			return 0;
		case EHashType::Unknown_128:
			return 16;
		case EHashType::Unknown_160:
			return 20;
		case EHashType::Unknown_256:
			return 32;
		case EHashType::MD5:
			return 16;
		case EHashType::Blake3_128:
			return 16;
		case EHashType::Blake3_160:
			return 20;
		case EHashType::Blake3_256:
			return 32;
	}
}

EHashType ToHashType(EStrongHashAlgorithmID StrongHasher);

// Generic hash type intended for runtime uses only.
// Serialized formats should always use Hash128/160/256.
struct FGenericHash
{
	alignas(uint32) uint8 Data[32];
	EHashType Type = EHashType::Unknown;

	size_t Size() const { return GetHashSize(Type); }

	bool operator==(const FGenericHash& Other) const { return !memcmp(Data, Other.Data, Size()) && Type == Other.Type; }

	bool operator!=(const FGenericHash& Other) const { return !(*this == Other); }

	static FGenericHash FromHash128(const FHash128& Other, EHashType Type)
	{
		UNSYNC_ASSERT(GetHashSize(Type) == 16);	 // #wip-widehash
		FGenericHash Result = {};
		Result.Type			= Type;
		memcpy(Result.Data, Other.Data, 16);
		return Result;
	}

	static FGenericHash FromMd5(const FHash128& Other) { return FromHash128(Other, EHashType::MD5); }

	static FGenericHash FromBlake3_128(const FHash128& Other)  // NOLINT
	{
		return FromHash128(Other, EHashType::Blake3_128);
	}

	static FGenericHash FromBlake3_160(const FHash160& Other)  // NOLINT
	{
		FGenericHash Result = {};
		Result.Type			= EHashType::Blake3_160;
		memcpy(Result.Data, Other.Data, 20);
		return Result;
	}

	static FGenericHash FromBlake3_256(const FHash256& Other)  // NOLINT
	{
		FGenericHash Result = {};
		Result.Type			= EHashType::Blake3_256;
		memcpy(Result.Data, Other.Data, 32);
		return Result;
	}

	FHash128 ToHash128() const
	{
		FHash128 Result = {};
		memcpy(Result.Data, Data, std::min(Size(), sizeof(Result)));
		return Result;
	}

	FHash160 ToHash160() const
	{
		FHash160 Result = {};
		memcpy(Result.Data, Data, std::min(Size(), sizeof(Result)));
		return Result;
	}

	struct Hasher
	{
		uint32 operator()(const FGenericHash& Hash) const
		{
			uint32 Result;
			memcpy(&Result, Hash.Data, sizeof(Result));
			return Result;
		}
	};
};

bool ParseHashFromHexString(EStrongHashAlgorithmID StrongHasher, std::string_view HexString, FGenericHash& Output);
bool ParseUint64FromHexString(std::string_view HexString, uint64& Output);

inline FHash160
ToHash160(const FHash256& Other)
{
	FHash160 Result;
	memcpy(Result.Data, Other.Data, Result.Size());
	return Result;
}

inline FHash128
ToHash128(const FHash256& Other)
{
	FHash128 Result;
	memcpy(Result.Data, Other.Data, Result.Size());
	return Result;
}

inline FHash128
ToHash128(const FHash160& Other)
{
	FHash128 Result;
	memcpy(Result.Data, Other.Data, Result.Size());
	return Result;
}

inline uint32
MurmurMix(uint32 X)
{
	X ^= X >> 16;
	X *= 0x85ebca6b;
	X ^= X >> 13;
	X *= 0xc2b2ae35;
	X ^= X >> 16;
	return X;
}

struct FRollingChecksum
{
	uint16 A	 = 0;
	uint16 B	 = 0;
	uint64 Count = 0;

	void Update(const uint8* Data, uint64 Size)
	{
		for (uint64 I = 0; I < Size; ++I)
		{
			Add(Data[I]);
		}
	}

	void Reset() { *this = FRollingChecksum(); }

	uint32 Get() const { return uint32(A) | uint32(B) << 16; }

	void Add(uint8 X)
	{
		uint16 X2 = X + 31;
		A += X2;
		B += A;
		Count++;
	}

	void Sub(uint8 X)
	{
		uint16 X2 = X + 31;
		A -= X2;
		B -= uint16(Count * X2);
		Count--;
	}
};

struct FBuzHash
{
	static const uint32 TABLE[256];

	static inline uint32 Rol32(uint32 V, uint32 N)
	{
		N &= 31;
		return ((V) << (N)) | ((V) >> (32 - N));
	}

	uint64 Count = 0;
	uint32 State = 0;

	void Update(const uint8* Data, uint64 Size)
	{
		for (uint64 I = 0; I < Size; ++I)
		{
			Add(Data[I]);
		}
	}

	void Reset() { *this = FBuzHash(); }

	uint32 Get() const { return State; }

	void Add(uint8 X)
	{
		State = Rol32(State, 1) ^ TABLE[X];
		Count++;
	}

	void Sub(uint8 X)
	{
		State = State ^ Rol32(TABLE[X], uint32(Count - 1));
		Count--;
	}
};

template<typename HashResultType>
HashResultType HashBlake3Bytes(const uint8* Data, uint64 Size);

template<typename HashResultType>
HashResultType
HashBlake3String(std::string_view Str)
{
	return HashBlake3Bytes<HashResultType>((const uint8*)Str.data(), Str.length());
}

template<typename HashResultType>
HashResultType
HashBlake3String(const char* Str)
{
	return HashBlake3Bytes<HashResultType>((const uint8*)Str, strlen(Str));
}

template<typename HashResultType>
HashResultType
HashBlake3String(const wchar_t* Str)
{
	return HashBlake3Bytes<HashResultType>((const uint8*)Str, wcslen(Str) * sizeof(*Str));
}

template<typename HashResultType>
HashResultType
HashBlake3String(const std::string& Str)
{
	return HashBlake3Bytes<HashResultType>((const uint8*)Str.c_str(), Str.length());
}

template<typename HashResultType>
HashResultType
HashBlake3String(const std::wstring& Str)
{
	return HashBlake3Bytes<HashResultType>((const uint8*)Str.c_str(), Str.length() * sizeof(Str[0]));
}

class alignas(8) FBlake3Hasher
{
public:
	FBlake3Hasher();

	void	 Update(const void* Data, size_t Size);
	FHash256 Finalize();
	void	 Reset();

private:
	static constexpr size_t STATE_SIZE = 1912;
	uint8 State[STATE_SIZE];
};

class alignas(8) FMD5Hasher
{
public:
	FMD5Hasher();

	void	 Update(const void* Data, size_t Size);
	FHash128 Finalize();
	void	 Reset();

private:
	static constexpr size_t STATE_SIZE = 88;
	uint8					State[STATE_SIZE];
};

class FStrongHasher
{
public:
	FStrongHasher(EStrongHashAlgorithmID InAlgorithm) : Algorithm(InAlgorithm) {}

	void		 Reset();
	void		 Update(const void* Data, uint64 Size);
	FGenericHash Finalize();

private:
	const EStrongHashAlgorithmID Algorithm;
	FBlake3Hasher				 Blake3;
	FMD5Hasher					 MD5;
};

FHash128 HashMd5Bytes(const uint8* Data, uint64 Size);

FGenericHash HashBytes(const uint8* Data, uint64 Size, EHashType HashType);

FHash128	 ComputeHash128(const uint8* Data, uint64 Size, EStrongHashAlgorithmID Algorithm);
FGenericHash ComputeHash(const uint8* Data, uint64 Size, EStrongHashAlgorithmID Algorithm);

FGenericHash ComputeHash(const FBuffer& Buffer, EStrongHashAlgorithmID Algorithm);
FGenericHash ComputeHash(const FBufferView& Buffer, EStrongHashAlgorithmID Algorithm);

inline FHash128
ToHash128(const FHash128& X)
{
	return X;
}

inline FHash128
ToHash128(const FGenericHash& X)
{
	return X.ToHash128();
}

}  // namespace unsync

namespace std {
template<size_t SizeBytes>
struct hash<unsync::THashValue<SizeBytes>>
{
	size_t operator()(const unsync::THashValue<SizeBytes>& Hash) const
	{
		typename unsync::THashValue<SizeBytes>::Hasher Hasher;
		return (size_t)Hasher(Hash);
	}
};

template<>
struct hash<unsync::FGenericHash>
{
	size_t operator()(const unsync::FGenericHash& Hash) const
	{
		size_t Result;
		memcpy(&Result, Hash.Data, sizeof(Result));
		return Result;
	}
};
}  // namespace std
