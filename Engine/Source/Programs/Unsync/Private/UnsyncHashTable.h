// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

UNSYNC_THIRD_PARTY_INCLUDES_START
#include <flat_hash_map.hpp>
UNSYNC_THIRD_PARTY_INCLUDES_END

namespace unsync {

template<typename K, typename V, typename Hash = std::hash<K>, typename Eq = std::equal_to<K>>
// using THashMap = std::unordered_map<K,V,Hash,Eq>;
using THashMap = ska::flat_hash_map<K, V, Hash, Eq>;

template<typename K, typename Hash = std::hash<K>, typename Eq = std::equal_to<K>>
// using THashSet = std::unordered_set<K, Hash, Eq>;
using THashSet = ska::flat_hash_set<K, Hash, Eq>;

}  // namespace unsync
