// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncHorde.h"
#include "UnsyncAuth.h"
#include "UnsyncCompression.h"
#include "UnsyncFile.h"

#include <regex>
#include <json11.hpp>
#include <optional>
#include <stdio.h>
#include <span>

namespace unsync {

FHordeProtocolImpl::FHordeProtocolImpl(const FRemoteDesc& InRemoteDesc, const FBlockRequestMap* InRequestMap, FProxyPool& InProxyPool)
: FRemoteProtocolBase(InRemoteDesc, InRequestMap)
, ProxyPool(InProxyPool)
{
}

FDownloadResult
FHordeProtocolImpl::Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback)
{
	// Requested blocks may come from different sources, in which case we have to split up the request by artifact root.

	struct FBatch
	{
		std::vector<FNeedBlock> NeedBlocks;
	};

	std::vector<FBatch> Batches;
	Batches.resize(RequestMap->GetSourceRoots().size());

	for (const FNeedBlock Block : NeedBlocks)
	{
		const FBlockRequestMap::FBlockRequestEx* FoundRequest = RequestMap->FindRequest(Block.Hash);
		if (!FoundRequest)
		{
			UNSYNC_ERROR(L"Could not find block request metadata");
			return FDownloadError(EDownloadRetryMode::Abort);
		}

		if (FoundRequest->SourceId == ~0u)
		{
			UNSYNC_ERROR(L"Block request metadata does not contain a valid artifact source ID");
			return FDownloadError(EDownloadRetryMode::Abort);
		}

		Batches[FoundRequest->SourceId].NeedBlocks.push_back(Block);
	}

	for (size_t BatchIndex = 0; BatchIndex < Batches.size(); ++BatchIndex)
	{
		const FBatch& Batch = Batches[BatchIndex];
		if (Batch.NeedBlocks.empty())
		{
			continue;
		}

		const FPath& ArtifactPath	  = RequestMap->GetSourceRoots()[BatchIndex];
		std::string	 ArtifactPathUtf8 = ConvertWideToUtf8(ArtifactPath.wstring());

		FDownloadResult BatchDownloadResult = DownloadArtifactBlobs(ArtifactPathUtf8, MakeView(Batch.NeedBlocks), CompletionCallback);

		if (BatchDownloadResult.IsError())
		{
			return BatchDownloadResult;
		}
	}

	return ResultOk<FDownloadError>();
}

FDownloadResult FHordeProtocolImpl::DownloadArtifactBlobs(std::string_view ArtifactPath, const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback)
{
	if (NeedBlocks.Size() == 0)
	{
		return ResultOk<FDownloadError>();
	}

	std::string RequestJson = FormatBlockRequestJson(*RequestMap, NeedBlocks);

	std::string RequestUrl = fmt::format("/{}/unsync-blobs?compress={}", ArtifactPath,
										 ProxyPool.RemoteDesc.bPreferCompression ? "true" : "false");

	const EStrongHashAlgorithmID StrongHasher = RequestMap->GetStrongHasher();

	std::string BearerToken = ProxyPool.GetAccessToken();

	FPooledHttpConnection HttpConnection(ProxyPool);

	if (!HttpConnection.IsValid())
	{
		UNSYNC_ERROR(L"HTTP connection cannot be used");
		return FDownloadError(EDownloadRetryMode::Disconnect);
	}

	FHttpRequest Request;
	Request.Method			   = EHttpMethod::POST;
	Request.PayloadContentType = EHttpContentType::Application_Json;
	Request.Payload.Data	   = reinterpret_cast<const uint8*>(RequestJson.data());
	Request.Payload.Size	   = RequestJson.length();
	Request.BearerToken		   = BearerToken;
	Request.Url				   = RequestUrl;

	FHttpResponse Response = HttpRequest(HttpConnection, Request);

	if (!Response.Success())
	{
		UNSYNC_ERROR(L"Failed to complete block request. HTTP error code: %d.", Response.Code);
		return FDownloadError(EDownloadRetryMode::Abort);
	}

	// std::string_view TransferEncoding = Response.FindHeader("transfer-encoding");
	const std::string_view ChunkContentEncoding = Response.FindHeader("x-chunk-content-encoding");
	const std::string_view ContentType			= Response.FindHeader("content-type");

	if (!UncasedStringEquals(ContentType, "application/x-horde-unsync-blob"))
	{
		std::string Value = std::string(ContentType);
		UNSYNC_ERROR(L"Got unexpected blob content type header: '%hs'", Value.c_str());
		return FDownloadError(EDownloadRetryMode::Abort);
	}

	if (Response.Buffer.Size() == 0)
	{
		UNSYNC_ERROR(L"Got unexpected empty response body while downloading blocks from Horde");
		return FDownloadError(EDownloadRetryMode::Abort);
	}

	// TODO: read body stream as it arrives using HTTP chunk callbacks
	FMemReader		BufferReader(Response.Buffer);
	FIOReaderStream Reader(BufferReader);

	while (Reader.RemainingSize())
	{
		uint64 HeaderId = 0;
		Reader.ReadInto(HeaderId);

		if (HeaderId == FHordeUnsyncBlobHeaderV1::MAGIC)
		{
			FHordeUnsyncBlobHeaderV1 PacketHeader = {};

			PacketHeader.Magic = HeaderId;
			Reader.ReadInto(PacketHeader.PayloadSize);
			Reader.ReadInto(PacketHeader.DecompressedSize);
			Reader.ReadInto(PacketHeader.DecompressedHash);

			FDownloadedBlock DownloadedBlock;
			DownloadedBlock.bCompressed = false;  // we always decompress the block before handing it over to the caller

			FBufferView Payload = Response.Buffer.View(Reader.Tell(), PacketHeader.PayloadSize);
			FBuffer		DecompressedBuffer;

			if (ChunkContentEncoding == "zstd")
			{
				if (PacketHeader.DecompressedSize != 0)
				{
					DecompressedBuffer				 = Decompress(Payload.Data, Payload.Size);
					DownloadedBlock.Data			 = DecompressedBuffer.Data();
					DownloadedBlock.DecompressedSize = DecompressedBuffer.Size();
				}
				else
				{
					DownloadedBlock.Data			 = DecompressedBuffer.Data();
					DownloadedBlock.DecompressedSize = DecompressedBuffer.Size();
				}
			}
			else if (ChunkContentEncoding == "" || ChunkContentEncoding == "identity")
			{
				DownloadedBlock.Data			 = Payload.Data;
				DownloadedBlock.DecompressedSize = Payload.Size;

				if (PacketHeader.DecompressedSize != PacketHeader.PayloadSize)
				{
					UNSYNC_ERROR(L"Received blob size (%llu bytes) does not match expected size (%llu bytes)",
								 llu(PacketHeader.DecompressedSize),
								 llu(PacketHeader.PayloadSize));
					return FDownloadError(EDownloadRetryMode::Abort);
				}
			}
			else
			{
				std::string Value = std::string(ChunkContentEncoding);
				UNSYNC_ERROR(L"Unexpected chunk content encoding: '%hs'", Value.c_str());
				return FDownloadError(EDownloadRetryMode::Abort);
			}

			if (DownloadedBlock.DecompressedSize != PacketHeader.DecompressedSize)
			{
				UNSYNC_ERROR(L"Received blob size (%llu bytes) does not match expected size (%llu bytes)",
							 llu(DownloadedBlock.DecompressedSize),
							 llu(PacketHeader.DecompressedSize));
				return FDownloadError(EDownloadRetryMode::Abort);
			}

			FGenericHash BlockHash = ComputeHash(DownloadedBlock.Data, DownloadedBlock.DecompressedSize, StrongHasher);

			if (BlockHash.ToHash160() != PacketHeader.DecompressedHash)
			{
				std::string BlockHashStr	= HashToHexString(BlockHash.ToHash160());
				std::string ExpectedHashStr = HashToHexString(PacketHeader.DecompressedHash);
				UNSYNC_ERROR(L"Received blob hash (%hs) does not match expected hash (%hs)", BlockHashStr.c_str(), ExpectedHashStr.c_str());
				return FDownloadError(EDownloadRetryMode::Abort);
			}

			FHash128 BlockHash128 = BlockHash.ToHash128();

			CompletionCallback(DownloadedBlock, BlockHash128);

			Reader.Skip(PacketHeader.PayloadSize);
		}
		else if (HeaderId == FHordeUnsyncBlobErrorHeaderV1::MAGIC)
		{
			FHordeUnsyncBlobErrorHeaderV1 PacketHeader = {};

			PacketHeader.Magic = HeaderId;
			Reader.ReadInto(PacketHeader.PayloadSize);

			if (PacketHeader.PayloadSize > 1)
			{
				uint32 StringLength = PacketHeader.PayloadSize - 1;	 // Error string is zero-terminated

				std::string ErrorString;
				ErrorString.resize(StringLength);
				Reader.Read(ErrorString.data(), StringLength);

				UNSYNC_ERROR(L"Horde server error: %hs", ErrorString.c_str());
			}
			else
			{
				UNSYNC_ERROR(L"Unknown Horde server error");
			}

			return FDownloadError(EDownloadRetryMode::Abort);
		}
		else
		{
			UNSYNC_ERROR(L"Got unexpected blob header identifier");
			return FDownloadError(EDownloadRetryMode::Abort);
		}
	}

	return ResultOk<FDownloadError>();
}

TResult<FDirectoryManifest>
FHordeProtocolImpl::DownloadManifest(std::string_view ManifestName)
{
	FPooledHttpConnection HttpConnection(ProxyPool);

	std::string BearerToken = ProxyPool.GetAccessToken();

	std::string ManifestUrl = fmt::format("/{}/unsync", ManifestName);

	UNSYNC_LOG(L"Downloading manifest from Horde: '%hs'", ManifestUrl.c_str());

	FHttpRequest Request;
	Request.Url			= ManifestUrl;
	Request.BearerToken = BearerToken;

	FHttpResponse Response = HttpRequest(HttpConnection, Request);

	if (!Response.Success())
	{
		return HttpError(Response.Code);
	}

	// Ensure response is terminated
	Response.Buffer.PushBack(0);

	if (Response.ContentType != EHttpContentType::Application_Json)
	{
		return AppError("Unexpected manifest encoding");
	}

	UNSYNC_LOG(L"Decoding manifest ...");

	return DecodeHordeManifestJson((const char*)Response.Buffer.Data(), ManifestName);
}

bool
RequestPathLooksLikeHordeArtifact(std::string_view RequestPath)
{
	static const std::regex Pattern("^api\\/v\\d+\\/artifacts\\/[a-fA-F0-9]+$");
	return std::regex_match(RequestPath.begin(), RequestPath.end(), Pattern);
}

TResult<ProxyQuery::FHelloResponse>
FHordeProtocolImpl::QueryHello(FHttpConnection& HttpConnection)
{
	ProxyQuery::FHelloResponse Result;

	const std::string_view Url = "/api/v1/server/auth";

	FHttpResponse Response = HttpRequest(HttpConnection, EHttpMethod::GET, Url);

	if (!Response.Success())
	{
		UNSYNC_ERROR(L"Failed to establish connection to Horde server. Error code: %d.", Response.Code);
		return HttpError(fmt::format("{}:{}{}", HttpConnection.HostAddress.c_str(), HttpConnection.HostPort, Url), Response.Code);
	}

	using namespace json11;
	std::string JsonString = std::string(Response.AsStringView());

	std::string JsonErrorString;
	Json		JsonObject = Json::parse(JsonString, JsonErrorString);

	if (!JsonErrorString.empty())
	{
		return AppError(std::string("JSON parse error while connecting to Horde server: ") + JsonErrorString);
	}

	if (const Json& Field = JsonObject["serverUrl"]; Field.is_string())
	{
		Result.AuthServerUri = Field.string_value();
	}

	if (const Json& Field = JsonObject["clientId"]; Field.is_string())
	{
		Result.AuthClientId = Field.string_value();
	}

	if (const Json& Field = JsonObject["localRedirectUrls"]; Field.is_array())
	{
		if (Field.array_items().size() && Field.array_items()[0].is_string())
		{
			// TODO: parse all allowed callback URIs
			Result.CallbackUri = Field.array_items()[0].string_value();
		}
	}

	return ResultOk(Result);
}

static std::optional<std::string_view>
FindFirstByPrefix(std::span<const std::string> Strings, std::string_view Prefix)
{
	for (const std::string& It : Strings)
	{
		if (StringStartsWith(It, Prefix, false))
		{
			return std::string_view(It);
		}
	}

	return {};
}

std::optional<std::string_view>
GetMetadataValueByPrefix(std::span<const std::string> MetadataEntries, std::string_view Prefix)
{
	std::optional<std::string_view> Entry = FindFirstByPrefix(MetadataEntries, Prefix);
	if (Entry)
	{
		return Entry->substr(Prefix.length());
	}
	else
	{
		return {};
	}
}

std::optional<std::string_view>
TryStripPrefix(std::string_view String, std::string_view Prefix)
{
	if (StringStartsWith(String, Prefix, false))
	{
		return String.substr(Prefix.length());
	}
	else
	{
		return {};
	}
}

static std::string
UrlEncode(std::string_view String)
{
	std::string Result;
	Result.reserve(String.length() * 3);

	for (char C : String)
	{
		if (IsAsciiAlphaNumericCharacter(C))
		{
			Result.push_back(C);
		}
		else
		{
			char HexChars[2] = {};
			BytesToHexChars(HexChars, 2, reinterpret_cast<const uint8*>(&C), 1);
			Result.push_back('%');
			Result.push_back(HexChars[0]);
			Result.push_back(HexChars[1]);
		}
	}

	return Result;
}

TResult<FHordeArtifactQuery>
FHordeArtifactQuery::FromString(std::string_view Str)
{
	FHordeArtifactQuery Result;

	std::vector<std::string_view> PathParts = SplitBy(Str, PATH_SEPARATOR);
	std::string_view			  RootPath	= PathParts[0];

	for (std::string_view PathPart : PathParts)
	{
		if (!Result.Parts.empty() || PathParts.size() == 1)
		{
			// TODO: derived the ID by parsing the request string using the format specification
			size_t ArtifactIdPos = PathPart.find_last_of('#');
			if (ArtifactIdPos != std::string::npos)
			{
				Result.Id = PathPart.substr(ArtifactIdPos + 1);
			}
		}

		Result.Parts.push_back(std::string(PathPart));
	}

	std::vector<std::string_view> FilterParts = SplitBy(RootPath, ';');

	for (std::string_view FilterPart : FilterParts)
	{
		std::optional<std::string_view> Value;
		if (Value = TryStripPrefix(FilterPart, "type="); Value)
		{
			Result.Type = StringToLower(*Value);
		}
		else if (Value = TryStripPrefix(FilterPart, "key="); Value)
		{
			Result.Keys.push_back(UrlEncode(StringToLower(std::string(*Value))));
		}
		else if (Value = TryStripPrefix(FilterPart, "change="); Value)
		{
			Result.Change = *Value;
		}
		else if (Value = TryStripPrefix(FilterPart, "id="); Value)
		{
			Result.Id = *Value;
		}
		else
		{
			Result.Format = FilterPart;
		}
	}

	return ResultOk(Result);
}

std::string
GetEscapedFormattedNameField(std::string_view StreamName)
{
	std::string Result = std::string(StreamName);
	for (char& C : Result)
	{
		if (C == '/' || C == '\\')
		{
			C = '+';
		}
	}

	return Result;
}

TResult<std::string>
FormatArtifactName(const FHordeArtifactEntry& Artifact, std::string_view Format, std::string_view PlaceholderValue = "[unknown]")
{
	std::string Result;

	while (!Format.empty())
	{
		if (Format.front() == '{')
		{
			size_t ClosingPos = Format.find('}');
			if (ClosingPos == std::string::npos)
			{
				return AppError(fmt::format("Expected to find '{}' character in the format template string: '{}'", '}', Format));
			}

			std::string_view VarName = Format.substr(1, ClosingPos - 1);

			Format = Format.substr(ClosingPos + 1);

			if (UncasedStringEquals(VarName, "dir"))
			{
				Result.push_back(PATH_SEPARATOR);
				continue;
			}

			std::string Key = fmt::format("{}=", VarName);

			// First try to get the value from metadata, then from artifact itself

			std::optional<std::string_view> Value = GetMetadataValueByPrefix(Artifact.Metadata, Key);

			if (!Value)
			{
				if (UncasedStringEquals(Key, "change="))
				{
					Value = Artifact.Change;
				}
				else if (UncasedStringEquals(Key, "id="))
				{
					Value = Artifact.Id;
				}
				else if (UncasedStringEquals(Key, "name="))
				{
					Value = Artifact.Name;
				}
				else if (UncasedStringEquals(Key, "streamid="))
				{
					Value = Artifact.StreamId;
				}
			}

			if (Value)
			{
				Result.append(GetEscapedFormattedNameField(*Value));
			}
			else
			{
				Result.append(PlaceholderValue);
			}
		}
		else
		{
			Result.push_back(Format.front());
			Format = Format.substr(1);
		}
	}

	return ResultOk(Result);
}

TResult<ProxyQuery::FDirectoryListing>
FHordeProtocolImpl::QueryListDirectory(FHttpConnection& Connection, const FAuthDesc* AuthDesc, const std::string& Path)
{
	using ProxyQuery::FDirectoryListing;
	using ProxyQuery::FDirectoryListingEntry;

	FDirectoryListing Result;

	TResult<FHordeArtifactQuery> QueryParseResult = FHordeArtifactQuery::FromString(Path);
	if (QueryParseResult.IsError())
	{
		return MoveError<ProxyQuery::FDirectoryListing>(QueryParseResult);
	}

	const FHordeArtifactQuery& Query = QueryParseResult.GetData();

	std::string_view Root = Query.Parts[0];

	std::string RequiredPathPrefix;
	if (Root.length() != Path.length())
	{
		RequiredPathPrefix = Path.substr(Root.length() + 1);
		if (RequiredPathPrefix.back() != (PATH_SEPARATOR))
		{
			RequiredPathPrefix.push_back(PATH_SEPARATOR);
		}
	}

	std::string RequestUrl = fmt::format("/api/v2/artifacts?maxResults={}", Query.MaxResults);

	if (!Query.Type.empty())
	{
		RequestUrl += fmt::format("&type={}", Query.Type);
	}

	if (!Query.Change.empty())
	{
		RequestUrl += fmt::format("&minChange={0}&maxChange={0}", Query.Change);
	}

	for (const std::string& Key : Query.Keys)
	{
		RequestUrl += fmt::format("&key={}", Key);
	}

	std::string BearerToken;

	TResult<FAuthToken> AuthTokenResult = Authenticate(*AuthDesc, Connection.RecvTimeoutSeconds);
	if (const FAuthToken* AuthToken = AuthTokenResult.TryData())
	{
		BearerToken = AuthToken->Access;
	}

	FHttpRequest Request;
	Request.Url			= RequestUrl;
	Request.BearerToken = BearerToken;

	FHttpResponse Response = HttpRequest(Connection, Request);

	if (!Response.Success())
	{
		return HttpError(Response.Code);
	}

	Response.Buffer.PushBack(0);

	using namespace json11;
	std::string JsonErrorString;
	Json		JsonObject = Json::parse((const char*)Response.Buffer.Data(), JsonErrorString);

	if (!JsonErrorString.empty())
	{
		return AppError(std::string("JSON parse error while listing Horde artifacts: ") + JsonErrorString);
	}

	Json JsonArtifacts = JsonObject["artifacts"];
	if (!JsonArtifacts.is_array())
	{
		return AppError(L"Horde artifact listing is expected to have 'artifacts' array field");
	}

	std::vector<FHordeArtifactEntry> HordeArtifacts;

	for (const Json& It : JsonArtifacts.array_items())
	{
		FHordeArtifactEntry Entry;

		Entry.Name = It["name"].string_value();

		const Json& ChangeJson = It["change"];
		if (ChangeJson.is_number())
		{
			Entry.Change = fmt::format("{}", llu(ChangeJson.number_value()));
		}
		else if (ChangeJson.is_string())
		{
			Entry.Change = ChangeJson.string_value();
		}

		Entry.Id		  = It["id"].string_value();
		Entry.Description = It["description"].string_value();
		Entry.StreamId	  = It["streamId"].string_value();

		for (const Json& KeyJson : It["keys"].array_items())
		{
			const std::string& Key = KeyJson.string_value();
			if (!Key.empty())
			{
				Entry.Keys.push_back(Key);
			}
		}

		for (const Json& MetadataJson : It["metadata"].array_items())
		{
			const std::string& Metadata = MetadataJson.string_value();
			if (!Metadata.empty())
			{
				Entry.Metadata.push_back(Metadata);
			}
		}

		HordeArtifacts.push_back(Entry);
	}

	std::sort(HordeArtifacts.begin(),
			  HordeArtifacts.end(),
			  [](const FHordeArtifactEntry& A, const FHordeArtifactEntry& B) { return A.Change > B.Change; });

	if (!Query.Format.empty())
	{
		for (const FHordeArtifactEntry& HordeArtifactEntry : HordeArtifacts)
		{
			FDirectoryListingEntry DirectoryEntry;

			TResult<std::string> FormattedName = FormatArtifactName(HordeArtifactEntry, Query.Format);

			if (FormattedName.IsOk())
			{
				if (FormattedName->starts_with(RequiredPathPrefix))
				{
					std::string_view Name = std::string_view(*FormattedName).substr(RequiredPathPrefix.length());

					DirectoryEntry.bDirectory = true;
					DirectoryEntry.Name		  = Name;

					Result.Entries.push_back(DirectoryEntry);
				}
			}
			else
			{
				LogError(FormattedName.GetError(), L"Failed to format Horde artifact name");
				break;
			}
		}
	}
	else
	{
		for (const FHordeArtifactEntry& HordeArtifactEntry : HordeArtifacts)
		{
			FDirectoryListingEntry DirectoryEntry;
			DirectoryEntry.bDirectory = true;
			DirectoryEntry.Name		  = fmt::format("#{}", HordeArtifactEntry.Id);
			Result.Entries.push_back(DirectoryEntry);
		}
	}

	// Create intermediate virtual directories
	{
		std::vector<FDirectoryListingEntry> VirtualEntries;
		std::unordered_set<std::string>		UniqueEntries;

		for (const FDirectoryListingEntry& Entry : Result.Entries)
		{
			UniqueEntries.insert(Entry.Name);
		}

		for (const FDirectoryListingEntry& Entry : Result.Entries)
		{
			std::vector<std::string_view> Parts = SplitBy(Entry.Name, PATH_SEPARATOR);
			std::string					  PartialName;
			for (std::string_view Part : Parts)
			{
				if (!PartialName.empty())
				{
					PartialName.push_back(PATH_SEPARATOR);
				}
				PartialName += Part;

				bool bInserted = UniqueEntries.insert(PartialName).second;
				if (bInserted)
				{
					FDirectoryListingEntry VirtualEntry;
					VirtualEntry.Name = PartialName;
					VirtualEntry.bDirectory = true;
					VirtualEntries.push_back(VirtualEntry);
				}
			}
		}

		for (const FDirectoryListingEntry& Entry : VirtualEntries)
		{
			Result.Entries.push_back(Entry);
		}
	}

	return ResultOk(Result);
}

TResult<FDirectoryManifest> DecodeHordeManifestJson(const char* JsonString, std::string_view ArtifactRoot)
{
	using namespace json11;
	std::string JsonErrorString;
	Json		JsonObject = Json::parse(JsonString, JsonErrorString);

	if (!JsonErrorString.empty())
	{
		return AppError(std::string("JSON parse error while downloading manifest from Horde server: ") + JsonErrorString);
	}

	const uint32 DefaultBlockSize = uint32(64_KB); // TODO: get from manifest JSON

	FDirectoryManifest Manifest;
	Manifest.Version = FDirectoryManifest::VERSION;

	if (JsonObject["type"] != "unsync_manifest")
	{
		return AppError("Manifest JSON is expected to have a 'type' string field with 'unsync_manifest' value");
	}

	if (const Json& Field = JsonObject["hash_strong"]; Field.is_string())
	{
		const std::string Value = StringToLower(Field.string_value());

		if (Value == "md5")
		{
			Manifest.Algorithm.StrongHashAlgorithmId = EStrongHashAlgorithmID::MD5;
		}
		else if (Value == "blake3.128")
		{
			Manifest.Algorithm.StrongHashAlgorithmId = EStrongHashAlgorithmID::Blake3_128;
		}
		else if (Value == "blake3.160" || Value == "iohash")
		{
			Manifest.Algorithm.StrongHashAlgorithmId = EStrongHashAlgorithmID::Blake3_160;
		}
		else if (Value == "blake3.256")
		{
			Manifest.Algorithm.StrongHashAlgorithmId = EStrongHashAlgorithmID::Blake3_256;
		}
		else
		{
			return AppError(fmt::format("Unsupported strong hash algorithm '{}'", Value));
		}
	}

	if (const Json& Field = JsonObject["hash_weak"]; Field.is_string())
	{
		const std::string Value = StringToLower(Field.string_value());

		if (Value == "buzhash")
		{
			Manifest.Algorithm.WeakHashAlgorithmId = EWeakHashAlgorithmID::BuzHash;
		}
		else if (Value == "naive")
		{
			Manifest.Algorithm.WeakHashAlgorithmId = EWeakHashAlgorithmID::Naive;
		}
		else
		{
			return AppError(fmt::format("Unsupported weak hash algorithm '{}'", Value));
		}
	}

	if (const Json& Field = JsonObject["chunking"]; Field.is_string())
	{
		const std::string Value = StringToLower(Field.string_value());

		if (Value == "variable")
		{
			Manifest.Algorithm.ChunkingAlgorithmId = EChunkingAlgorithmID::VariableBlocks;
		}
		else if (Value == "fixed")
		{
			Manifest.Algorithm.ChunkingAlgorithmId = EChunkingAlgorithmID::FixedBlocks;
		}
		else if (Value == "rollingbuzhash")
		{
			Manifest.Algorithm.ChunkingAlgorithmId = EChunkingAlgorithmID::VariableBlocks;
			Manifest.Algorithm.WeakHashAlgorithmId = EWeakHashAlgorithmID::BuzHash;
		}
		else
		{
			return AppError(fmt::format("Unsupported chunking algorithm '{}'", Value));
		}
	}

	const auto FileTimeNow = std::chrono::time_point<std::chrono::file_clock>::clock::now();
	const uint64 CurrentWindowsFileTime = ToWindowsFileTime(FileTimeNow);
	uint32		 NumInvalidTimestamps	= 0;

	if (const Json& FiledField = JsonObject["files"]; FiledField.is_array())
	{
		for (const Json& FileObject : FiledField.array_items())
		{
			std::string FileNameUtf8 = FileObject["name"].string_value();
			std::wstring FileName	  = ConvertUtf8ToWide(FileNameUtf8);

			// Don't include the actual native unsync manifest
			if (FileName.starts_with(L".unsync"))
			{
				continue;
			}

			ConvertDirectorySeparatorsToNative(FileName);

			FFileManifest FileManifest;
			FileManifest.BlockSize = DefaultBlockSize;
			FileManifest.CurrentPath = FileName;

			if (const Json& Field = FileObject["size"]; Field.is_number())
			{
				FileManifest.Size = uint64(Field.number_value());
			}

			if (const Json& Field = FileObject["mtime"]; Field.is_number())
			{
				FileManifest.Mtime = uint64(Field.number_value());
			}

			if (FileManifest.Mtime == 0)
			{
				FileManifest.Mtime = CurrentWindowsFileTime;
				++NumInvalidTimestamps;
			}

			if (const Json& Field = FileObject["read_only"]; Field.is_bool())
			{
				FileManifest.bReadOnly = Field.bool_value();
			}

			if (const Json& Field = FileObject["executable"]; Field.is_bool())
			{
				FileManifest.bIsExecutable = Field.bool_value();
			}

			if (const Json& BlocksField = FileObject["blocks"]; BlocksField.is_array())
			{
				for (const Json& BlockObject : BlocksField.array_items())
				{
					FGenericBlock Block;
					Block.Offset = uint64(BlockObject["offset"].number_value());
					Block.Size = uint32(BlockObject["size"].number_value());
					Block.HashWeak = uint32(BlockObject["hash_weak"].number_value());
					const std::string& StrongHashStr = BlockObject["hash_strong"].string_value();
					if (!ParseHashFromHexString(Manifest.Algorithm.StrongHashAlgorithmId, StrongHashStr, Block.HashStrong))
					{
						return AppError(fmt::format("Failed to parse block strong hash '{}'", StrongHashStr));
					}
					FileManifest.Blocks.push_back(Block);
				}
			}

			Manifest.Files[FileName] = FileManifest;
		}
	}

	if (NumInvalidTimestamps != 0)
	{
		UNSYNC_WARNING(L"Manifest contains files with invalid time stamps (%d) which were set to current time", NumInvalidTimestamps);
	}

	return ResultOk(std::move(Manifest));
}

static const char* GTestHordeManifestJson = R"(
{
  "type": "unsync_manifest",
  "hash_strong": "Blake3.160",
  "chunking": "Variable",
  "files": [
    {
      "name": "hello_world.txt",
      "read_only": false,
      "size": 1095,
      "blocks": [
        {
          "offset": 0,
          "size": 1095,
          "hash_strong": "1d9f987b21a19769b758f6fc6354808752620d20"
        },
        {
          "offset": 1095,
          "size": 1024,
          "hash_strong": "f52b611e85cf46f466aac6eee0f69b87dff37831"
        }
      ]
    }
  ]
}
)";

void TestHordeManifestDecode()
{
	UNSYNC_LOG(L"TestHordeManifestDecode()");
	UNSYNC_LOG_INDENT;

	TResult<FDirectoryManifest> Manifest = DecodeHordeManifestJson(GTestHordeManifestJson, "api/v2/artifacts/12345");
	if (Manifest.IsError())
	{
		LogError(Manifest.GetError(), L"Failed to decode Horde manifest from JSON");
	}
}


void
TestHordeArtifactFormat()
{
	UNSYNC_LOG(L"TestHordeArtifactFormat()");
	UNSYNC_LOG_INDENT;

	FHordeArtifactEntry Artifact;
	Artifact.Change	  = "12345";
	Artifact.Id		  = "aabbcc";
	Artifact.Name	  = "test_name";
	Artifact.Metadata.push_back("foo=bar");
	Artifact.Metadata.push_back("stream=test_stream");
	Artifact.Metadata.push_back("buildname=build_name");

	TResult<std::string> Name = FormatArtifactName(Artifact, "{Stream}.{BuildName}.{Change}");
	UNSYNC_ASSERT(Name.GetData() == "test_stream.build_name.12345");
}

}
