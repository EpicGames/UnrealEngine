// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCommon.h"
#include "UnsyncProxy.h"

#include <optional>

namespace unsync {

class FBlockRequestMap;
struct FRemoteDesc;
struct FAuthDesc;

struct FHordeProtocolImpl : FRemoteProtocolBase
{
	FHordeProtocolImpl(const FRemoteDesc& InRemoteDesc, const FBlockRequestMap* InRequestMap, FProxyPool& InProxyPool);

	virtual bool IsValid() const { return bValid; }
	virtual void Invalidate() { bValid = false; }

	virtual FDownloadResult	 Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback);
	virtual TResult<FDirectoryManifest> DownloadManifest(std::string_view ManifestName);

	static TResult<ProxyQuery::FHelloResponse> QueryHello(FHttpConnection& HttpConnection);
	static TResult<ProxyQuery::FDirectoryListing> QueryListDirectory(FHttpConnection&	Connection,
																	 const FAuthDesc*	AuthDesc,
																	 const std::string& Path);

	FProxyPool& ProxyPool;
	bool bValid = true;

	FDownloadResult DownloadArtifactBlobs(std::string_view ArtifactPath, const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback);
};


struct FHordeArtifactQuery
{
	std::string Type;
	std::string Id;
	std::string Change;
	std::vector<std::string> Keys;

	uint32 MaxResults = 1000000;

	// Template to use for formatting results, e.g.: `{StreamName}.{BuildName}/{Platform}#{Id}`
	std::string Format;

	// Query parts, delimited by directory separator character
	std::vector<std::string> Parts;

	static TResult<FHordeArtifactQuery> FromString(std::string_view Str);
};

struct FHordeArtifactEntry
{
	std::string				 Change;
	std::string				 Id;
	std::string				 Name;
	std::string				 Description;
	std::string				 StreamId;
	std::vector<std::string> Keys;
	std::vector<std::string> Metadata;
};

bool RequestPathLooksLikeHordeArtifact(std::string_view RequestPath);

TResult<FDirectoryManifest> DecodeHordeManifestJson(const char* JsonString, std::string_view ArtifactRoot);

}  // namespace unsync
