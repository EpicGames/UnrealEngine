// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncHttp.h"
#include "UnsyncCore.h"
#include "UnsyncUtil.h"
#include "UnsyncRemote.h"
#include "UnsyncAuth.h"
#include "UnsyncVersion.h"

#include <http_parser.h>
#include <string.h>
#include <functional>

#ifdef __GNUC__
#	define _strnicmp strncasecmp
#endif

namespace unsync {

enum class EViewUpdateResult
{
	Append,
	Replace,
};

static EViewUpdateResult
UpdateView(std::string_view& View, const char* Data, size_t Size)
{
	if (View.data() + View.length() == Data)
	{
		View = std::string_view(View.data(), View.length() + Size);
		return EViewUpdateResult::Append;
	}
	else
	{
		View = std::string_view(Data, Size);
		return EViewUpdateResult::Replace;
	}
}

std::string_view FHttpResponse::FindHeader(const std::string_view Name) const
{
	for (const auto& It : Headers)
	{
		if (UncasedStringEquals(It.first, Name))
		{
			return It.second;
		}
	}
	return {};
}

struct FHttpParser
{
	static FHttpParser* ToThis(http_parser* Parser) { return (FHttpParser*)(Parser->data); }

	FHttpParser(FHttpMessageCallback InResponseCallback,
				FHttpChunkCallback   InChunkCallback,
				uint8*				InScratchBuffer,
				uint64				InScratchSize,
				http_parser_type	Type,
				EHttpMethod			InMethod)
	: ResponseCallback(InResponseCallback)
	, ChunkCallback(InChunkCallback)
	, Method(InMethod)
	, ScratchBuffer(InScratchBuffer)
	, ScratchSize(InScratchSize)
	{
		http_parser_settings_init(&Settings);
		http_parser_init(&Parser, Type);

		Parser.data = this;

		Settings.on_message_begin	 = [](http_parser* P) { return ToThis(P)->OnMsgBegin(); };
		Settings.on_message_complete = [](http_parser* P) { return ToThis(P)->OnMsgComplete(); };
		Settings.on_header_field	 = [](http_parser* P, const char* Data, size_t Size) { return ToThis(P)->OnHdrField(Data, Size); };
		Settings.on_header_value	 = [](http_parser* P, const char* Data, size_t Size) { return ToThis(P)->OnHdrValue(Data, Size); };
		Settings.on_headers_complete = [](http_parser* P) { return ToThis(P)->OnHdrComplete(); };
		Settings.on_body			 = [](http_parser* P, const char* Data, size_t Size) { return ToThis(P)->OnBody(Data, Size); };
		Settings.on_status			 = [](http_parser* P, const char* Data, size_t Size) { return ToThis(P)->OnStatus(Data, Size); };
		Settings.on_chunk_header	 = [](http_parser* P) { return ToThis(P)->OnChunkHeader(); };
		Settings.on_chunk_complete	 = [](http_parser* P) { return ToThis(P)->OnChunkComplete(); };
	}

	void Reset()
	{
		bComplete		= false;
		bHeaderComplete = false;
		PendingHeader	= {};
		PendingValue	= {};
		ScratchCursor	= 0;

		Response = FHttpResponse();
	}

	int OnMsgBegin()
	{
		Reset();
		return 0;
	}

	int OnMsgComplete()
	{
		bComplete = true;
		ResponseCallback(std::move(Response));

		return 0;
	}

	int OnChunkHeader()
	{ 
		return 0;
	}

	int OnChunkComplete()
	{
		if (ChunkCallback)
		{
			ChunkCallback(Response);
		}
		return 0;
	}

	int OnHdrField(const char* Data, size_t Size)
	{
		if (UpdateView(PendingHeader, Data, Size) == EViewUpdateResult::Replace)
		{
			PendingValue = {};
		}
		return 0;
	}

	int OnHdrValue(const char* Data, size_t Size)
	{
		auto MatchUncased = [](std::string_view A, std::string_view B) {
			return _strnicmp(A.data(), B.data(), std::min(A.length(), B.length())) == 0;
		};

		UpdateView(PendingValue, Data, Size);

		if (MatchUncased(PendingHeader, "content-length"))
		{
			ContentLength		   = strtoull(Data, nullptr, 10);
			Response.ContentLength = uint64(ContentLength);
			Response.Buffer.Reserve(ContentLength);
		}
		else if (MatchUncased(PendingHeader, "content-type"))
		{
			if (MatchUncased(PendingValue, "application/json"))
			{
				Response.ContentType = EHttpContentType::Application_Json;
			}
			else if (MatchUncased(PendingValue, "application/octet-stream"))
			{
				Response.ContentType = EHttpContentType::Application_OctetStream;
			}
			else if (MatchUncased(PendingValue, "application/x-ue-cb"))
			{
				Response.ContentType = EHttpContentType::Application_UECB;
			}
			else if (MatchUncased(PendingValue, "application/x-www-form-urlencoded"))
			{
				Response.ContentType = EHttpContentType::Application_WWWFormUrlEncoded;
			}
			else if (MatchUncased(PendingValue, "text/html"))
			{
				Response.ContentType = EHttpContentType::Text_Html;
			}
			else if (MatchUncased(PendingValue, "text/plain"))
			{
				Response.ContentType = EHttpContentType::Text_Plain;
			}
		}

		Response.Headers.push_back(std::make_pair(std::string(PendingHeader), std::string(PendingValue)));

		return 0;
	}

	int OnHdrComplete()
	{
		Response.Code	= Parser.status_code;
		bHeaderComplete = true;

		if (Method == EHttpMethod::HEAD)
		{
			return 1;
		}
		else
		{
			return 0;
		}
	}

	int OnStatus(const char* Data, size_t Size) { return 0; }

	int OnBody(const char* Data, size_t Size)
	{
		// TODO: make a zero-copy path
		Response.Buffer.Append((const uint8*)Data, Size);
		ScratchCursor = 0;

		return 0;
	}

	// Receives data from the socket and executes HTTP parser state machine.
	// Returns true if more data is expected and this function should be called again.
	bool Recv(FSocketBase& Socket)
	{
		int32 RecvSize = 0;

		uint8* RecvBuffer  = nullptr;
		uint64 RecvMaxSize = 0;

		RecvBuffer	= ScratchBuffer + ScratchCursor;
		RecvMaxSize = ScratchSize - std::min<uint64>(ScratchCursor, ScratchSize);

		RecvSize = SocketRecvAny(Socket, RecvBuffer, RecvMaxSize);

		std::string_view RecvString((const char*)RecvBuffer, RecvSize);

		uint64 ParsedBytes = 0;
		if (RecvSize > 0)
		{
			ScratchCursor += RecvSize;
			ParsedBytes = http_parser_execute(&Parser, &Settings, RecvString.data(), RecvString.size());

			TotalReceivedBytes += RecvSize;
			TotalParsedBytes += ParsedBytes;
		}

		bool bShouldContinue = RecvSize > 0 && ParsedBytes != 0 && Parser.http_errno == 0;

		if (!bShouldContinue && bHeaderComplete && !bComplete)
		{
			OnMsgComplete();
		}

		return bShouldContinue;
	}

	FHttpMessageCallback ResponseCallback;
	FHttpChunkCallback ChunkCallback;
	FHttpResponse		Response;

	EHttpMethod Method = EHttpMethod::GET;

	http_parser_settings Settings;
	http_parser			 Parser;

	uint8* ScratchBuffer;
	uint64 ScratchSize;
	uint64 ScratchCursor = 0;

	bool bComplete		 = false;
	bool bHeaderComplete = false;

	std::string_view PendingHeader = {};
	std::string_view PendingValue  = {};

	uint64 ContentLength = 0;

	uint64 TotalReceivedBytes = 0;
	uint64 TotalParsedBytes	  = 0;
};

FHttpResponse
HttpRequest(FHttpConnection& Connection, const FHttpRequest& Request, FHttpChunkCallback ChunkCallback)
{
	FHttpResponse Result;

	if (HttpRequestBegin(Connection, Request))
	{
		Result = HttpRequestEnd(Connection, ChunkCallback);
	}

	return Result;
}

FHttpResponse
HttpRequest(const FRemoteDesc& RemoteDesc,
			EHttpMethod		   Method,
			std::string_view   RequestUrl,
			EHttpContentType   PayloadContentType,
			FBufferView		   Payload,
			std::string_view   BearerToken)
{
	FTlsClientSettings TlsSettings = RemoteDesc.GetTlsClientSettings();
	FHttpConnection	   Connection(RemoteDesc.Host.Address, RemoteDesc.Host.Port, RemoteDesc.TlsRequirement, TlsSettings, RemoteDesc.RecvTimeoutSeconds);

	FHttpRequest Request;

	Request.Method			   = Method;
	Request.CustomHeaders	   = RemoteDesc.HttpHeaders;
	Request.Url				   = RequestUrl;
	Request.PayloadContentType = PayloadContentType;
	Request.Payload			   = Payload;
	Request.BearerToken		   = BearerToken;

	FHttpResponse Response = HttpRequest(Connection, Request);

	return Response;
}

FHttpResponse
HttpRequest(const FRemoteDesc& RemoteDesc, EHttpMethod Method, std::string_view RequestUrl, std::string_view BearerToken)
{
	return HttpRequest(RemoteDesc, Method, RequestUrl, EHttpContentType::Unknown, /*Payload*/ {}, BearerToken);
}

static const char*
ToString(EHttpMethod Method)
{
	switch (Method)
	{
		default:
			UNSYNC_FATAL(L"Unexpected HTTP method %d", (int)Method);
			return "INVALID";
		case EHttpMethod::GET:
			return "GET";
		case EHttpMethod::HEAD:
			return "HEAD";
		case EHttpMethod::POST:
			return "POST";
		case EHttpMethod::PUT:
			return "PUT";
	}
}

bool
HttpRequestBegin(FHttpConnection& Connection, const FHttpRequest& Request)
{
	bool bConnected = Connection.Open();
	if (!bConnected)
	{
		return false;
	}

	if (Connection.NumActiveRequests > 0 && Connection.Method != Request.Method)
	{
		UNSYNC_ERROR(L"HTTP connection must not have outstanding requests when request method is changed");
		return false;
	}

	Connection.Method = Request.Method;

	// TODO: use a string builder
	std::string HttpHeader;
	HttpHeader.append(ToString(Request.Method));
	HttpHeader.append(" ");
	HttpHeader.append(Request.Url);
	HttpHeader.append(" HTTP/1.1\r\n");

	if (!Request.CustomHeaders.empty())
	{
		HttpHeader.append(Request.CustomHeaders);

		if (!HttpHeader.ends_with("\r\n"))
		{
			HttpHeader += "\r\n";
		}
	}

	if (!Request.BearerToken.empty())
	{
		HttpHeader += "Authorization: Bearer ";
		HttpHeader += Request.BearerToken;
		HttpHeader += "\r\n";
	}

	HttpHeader += "Host: " + Connection.HostAddress + "\r\n";

	HttpHeader += "User-Agent: unsync v" + GetVersionString() + "\r\n";

	if (Connection.bKeepAlive)
	{
		HttpHeader += "Connection: keep-alive\r\n";
	}

	if (Request.Payload.Size)
	{
		switch (Request.PayloadContentType)
		{
			case EHttpContentType::Text_Html:
				HttpHeader += "Content-type: text/html\r\n";
				break;
			case EHttpContentType::Text_Plain:
				HttpHeader += "Content-type: text/plain\r\n";
				break;
			case EHttpContentType::Application_OctetStream:
				HttpHeader += "Content-type: application/octet-stream\r\n";
				break;
			case EHttpContentType::Application_UECB:
				HttpHeader += "Content-type: application/x-ue-cb\r\n";
				break;
			case EHttpContentType::Application_WWWFormUrlEncoded:
				HttpHeader += "Content-type: application/x-www-form-urlencoded\r\n";
				break;
			case EHttpContentType::Application_Json:
				HttpHeader += "Content-type: application/json\r\n";
				break;

			default:
				UNSYNC_FATAL(L"HTTP content type not supported");
		}
	}

	if (Request.Payload.Size || Request.Method == EHttpMethod::POST)
	{
		char LengthStr[64];
		snprintf(LengthStr, sizeof(LengthStr), "Content-length: %llu\r\n", (long long unsigned)Request.Payload.Size);
		HttpHeader += LengthStr;
	}

	if (Request.AcceptContentType == EHttpContentType::Unknown)
	{
		HttpHeader += "Accept: */*\r\n";
	}
	else
	{
		switch (Request.AcceptContentType)
		{
			case EHttpContentType::Text_Html:
				HttpHeader += "Accept: text/html\r\n";
				break;
			case EHttpContentType::Text_Plain:
				HttpHeader += "Accept: text/plain\r\n";
				break;
			case EHttpContentType::Application_OctetStream:
				HttpHeader += "Accept: application/octet-stream\r\n";
				break;
			case EHttpContentType::Application_UECB:
				HttpHeader += "Accept: application/x-ue-cb\r\n";
				break;
			case EHttpContentType::Application_Json:
				HttpHeader += "Accept: application/json\r\n";
				break;

			default:
				UNSYNC_FATAL(L"HTTP content type not supported");
		}
	}

	// Finish HTTP header section
	HttpHeader += "\r\n";

	int32 SentBytes = 0;

	Connection.NumActiveRequests += 1;

	UNSYNC_VERBOSE2(L"HTTP %hs %.*hs", ToString(Request.Method), int32(Request.Url.length()), Request.Url.data());

	// TODO: detect and handle errors
	SentBytes += SocketSend(Connection.GetSocket(), HttpHeader.c_str(), HttpHeader.length());
	if (Request.Payload.Size)
	{
		SentBytes += SocketSend(Connection.GetSocket(), Request.Payload.Data, Request.Payload.Size);
	}

	uint64 ExpectedSentBytes = HttpHeader.length() + Request.Payload.Size;

	return SentBytes == ExpectedSentBytes;
}

FHttpResponse
HttpRequestEnd(FHttpConnection& Connection, FHttpChunkCallback ChunkCallback)
{
	FHttpResponse Result;

	if (!Connection.ResponseQueue.empty())
	{
		std::swap(Result, Connection.ResponseQueue.front());
		Connection.ResponseQueue.pop_front();
		Connection.NumActiveRequests -= 1;
		return Result;
	}

	uint64 NumMessages	   = 0;
	auto   MessageCallback = [&NumMessages, &Result, &Connection](FHttpResponse&& Response) {
		  if (NumMessages == 0)
		  {
			  Result = std::move(Response);
		  }
		  else
		  {
			  Connection.ResponseQueue.push_back(std::move(Response));
		  }

		  NumMessages++;
	};

	// TODO: dynamic scratch buffer, if headers are pathologically large
	// TODO: user-provided scratch buffer
	uint8 ScratchBuffer[256_KB];
	ScratchBuffer[0] = 0;
	FHttpParser Parser(MessageCallback, ChunkCallback, ScratchBuffer, sizeof(ScratchBuffer), HTTP_RESPONSE, Connection.Method);

	while (!Parser.bComplete)
	{
		if (!Parser.Recv(Connection.GetSocket()))
		{
			break;
		}
	}

	if (Parser.TotalParsedBytes != Parser.TotalReceivedBytes && !Parser.bComplete)
	{
		UNSYNC_FATAL(L"TODO: save the unparsed scratch buffer for next time!");
	}

	Connection.NumActiveRequests -= 1;

	Result.bConnectionEncrypted = Connection.IsEncrypted();

	// TODO: report errors

	return Result;
}

FHttpConnection::FHttpConnection(const std::string_view	   InHostAddress,
								 uint16					   InPort,
								 ETlsRequirement		   InTlsRequirement,
								 const FTlsClientSettings& InTlsSettings,
								 uint32					   InRecvTimeoutSeconds)
: HostAddress(InHostAddress)
, HostPort(InPort)
, TlsRequirement(InTlsRequirement)
, RecvTimeoutSeconds(InRecvTimeoutSeconds)
{
	if (InTlsSettings.Subject.empty())
	{
		TlsSubject = std::string(InHostAddress);
	}
	else
	{
		TlsSubject = std::string(InTlsSettings.Subject);
	}

	bTlsVerifyCertificate = InTlsSettings.bVerifyCertificate;
	bTlsVerifySubject	  = InTlsSettings.bVerifySubject;
	if (InTlsSettings.CACert.Data)
	{
		TlsCacert = std::make_shared<FBuffer>();
		TlsCacert->Append(InTlsSettings.CACert.Data, InTlsSettings.CACert.Size);
	}
}

FHttpConnection::FHttpConnection(const FHttpConnection& Other)
: HostAddress(Other.HostAddress)
, HostPort(Other.HostPort)
, bKeepAlive(Other.bKeepAlive)
, TlsRequirement(Other.TlsRequirement)
, bTlsVerifySubject(Other.bTlsVerifySubject)
, TlsSubject(Other.TlsSubject)
, bTlsVerifyCertificate(Other.bTlsVerifyCertificate)
, TlsCacert(Other.TlsCacert)
, RecvTimeoutSeconds(Other.RecvTimeoutSeconds)
{
}

FHttpConnection
FHttpConnection::CreateDefaultHttp(const std::string_view InHostAddress, uint16 Port, uint32 RecvTimeoutSeconds)
{
	return FHttpConnection(InHostAddress, Port, RecvTimeoutSeconds);
}

FHttpConnection
FHttpConnection::CreateDefaultHttps(const std::string_view InHostAddress, uint16 Port, uint32 RecvTimeoutSeconds)
{
	FTlsClientSettings TlsSettings;
	TlsSettings.Subject = InHostAddress.data();
	return FHttpConnection(InHostAddress, Port, ETlsRequirement::Required, TlsSettings, RecvTimeoutSeconds);
}

FHttpConnection
FHttpConnection::CreateDefaultHttps(const FRemoteDesc& RemoteDesc)
{
	FTlsClientSettings TlsSettings = RemoteDesc.GetTlsClientSettings();
	return FHttpConnection(RemoteDesc.Host.Address, RemoteDesc.Host.Port, ETlsRequirement::Required, TlsSettings, RemoteDesc.RecvTimeoutSeconds);
}

bool
FHttpConnection::Open()
{
	if (Socket.get())
	{
		if (SocketValid(*Socket))
		{
			return true;
		}
	}

	if (TlsRequirement != ETlsRequirement::None)
	{
		FSocketHandle RawSocketHandle = SocketConnectTcp(HostAddress.c_str(), HostPort);
		if (!SocketValid(RawSocketHandle))
		{
			return false;
		}

		SocketSetRecvTimeout(RawSocketHandle, RecvTimeoutSeconds);

		FTlsClientSettings ClientSettings;
		ClientSettings.bVerifyCertificate = bTlsVerifyCertificate;
		ClientSettings.bVerifySubject	  = bTlsVerifySubject;
		ClientSettings.Subject			  = TlsSubject;

		if (TlsCacert && !TlsCacert->Empty())
		{
			ClientSettings.CACert = TlsCacert->View();
		}

		FSocketTls* TlsSocket = new FSocketTls(RawSocketHandle, ClientSettings);

		if (TlsSocket->IsTlsValid())
		{
			Socket = std::unique_ptr<FSocketTls>(TlsSocket);
		}
		else
		{
			delete TlsSocket;
			TlsSocket = nullptr;
		}
	}

	if (!Socket && TlsRequirement != ETlsRequirement::Required)
	{
		FSocketHandle RawSocketHandle = SocketConnectTcp(HostAddress.c_str(), HostPort);
		if (!SocketValid(RawSocketHandle))
		{
			return false;
		}

		SocketSetRecvTimeout(RawSocketHandle, RecvTimeoutSeconds);

		Socket = std::unique_ptr<FSocketRaw>(new FSocketRaw(RawSocketHandle));
	}

	return Socket.get() && SocketValid(*Socket);
}

void
FHttpConnection::Close()
{
	NumActiveRequests = 0;
	ResponseQueue.clear();

	Socket = {};
}

const char*
HttpStatusToString(int32 Code)
{
	if (Code == 0)
	{
		return "Failed to establish connection";
	}
	else
	{
		return http_status_str((http_status)Code);
	}
}

}  // namespace unsync
