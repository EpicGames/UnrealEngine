// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncBuffer.h"
#include "UnsyncCommon.h"
#include "UnsyncSocket.h"
#include "UnsyncUtil.h"

#include <deque>
#include <memory>
#include <string>
#include <functional>

namespace unsync {

struct FRemoteDesc;
struct FHttpResponse;

enum class EHttpContentType
{
	Unknown,
	Text_Html,
	Text_Plain,
	Application_OctetStream,
	Application_Json,
	Application_WWWFormUrlEncoded,
	Application_UECB,  // Unreal Engine Compact Binary
};

enum class EHttpMethod
{
	GET,
	HEAD,
	POST,
	PUT,
};

using FHttpMessageCallback = std::function<void(FHttpResponse&& Response)>;
using FHttpChunkCallback   = std::function<void(FHttpResponse& Response)>;

struct FHttpRequest
{
	EHttpMethod		 Method		   = EHttpMethod::GET;
	std::string_view Url		   = {};
	std::string_view CustomHeaders = {};
	std::string_view BearerToken   = {};

	EHttpContentType PayloadContentType = EHttpContentType::Unknown;
	FBufferView		 Payload			= {};

	EHttpContentType AcceptContentType = EHttpContentType::Unknown;
};

struct FHttpResponse
{
	FBuffer Buffer;	 // TODO: use pooled IOBuffer
	uint64	ContentLength = 0;
	int32	Code = 0;

	EHttpContentType ContentType = EHttpContentType::Unknown;

	std::vector<std::pair<std::string, std ::string>> Headers;

	bool bConnectionEncrypted = false;

	bool Success() const { return Code >= 200 && Code < 300; }

	std::string_view AsStringView() const { return unsync::AsStringView(Buffer); }

	std::string_view FindHeader(const std::string_view Name) const;
};

struct FHttpConnection
{
	FHttpConnection(const std::string_view InHostAddress, uint16 InPort, uint32 InRecvTimeoutSeconds) : FHttpConnection(InHostAddress, InPort, ETlsRequirement::None, FTlsClientSettings{}, InRecvTimeoutSeconds) {}
	FHttpConnection(const std::string_view InHostAddress, uint16 InPort, ETlsRequirement InTlsRequirement, const FTlsClientSettings& InTlsSettings, uint32 InRecvTimeoutSeconds);
	FHttpConnection(const FHttpConnection& Other);

	[[nodiscard]] static FHttpConnection CreateDefaultHttp(const std::string_view InHostAddress, uint16 Port = 80, uint32 RecvTimeoutSeconds = 0);
	[[nodiscard]] static FHttpConnection CreateDefaultHttps(const std::string_view InHostAddress, uint16 Port = 443, uint32 RecvTimeoutSeconds = 0);
	[[nodiscard]] static FHttpConnection CreateDefaultHttps(const FRemoteDesc& RemoteDesc);

	bool Open();
	void Close();

	std::string		HostAddress;
	uint16			HostPort	   = 80;

	bool bKeepAlive = true;

	ETlsRequirement			 TlsRequirement	   = ETlsRequirement::None;
	bool					 bTlsVerifySubject = true;
	std::string				 TlsSubject;
	bool					 bTlsVerifyCertificate = true;
	std::shared_ptr<FBuffer> TlsCacert;

	uint32 RecvTimeoutSeconds = 0;

	uint64 NumActiveRequests = 0;

	// TODO: use single memory allocation for multiple reponse objects (perhaps a ring buffer)
	std::deque<FHttpResponse> ResponseQueue;  // Contains HTTP responses for pipelined requests

	EHttpMethod Method = EHttpMethod::GET;

	FTimePoint LastUsed = {};

	FSocketBase& GetSocket()
	{
		LastUsed = TimePointNow();
		return *Socket;
	}

	bool IsEncrypted() const { return Socket && Socket->IsEncrypted(); }

protected:
	std::unique_ptr<FSocketBase> Socket;
};

const char* HttpStatusToString(int32 Code);

// Synchronous HTTP request API

FHttpResponse HttpRequest(FHttpConnection& Connection, const FHttpRequest& Request, FHttpChunkCallback ChunkCallback = {});

inline FHttpResponse
HttpRequest(FHttpConnection& Connection,
			EHttpMethod		 Method,
			std::string_view Url,
			EHttpContentType ContentType,
			FBufferView		 Payload,
			std::string_view CustomHeaders = {},
			std::string_view BearerToken = {})
{
	FHttpRequest Request;

	Request.Method			   = Method;
	Request.Url				   = Url;
	Request.PayloadContentType = ContentType;
	Request.Payload			   = Payload;
	Request.CustomHeaders	   = CustomHeaders;
	Request.BearerToken		   = BearerToken;

	return HttpRequest(Connection, Request);
}

inline FHttpResponse
HttpRequest(FHttpConnection& Connection,
			EHttpMethod		 Method,
			std::string_view Url,
			std::string_view CustomHeaders = {},
			std::string_view BearerToken   = {})
{
	FHttpRequest Request;
	Request.Method		  = Method;
	Request.Url			  = Url;
	Request.CustomHeaders = CustomHeaders;
	Request.BearerToken	  = BearerToken;
	return HttpRequest(Connection, Request);
}

FHttpResponse HttpRequest(const FRemoteDesc& RemoteDesc,
						  EHttpMethod		 Method,
						  std::string_view	 RequestUrl,
						  std::string_view	 BearerToken = {});

FHttpResponse HttpRequest(const FRemoteDesc& RemoteDesc,
						  EHttpMethod		 Method,
						  std::string_view	 RequestUrl,
						  EHttpContentType	 PayloadContentType,
						  FBufferView		 Payload,
						  std::string_view	 BearerToken = {});

// Pipelined HTTP request API

bool HttpRequestBegin(FHttpConnection& Connection, const FHttpRequest& Request);

FHttpResponse HttpRequestEnd(FHttpConnection& Connection, FHttpChunkCallback ChunkCallback = {});

}  // namespace unsync
