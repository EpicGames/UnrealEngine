// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncJupiter.h"
#include "UnsyncCore.h"
#include "UnsyncFile.h"
#include "UnsyncMiniCb.h"
#include "UnsyncPool.h"
#include "UnsyncSerialization.h"
#include "UnsyncThread.h"
#include "UnsyncScheduler.h"

#include <fmt/format.h>
#if __has_include(<fmt/xchar.h>)
#	include <fmt/xchar.h>
#endif

#include <deque>
#include <span>
#include <unordered_set>

namespace unsync {

// Workaround for Jupiter timeouts when dealing with large numbers of attachments
static constexpr uint64 JUPITER_MAX_ATTACHMENT_REFERENCES = 1000;

template<typename T>
static void
AppendHash(T& Output, const FHash160& Hash)
{
	enum { HashStrMaxLen = sizeof(Hash.Data) * 2 };

	char HashStr[HashStrMaxLen];

	uint64 HashStrLen = BytesToHexChars(HashStr, sizeof(HashStr), Hash.Data, sizeof(Hash.Data));

	Output.append(std::string_view(HashStr, HashStrLen));
}

static std::string
MakeJupiterBlobUrl(std::string_view JupiterNamespace, std::string_view BlobName)
{
	return fmt::format("/api/v1/blobs/{}/{}", JupiterNamespace, BlobName);
}

static std::string
MakeJupiterBlobUrl(std::string_view JupiterNamespace, const FHash160& Hash)
{
	enum { HashStrMaxLen = sizeof(Hash.Data) * 2 };

	fmt::basic_memory_buffer<char, HashStrMaxLen> HashStr;
	AppendHash(HashStr, Hash);

	return MakeJupiterBlobUrl(JupiterNamespace, ToStringView(HashStr));
}

TResult<>
JupiterCheckAccess(FHttpConnection& Connection, std::string_view JupiterNamespace, std::string_view HttpHeaders)
{
	FHash160	Hash = {};
	std::string Url	 = MakeJupiterBlobUrl(JupiterNamespace, Hash);

	FHttpResponse Response = HttpRequest(Connection, EHttpMethod::GET, ToStringView(Url), HttpHeaders);

	// We expect a 404 for a dummy blob. Auth issues will show up as 401.
	if (Response.Code != 404 && Response.Code != 200)
	{
		return HttpError(Response.Code);
	}

	Response = HttpRequest(Connection, EHttpMethod::GET, "/health/live", HttpHeaders);

	if (Response.Success())
	{
		return ResultOk();
	}
	else
	{
		return HttpError(Response.Code);
	}
}

TResult<>
jupiter_put_raw_blob(FHttpConnection& Connection,
					 std::string_view JupiterNamespace,
					 std::string_view HttpHeaders,
					 FBufferView	  Blob,
					 const FHash160&  Hash)
{
	std::string Url = MakeJupiterBlobUrl(JupiterNamespace, Hash);

	FHttpResponse Response =
		HttpRequest(Connection, EHttpMethod::PUT, ToStringView(Url), EHttpContentType::Application_OctetStream, Blob, HttpHeaders);

	if (Response.Success())
	{
		return ResultOk();
	}
	else
	{
		const char* StatusStr = HttpStatusToString(Response.Code);
		UNSYNC_WARNING(L"Received HTTP code %d - %hs", Response.Code, StatusStr);
		return HttpError(Response.Code);
	}
}

FHttpResponse
JupiterPutRef(FHttpConnection& Connection,
			  std::string_view JupiterNamespace,
			  std::string_view JupiterBucket,
			  std::string_view ObjectKey,
			  std::string_view HttpHeaders,
			  FBufferView	   ObjectCb)
{
	std::string Url = fmt::format("/api/v1/refs/{}/{}/{}.uecb", JupiterNamespace, JupiterBucket, ObjectKey);

	FHash160 ObjectHash = HashBlake3Bytes<FHash160>(ObjectCb.Data, ObjectCb.Size);

	std::string ObjectHashStr = HashToHexString(ObjectHash);
	std::string ExtraHeaders  = fmt::format("X-Jupiter-IoHash: {}\r\n{}", ObjectHashStr, HttpHeaders);

	FHttpResponse Response;

	Response = HttpRequest(Connection, EHttpMethod::PUT, Url, EHttpContentType::Application_UECB, ObjectCb, ExtraHeaders);

	return Response;
}

TResult<std::span<const FHash160>>
DecodeJupiterNeedsCb(FBufferView Cb)
{
	FMiniCbReader RootReader = FMiniCbReader(Cb.Data, Cb.Size);

	FMiniCbFieldView ObjectView = RootReader.Child();
	if (ObjectView.Type != EMiniCbFieldType::Object)
	{
		return AppError(L"Jupiter response comact binary does not have a root object");
	}

	FMiniCbFieldView ElementView = ObjectView.Child();

	if ((ObjectView.Data + ObjectView.Size) > (Cb.Data + Cb.Size))
	{
		uint64 ObjectOffset = ObjectView.Data - Cb.Data;
		uint64 ActualSize	= Cb.Size - ObjectOffset;

		return AppError(fmt::format(L"Jupiter binary attachment array is {} bytes, but HTTP response only contained {} bytes",
									ObjectView.Size,
									ActualSize));
	}

	if (ElementView.Name == "needs" && ElementView.Type == EMiniCbFieldType::UniformArray &&
		ElementView.UniformArrayItemType == EMiniCbFieldType::BinaryAttachment)
	{
		UNSYNC_ASSERT(ElementView.UniformArrayItemCount * sizeof(FHash160) <= ElementView.Size);

		return ResultOk(ElementView.GetUniformArray<FHash160>());
	}
	else
	{
		return AppError(L"Jupiter response comact binary does not have a 'needs' array of binary attachment references");
	}
}

struct FJupiterPutManifestResult
{
	std::unordered_set<FGenericHash> NeedBlocks;

	uint32 NumManifestParts = 0;
	uint32 NumUploadedParts = 0;

	uint64 TotalSourceSize = 0;
	uint64 TotalNeedSize   = 0;

	uint32 TotalNeedObjects = 0;

	bool Complete() const { return TotalNeedObjects == 0 && TotalNeedSize == 0 && NeedBlocks.empty(); }
};

// Upload manifest to Jupiter as N CompactBinary objects with binary attachment references.
TResult<FJupiterPutManifestResult>
JupiterPutManifest(const FDirectoryManifest&	 Manifest,
				   std::string_view				 JupiterNamespace,
				   std::string_view				 JupiterBucket,
				   std::string_view				 HttpHeaders,  // TODO: just store it in the HTTP connection
				   TObjectPool<FHttpConnection>& ConnectionPool,
				   bool							 bIncludeManifestFile)
{
	FBuffer	 SerializedManifestBuffer;
	FHash160 SerializedManifestHash = {};

	FHash160 ManifestSignature = ToHash160(ComputeManifestStableSignature(Manifest));

	if (bIncludeManifestFile)
	{
		FVectorStreamOut ManifestStream(SerializedManifestBuffer);
		bool			 bSerializedOk = SaveDirectoryManifest(Manifest, ManifestStream);
		if (!bSerializedOk)
		{
			return AppError(L"Failed to serialize manifest");
		}
		SerializedManifestHash = HashBlake3Bytes<FHash160>(SerializedManifestBuffer.Data(), SerializedManifestBuffer.Size());
	}

	// Upload macro-blocks referenced by the manifest

	FJupiterPutManifestResult Result;
	std::mutex				  ResultMutex;

	uint64											TotalSourceMacroBlocks = 0;
	uint64											UniqueSourceSize	   = 0;
	std::unordered_map<FGenericHash, FGenericBlock> UniqueBlocks;
	std::vector<FMiniCbWriter>						ChunkedRefManifestCb;

	{
		FMiniCbWriter CurrentManifestPart;

		for (const auto& It : Manifest.Files)
		{
			const FFileManifest& FileManifest = It.second;
			for (const FGenericBlock& Block : FileManifest.MacroBlocks)
			{
				auto InsertResult = UniqueBlocks.insert(std::pair(Block.HashStrong, Block));
				if (InsertResult.second)
				{
					UniqueSourceSize += Block.Size;

					CurrentManifestPart.AddBinaryAttachment(Block.HashStrong.ToHash160());

					if (CurrentManifestPart.GetNumAttachments() == JUPITER_MAX_ATTACHMENT_REFERENCES)
					{
						CurrentManifestPart.Finalize();

						ChunkedRefManifestCb.push_back(std::move(CurrentManifestPart));
						CurrentManifestPart = FMiniCbWriter();
					}
				}
				Result.TotalSourceSize += Block.Size;
			}
			TotalSourceMacroBlocks += FileManifest.MacroBlocks.size();
		}

		if (CurrentManifestPart.GetNumAttachments())
		{
			CurrentManifestPart.Finalize();
			ChunkedRefManifestCb.push_back(std::move(CurrentManifestPart));
		}
	}

	std::vector<FHash160> ManifestPartHashes;
	ManifestPartHashes.reserve(ChunkedRefManifestCb.size());
	for (const FMiniCbWriter& Part : ChunkedRefManifestCb)
	{
		FHash160 PartHash = HashBlake3Bytes<FHash160>(Part.Data(), Part.Size());
		ManifestPartHashes.push_back(PartHash);
	}

	Result.NumManifestParts = uint32(ChunkedRefManifestCb.size());

	UNSYNC_VERBOSE(L"Total macro blocks: %llu (%llu unique)", llu(TotalSourceMacroBlocks), llu(UniqueBlocks.size()));
	UNSYNC_VERBOSE(L"Total size: %.2f MB (%.2f MB unique)", SizeMb(Result.TotalSourceSize), SizeMb(UniqueSourceSize));
	UNSYNC_VERBOSE(L"Manifest parts: %llu", llu(ChunkedRefManifestCb.size()));

	if (bIncludeManifestFile)
	{
		UNSYNC_VERBOSE(L"Uploading serialized manifest");

		FMiniCbWriter ManifestFileCb;
		ManifestFileCb.AddBinaryAttachment(SerializedManifestHash, "unsync.manifest");
		ManifestFileCb.AddHashArray(ManifestPartHashes, "unsync.manifest.parts");
		ManifestFileCb.Finalize();

		std::unique_ptr<FHttpConnection> Connection = ConnectionPool.Acquire();

		std::string ManifestKey = HashToHexString(ManifestSignature);

		const uint32 MaxAttempts = 5;
		for (uint32 AttemptIndex = 0; AttemptIndex <= MaxAttempts; ++AttemptIndex)
		{
			if (AttemptIndex == MaxAttempts)
			{
				return AppError(fmt::format(L"Failed to upload serialized manifest to Jupiter after {} attempts", MaxAttempts));
			}

			FHttpResponse PutResult =
				JupiterPutRef(*Connection, JupiterNamespace, JupiterBucket, ManifestKey, HttpHeaders, ManifestFileCb.GetBufferView());

			bool bShouldUploadManifestFile = false;

			if (PutResult.Success())
			{
				auto DecodedNeeds = DecodeJupiterNeedsCb(PutResult.Buffer);
				if (const std::span<const FHash160>* Needs = DecodedNeeds.TryData())
				{
					for (const FHash160& Hash : *Needs)
					{
						if (SerializedManifestHash == Hash)
						{
							bShouldUploadManifestFile = true;
						}
					}
				}
			}

			if (bShouldUploadManifestFile)
			{
				++Result.TotalNeedObjects;

				if (jupiter_put_raw_blob(*Connection,
										 JupiterNamespace,
										 HttpHeaders,  // TODO: just store these on the connection
										 SerializedManifestBuffer.View(),
										 SerializedManifestHash)
						.IsError())
				{
					continue;
				}
			}
			else
			{
				break;
			}
		}

		ConnectionPool.Release(std::move(Connection));
	}

	FTaskGroup ManifestUploadTasks = GScheduler->CreateTaskGroup();

	struct ManifestPutTask
	{
		TResult<> Result = ResultOk();
	};
	std::vector<ManifestPutTask> TaskContexts(ChunkedRefManifestCb.size());

	for (uint64 ManifestChunkIndex = 0; ManifestChunkIndex < ChunkedRefManifestCb.size(); ++ManifestChunkIndex)
	{
		GScheduler->NetworkSemaphore.Acquire();	 // must be acquired before task is spawned

		ManifestPutTask&	 Context	   = TaskContexts[ManifestChunkIndex];
		const FMiniCbWriter& RefManifestCb = ChunkedRefManifestCb[ManifestChunkIndex];

		UNSYNC_VERBOSE(L"Uploading part %llu of %llu (attachments: %llu)",
					   llu(ManifestChunkIndex + 1),
					   llu(ChunkedRefManifestCb.size()),
					   llu(RefManifestCb.GetNumAttachments()));

		UNSYNC_LOG_INDENT;

		auto PutTask = [JupiterNamespace, JupiterBucket, HttpHeaders, &Context, &RefManifestCb, &ConnectionPool, &Result, &ResultMutex]()
		{
			std::unique_ptr<FHttpConnection> Connection = ConnectionPool.Acquire();

			FHash160	RefManifestHash	 = HashBlake3Bytes<FHash160>(RefManifestCb.Data(), RefManifestCb.Size());
			std::string ManifestChunkKey = HashToHexString(RefManifestHash);

			// TODO: start the blob upload tasks in parallel, right after getting the `needs` list
			FHttpResponse Response;

			// TODO: add a helper function to retry an operation N times
			uint32 MaxRetryAttempts = 5;
			for (uint32 AttemptIndex = 0; AttemptIndex <= MaxRetryAttempts; ++AttemptIndex)
			{
				if (AttemptIndex != 0)
				{
					UNSYNC_VERBOSE(L"Retry attempt %d of %d", AttemptIndex, MaxRetryAttempts);
					SchedulerSleep(1000);
				}

				Response = JupiterPutRef(*Connection,
										 JupiterNamespace,
										 JupiterBucket,
										 ManifestChunkKey,
										 HttpHeaders,
										 RefManifestCb.GetBufferView());

				if (Response.Success() || Response.Code == 401 || Response.Code == 403)
				{
					break;
				}
			}

			ConnectionPool.Release(std::move(Connection));
			GScheduler->NetworkSemaphore.Release();

			if (Response.Success())
			{
				auto DecodedNeeds = DecodeJupiterNeedsCb(Response.Buffer);
				if (const std::span<const FHash160>* Needs = DecodedNeeds.TryData())
				{
					std::lock_guard<std::mutex> LockGuard(ResultMutex);
					for (const FHash160& Hash : *Needs)
					{
						Result.NeedBlocks.insert(FGenericHash::FromBlake3_160(Hash));
						++Result.TotalNeedObjects;
					}
				}
				else
				{
					Context.Result = FError{.Code = -1};
					return;
				}
			}
			else if (Response.Code == 0)
			{
				UNSYNC_ERROR(L"Failed to connect to Jupiter");
				Context.Result = FError{.Code = -1};
				return;
			}
			else
			{
				std::lock_guard<std::mutex> LockGuard(ResultMutex);
				UNSYNC_WARNING(L"Jupiter returned code %d: %hs", Response.Code, HttpStatusToString(Response.Code));
				Context.Result = FError{.Code = int32(Response.Code), .Kind = EErrorKind::Http};
				return;
			}

			Context.Result = ResultOk();
		};
		ManifestUploadTasks.run(PutTask);
	}

	ManifestUploadTasks.wait();

	for (ManifestPutTask& Task : TaskContexts)
	{
		if (Task.Result.IsOk())
		{
			Result.NumUploadedParts++;
		}
	}

	for (const FGenericHash& Hash : Result.NeedBlocks)
	{
		auto BlockIt = UniqueBlocks.find(Hash);
		if (BlockIt == UniqueBlocks.end())
		{
			return AppError(L"Jupiter requested an unexpected block (not present in manifest)");
		}
		else
		{
			Result.TotalNeedSize += BlockIt->second.Size;
		}
	}

	if (Result.NumUploadedParts == 0)
	{
		return AppError(L"All Jupiter manifest upload tasks failed");
	}
	else if (Result.NumUploadedParts != Result.NumManifestParts)
	{
		UNSYNC_WARNING(L"Encountered errors while uploading manifest to Jupiter");
		UNSYNC_WARNING(L"Uploaded %d of %d parts", Result.NumUploadedParts, Result.NumManifestParts);
	}

	return ResultOk(Result);
}

TResult<uint64>
JupiterPush(const FDirectoryManifest& Manifest, const FRemoteDesc& RemoteDesc, const FTlsClientSettings& TlsSettings)
{
	auto CreateConnection = [RemoteDesc, TlsSettings]
	{ return new FHttpConnection(RemoteDesc.Host.Address, RemoteDesc.Host.Port, RemoteDesc.TlsRequirement, TlsSettings, RemoteDesc.RecvTimeoutSeconds); };

	TObjectPool<FHttpConnection> ConnectionPool(CreateConnection);

	{
		UNSYNC_VERBOSE(L"Connecting to Jupiter server '%hs:%d'", RemoteDesc.Host.Address.c_str(), RemoteDesc.Host.Port);

		// TODO: add an RAII helper to return connections to the pool
		std::unique_ptr<FHttpConnection> Connection = ConnectionPool.Acquire();

		auto JupiterAccessible = JupiterCheckAccess(*Connection, RemoteDesc.StorageNamespace, RemoteDesc.HttpHeaders);
		if (!JupiterAccessible.IsOk())
		{
			int32		ErrorCode = JupiterAccessible.GetError().Code;
			const char* StatusStr = HttpStatusToString(ErrorCode);
			UNSYNC_ERROR(L"Jupiter server is not accessible. Error code %d: %hs", ErrorCode, StatusStr);
			return HttpError(ErrorCode);
		}

		ConnectionPool.Release(std::move(Connection));
	}

	auto JupiterManifestUploadResult = [&RemoteDesc, &Manifest, &ConnectionPool]
	{
		UNSYNC_VERBOSE(L"Sending block references to Jupiter");
		UNSYNC_LOG_INDENT;
		const bool bIncludeManifestFile = true;
		return JupiterPutManifest(Manifest,
								  RemoteDesc.StorageNamespace,
								  RemoteDesc.StorageBucket,
								  RemoteDesc.HttpHeaders,
								  ConnectionPool,
								  bIncludeManifestFile);
	}();

	if (FError* E = JupiterManifestUploadResult.TryError())
	{
		return std::move(*E);
	}

	const std::unordered_set<FGenericHash>& JupiterNeedsBlocks = JupiterManifestUploadResult.GetData().NeedBlocks;

	const uint64 TotalSourceSize = JupiterManifestUploadResult.GetData().TotalSourceSize;
	const uint64 TotalNeedSize	 = JupiterManifestUploadResult.GetData().TotalNeedSize;

	if (JupiterManifestUploadResult->NeedBlocks.empty())
	{
		UNSYNC_VERBOSE(L"No blocks need to be uploaded to Jupiter");
		return ResultOk(uint64(0));
	}

	UNSYNC_VERBOSE(L"Jupiter needs blocks: %lld (%.2f MB)", JupiterNeedsBlocks.size(), SizeMb(TotalNeedSize));
	UNSYNC_VERBOSE(L"Total source size: %lld bytes (%.2f MB)", TotalSourceSize, SizeMb(TotalSourceSize));

	FTimePoint NextProgressLogTime = TimePointNow();

	std::atomic<int32>	NumUploadedBlocks = {};
	std::atomic<uint64> ProcessedBytes	  = {};
	FTaskGroup			UploadTasks		  = GScheduler->CreateTaskGroup();
	std::atomic<bool>	bGotError		  = false;

	for (const auto& It : Manifest.Files)
	{
		if (bGotError)
		{
			break;
		}

		const FFileManifest& FileManifest = It.second;

		FNativeFile File(FileManifest.CurrentPath, EFileMode::ReadOnlyUnbuffered);
		if (!File.IsValid())
		{
			UNSYNC_ERROR(L"Failed to open input file '%ls'", FileManifest.CurrentPath.wstring().c_str());
			return FError{.Code = -1};
		}

		std::unique_ptr<FAsyncReader> AsyncReader = File.CreateAsyncReader();

		for (const FGenericBlock& Block : FileManifest.MacroBlocks)
		{
			if (bGotError)
			{
				break;
			}

			if (JupiterNeedsBlocks.find(Block.HashStrong) == JupiterNeedsBlocks.end())
			{
				continue;
			}

			auto ReadCallback =
				[Block, &UploadTasks, &ProcessedBytes, &ConnectionPool, &bGotError, &RemoteDesc, &NumUploadedBlocks](FIOBuffer ReadBuffer,
																													 uint64	   ReadOffset,
																													 uint64	   ReadReadSize,
																													 uint64	   ReadUserData)
			{
				// TODO: compress blocks usign Oodle
				// Buffer compressed_buffer = compress(read_buffer.GetData(), read_buffer.GetSize());
				// jupiter_put_compressed_blob(connection, compressed_blob_base_url, compressed_buffer, block.HashStrong);

				GScheduler->NetworkSemaphore.Acquire();	 // must be acquired before task is spawned
				auto Task = [Block,
							 &ProcessedBytes,
							 &ConnectionPool,
							 &bGotError,
							 &NumUploadedBlocks,
							 &RemoteDesc,
							 ReadBuffer = MakeShared(std::move(ReadBuffer))]
				{
					std::unique_ptr<FHttpConnection> Connection = ConnectionPool.Acquire();

					bool		 bPutSucceeded = false;
					const uint32 MaxAttempts   = 5;
					for (uint32 AttemptIndex = 0; AttemptIndex <= MaxAttempts && !bGotError; ++AttemptIndex)
					{
						if (AttemptIndex != 0)
						{
							UNSYNC_WARNING(L"Retry attempt %d of %d", AttemptIndex, MaxAttempts);
							SchedulerSleep(1000);
						}

						// TODO: pipeline PUT commands
						UNSYNC_ASSERT(Block.HashStrong.Type == MACRO_BLOCK_HASH_TYPE);
						if (jupiter_put_raw_blob(*Connection,
												 RemoteDesc.StorageNamespace,
												 RemoteDesc.HttpHeaders,
												 ReadBuffer->GetBufferView(),
												 Block.HashStrong.ToHash160())
								.IsOk())
						{
							bPutSucceeded = true;
							break;
						}
					}

					ConnectionPool.Release(std::move(Connection));
					GScheduler->NetworkSemaphore.Release();

						if (bPutSucceeded)
						{
							NumUploadedBlocks++;
							ProcessedBytes += ReadBuffer->GetSize();
						}
						else
						{
							bGotError = true;
						}
					};

					UploadTasks.run(Task);
				};

			AsyncReader->EnqueueRead(Block.Offset, Block.Size, 0, ReadCallback);

			if (TimePointNow() > NextProgressLogTime && ProcessedBytes > 0)
			{
				LogPrintf(ELogLevel::Debug,
						  L"\r%.2f / %.2f MB",
						  double(ProcessedBytes) / double(1_MB),
						  double(TotalNeedSize) / double(1_MB));

				NextProgressLogTime = TimePointNow() + std::chrono::milliseconds(500);
			}
		}

		AsyncReader->Flush();

		UploadTasks.wait();
	}

	uint64 ResultUploadedBlocks = NumUploadedBlocks;

	LogPrintf(ELogLevel::Debug, L"\r%.2f / %.2f MB\n", double(ProcessedBytes) / double(1_MB), double(TotalNeedSize) / double(1_MB));

	if (bGotError)
	{
		LogPrintf(ELogLevel::Error, L"Jupiter push finished with errors\n");
		return FError{.Code = -1};
	}
	else
	{
		LogPrintf(ELogLevel::Debug, L"Jupiter push succeeded (uploaded blocks: %d)\n", ResultUploadedBlocks);
		return ResultOk(ResultUploadedBlocks);
	}
}

bool
JupiterGetRawBlobBegin(FHttpConnection& Connection, std::string_view JupiterNamespace, std::string_view HttpHeaders, FHash160 Hash)
{
	std::string Url = MakeJupiterBlobUrl(JupiterNamespace, Hash);

	FHttpRequest Request;

	Request.Method		  = EHttpMethod::GET;
	Request.Url			  = ToStringView(Url);
	Request.CustomHeaders = HttpHeaders;

	return HttpRequestBegin(Connection, Request);
}

TResult<FBuffer>
JupiterGetRawBlobEnd(FHttpConnection& Connection)
{
	FHttpResponse Response = HttpRequestEnd(Connection);

	if (Response.Success())
	{
		return ResultOk(std::move(Response.Buffer));
	}
	else
	{
		return FError{.Code = Response.Code, .Kind = EErrorKind::Http};
	}
}

TResult<FBuffer>
JupiterGetRawBlob(FHttpConnection& Connection, std::string_view JupiterNamespace, std::string_view HttpHeaders, FHash160 Hash)
{
	if (JupiterGetRawBlobBegin(Connection, JupiterNamespace, HttpHeaders, Hash))
	{
		return JupiterGetRawBlobEnd(Connection);
	}
	else
	{
		return AppError(L"Failed to begin jupiter raw blob download");
	}
}

FJupiterProtocolImpl::FJupiterProtocolImpl(const FRemoteDesc&		 InSettings,
										   const FBlockRequestMap*	 InRequestMap,
										   std::string_view			 InHttpHeaders)
: FRemoteProtocolBase(InSettings, InRequestMap)
, Connection(InSettings.Host.Address, InSettings.Host.Port, InSettings.TlsRequirement, InSettings.GetTlsClientSettings(), InSettings.RecvTimeoutSeconds)
, HttpHeaders(InHttpHeaders)
, RemoteDesc(InSettings)
{
	bConnected = Connection.Open();

	if (bConnected)
	{
		auto JupiterAccessible = JupiterCheckAccess(Connection, InSettings.StorageNamespace, InSettings.HttpHeaders);
		if (!JupiterAccessible.IsOk())
		{
			int32		ErrorCode = JupiterAccessible.GetError().Code;
			const char* StatusStr = HttpStatusToString(ErrorCode);
			UNSYNC_ERROR(L"Jupiter server is not accessible. Error code %d: %hs", ErrorCode, StatusStr);
			Invalidate();
		}
	}
}

bool
FJupiterProtocolImpl::IsValid() const
{
	return bConnected;
}

TResult<FDirectoryManifest>
FJupiterProtocolImpl::DownloadManifest(std::string_view ManifestName)
{
	// Get root manifest reference object
	std::string RefUrl = fmt::format("/api/v1/refs/{}/{}/{}.uecb", RemoteDesc.StorageNamespace, RemoteDesc.StorageBucket, ManifestName);

	FHttpResponse Response = HttpRequest(Connection,
										 EHttpMethod::GET,
										 ToStringView(RefUrl),
										 EHttpContentType::Application_UECB,
										 FBufferView{},
										 RemoteDesc.HttpHeaders);

	if (!Response.Success())
	{
		return HttpError(L"Failed to download directory manifest from Jupiter.", Response.Code);
	}

	const FHash160		  InvalidHash	   = {};
	FHash160			  ManifestBlobHash = InvalidHash;
	std::vector<FHash160> ManifestReferenceParts;

	{
		FMiniCbReader	 RootReader = FMiniCbReader(Response.Buffer.Data(), Response.Buffer.Size());
		FMiniCbFieldView ObjectView = RootReader.Child();
		FMiniCbReader	 FieldReader(ObjectView);
		for (auto Field = FieldReader.Child(); Field.IsValid(); Field = FieldReader.Child())
		{
			if (Field.Type == EMiniCbFieldType::BinaryAttachment && Field.Name == "unsync.manifest")
			{
				ManifestBlobHash = Field.GetValue<FHash160>();
			}
			else if (Field.Type == EMiniCbFieldType::Hash && Field.Name == "unsync.manifest.part")
			{
				ManifestReferenceParts.push_back(Field.GetValue<FHash160>());
			}
			else if (Field.Type == EMiniCbFieldType::UniformArray && Field.UniformArrayItemType == EMiniCbFieldType::Hash &&
					 Field.Name == "unsync.manifest.parts")
			{
				for (const FHash160& Hash : Field.GetUniformArray<FHash160>())
				{
					ManifestReferenceParts.push_back(Hash);
				}
			}
		}
	}

	if (ManifestBlobHash == InvalidHash)
	{
		return AppError(L"Manifest root reference object is expected to contain a binary attachment object for serialized unsync manifest");
	}

	// Verify that all additional manifest part refererence objects exist and their bump last access timestamps

	for (const FHash160& PartHash : ManifestReferenceParts)
	{
		// TODO: use pipelined requests
		// TODO: verify that manifest parts contain the expected macro block references

		std::string ObjectKey = HashToHexString(PartHash);
		std::string Url		  = fmt::format("/api/v1/refs/{}/{}/{}.uecb", RemoteDesc.StorageNamespace, RemoteDesc.StorageBucket, ObjectKey);

		Response =
			HttpRequest(Connection, EHttpMethod::GET, Url, EHttpContentType::Application_UECB, FBufferView{}, RemoteDesc.HttpHeaders);

		if (!Response.Success())
		{
			return HttpError(L"Failed to download manifest part from Jupiter.", Response.Code);
		}
	}

	// Download unsync manifest from blob store

	auto ManifestBlobUrl = MakeJupiterBlobUrl(RemoteDesc.StorageNamespace, ManifestBlobHash);

	Response = HttpRequest(Connection,
						   EHttpMethod::GET,
						   ToStringView(ManifestBlobUrl),
						   EHttpContentType::Application_UECB,
						   FBufferView{},
						   RemoteDesc.HttpHeaders);

	if (!Response.Success())
	{
		// TODO: add a helper to quickly map HttpResponse to a TResult
		return HttpError(L"Failed to download directory manifest from Jupiter.", Response.Code);
	}

	FDirectoryManifest Manifest;

	FMemReader		   Reader(Response.Buffer);
	FIOReaderStream	   Stream(Reader);
	FPath			   EmptyRoot;  // Don't have a sensible path when not using file system as source
	bool			   bSourceManifestOk = LoadDirectoryManifest(Manifest, EmptyRoot, Stream);

	if (!bSourceManifestOk)
	{
		return AppError(L"Failed to deserialize directory manifest from Jupiter.");
	}

	return ResultOk(std::move(Manifest));
}

FDownloadResult
FJupiterProtocolImpl::Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback)
{
	if (!IsValid())
	{
		return FDownloadError(EDownloadRetryMode::Disconnect);
	}

	if (NeedBlocks.Size() == 0)
	{
		return ResultOk<FDownloadError>();
	}

	UNSYNC_ASSERT(Connection.NumActiveRequests == 0);

	uint64 NumNeededBlocks = NeedBlocks.Size();

	struct TempNeedBlock : FNeedBlock
	{
		FGenericHash MacroBlockHash		  = {};
		uint64		 MacroBlockOffset	  = 0;
		uint64		 MacroBlockBaseOffset = 0;
		uint64		 MacroBlockTotalSize  = 0;
	};

	std::vector<TempNeedBlock> SortedBlocks;
	SortedBlocks.reserve(NumNeededBlocks);
	for (const FNeedBlock& Block : NeedBlocks)
	{
		TempNeedBlock TempBlock;
		static_cast<FNeedBlock&>(TempBlock) = Block;

		FMacroBlockRequest MacroBlockRequest = RequestMap->GetMacroBlockRequest(Block.Hash);

		if (MacroBlockRequest.IsValid())
		{
			TempBlock.MacroBlockHash	   = MacroBlockRequest.Hash;
			TempBlock.MacroBlockOffset	   = MacroBlockRequest.Offset;
			TempBlock.MacroBlockBaseOffset = MacroBlockRequest.MacroBlockBaseOffset;
			TempBlock.MacroBlockTotalSize  = MacroBlockRequest.MacroBlockTotalSize;
		}
		else
		{
			UNSYNC_FATAL(L"Failed to find a macro block request");
		}

		SortedBlocks.push_back(TempBlock);
	}

	std::sort(SortedBlocks.begin(), SortedBlocks.end(), [](const TempNeedBlock& A, const TempNeedBlock& B) {
		if (A.MacroBlockHash == B.MacroBlockHash)
		{
			return A.MacroBlockOffset < B.MacroBlockOffset;
		}
		else
		{
			return memcmp(A.MacroBlockHash.Data, B.MacroBlockHash.Data, A.MacroBlockHash.Size()) < 0;
		}
	});

	struct MacroBlockBatch
	{
		uint64 FirstBlock		= 0;
		uint64 NumBlocks		= 0;
		uint64 MacroBlockOffset = 0;
		uint64 SizeBytes		= 0;
	};

	std::vector<MacroBlockBatch> MacroBlockBatches;
	MacroBlockBatches.reserve(SortedBlocks.size());

	for (uint64 I = 0; I < SortedBlocks.size(); ++I)
	{
		const TempNeedBlock& Block = SortedBlocks[I];

		auto IsCompatible = [](const TempNeedBlock& A, const TempNeedBlock& B) {
			return A.MacroBlockHash == B.MacroBlockHash && A.MacroBlockOffset + A.Size == B.MacroBlockOffset;
		};

		if (I == 0 || !IsCompatible(SortedBlocks[I - 1], Block))
		{
			MacroBlockBatch Batch;
			Batch.FirstBlock	   = I;
			Batch.NumBlocks		   = 0;
			Batch.MacroBlockOffset = 0;
			Batch.SizeBytes		   = 0;

			MacroBlockBatches.push_back(Batch);
		}

		MacroBlockBatches.back().NumBlocks += 1;
		MacroBlockBatches.back().SizeBytes += Block.Size;
	}

	// Pre-validate all request batches before doing any actual work
	for (const MacroBlockBatch& MacroBatch : MacroBlockBatches)
	{
		const TempNeedBlock* BatchBegin = &SortedBlocks[MacroBatch.FirstBlock];
		for (uint64 I = 0; I < MacroBatch.NumBlocks; ++I)
		{
			const TempNeedBlock& Block = BatchBegin[I];
			UNSYNC_ASSERT(Block.MacroBlockHash == BatchBegin->MacroBlockHash);

			uint64 OffsetFromFirst = Block.MacroBlockOffset - BatchBegin->MacroBlockOffset;
			UNSYNC_ASSERT(OffsetFromFirst < MacroBatch.SizeBytes);
			UNSYNC_ASSERT(OffsetFromFirst + Block.Size <= MacroBatch.SizeBytes);
		}
	}

	auto CompleteBatch = [&SortedBlocks, &CompletionCallback](const MacroBlockBatch& MacroBatch, const FBuffer* Buffer) {
		const TempNeedBlock* BatchBegin = &SortedBlocks[MacroBatch.FirstBlock];

		for (uint64 I = 0; I < MacroBatch.NumBlocks; ++I)
		{
			const TempNeedBlock& Block = BatchBegin[I];

			UNSYNC_ASSERT(Block.MacroBlockOffset >= BatchBegin->MacroBlockOffset);
			uint64 OffsetFromFirst = Block.MacroBlockOffset - BatchBegin->MacroBlockOffset;

			UNSYNC_ASSERT(OffsetFromFirst + Block.Size <= Buffer->Size());

			FDownloadedBlock ResultBlock;
			ResultBlock.bCompressed		 = false;
			ResultBlock.CompressedSize	 = Block.Size;
			ResultBlock.DecompressedSize = Block.Size;
			ResultBlock.Data			 = Buffer->Data() + OffsetFromFirst;

			CompletionCallback(ResultBlock, Block.Hash.ToHash128());
		}
	};

	UNSYNC_ASSERT(!MacroBlockBatches.empty());

	const MacroBlockBatch* PendingBatchIt		= &MacroBlockBatches[0];
	const MacroBlockBatch* MacroBlockBatchesEnd = &MacroBlockBatches.back() + 1;

	const uint64 MaxPipelinedBytes = 8_MB;
	uint64		 PipelinedBytes	   = 0;

	std::deque<const MacroBlockBatch*> DownloadQueue;

	while (PendingBatchIt != MacroBlockBatchesEnd || !DownloadQueue.empty())
	{
		if (PendingBatchIt != MacroBlockBatchesEnd)
		{
			const MacroBlockBatch& MacroBatch = *PendingBatchIt;
			const TempNeedBlock*   BatchBegin = &SortedBlocks[MacroBatch.FirstBlock];

			if (PipelinedBytes < MaxPipelinedBytes)
			{
				FMacroBlockRequest MacroBlockRequest;
				MacroBlockRequest.Hash	 = BatchBegin->MacroBlockHash;
				MacroBlockRequest.Offset = BatchBegin->MacroBlockOffset;
				MacroBlockRequest.Size	 = MacroBatch.SizeBytes;

				std::string RequestHeaders = fmt::format("Range: bytes={}-{}\r\n{}",
														 MacroBlockRequest.Offset,
														 MacroBlockRequest.Offset + MacroBlockRequest.Size - 1,
														 ToStringView(HttpHeaders));

				UNSYNC_ASSERT(MacroBlockRequest.Hash.Type == MACRO_BLOCK_HASH_TYPE);
				bool bRequestAdded = JupiterGetRawBlobBegin(Connection,
															RemoteDesc.StorageNamespace,
															ToStringView(RequestHeaders),
															MacroBlockRequest.Hash.ToHash160());

				if (bRequestAdded)
				{
					DownloadQueue.push_back(PendingBatchIt);
					PipelinedBytes += PendingBatchIt->SizeBytes;
				}

				PendingBatchIt++;

				if (bRequestAdded)
				{
					continue;
				}
			}
		}

		if (!DownloadQueue.empty())
		{
			const MacroBlockBatch* CompletedBatch = DownloadQueue.front();
			DownloadQueue.pop_front();

			PipelinedBytes -= CompletedBatch->SizeBytes;

			TResult<FBuffer> DownloadResult = JupiterGetRawBlobEnd(Connection);

			if (const FBuffer* BlockBuffer = DownloadResult.TryData())
			{
				UNSYNC_ASSERTF(BlockBuffer->Size() == CompletedBatch->SizeBytes,
							   L"Downloaded bytes: %llu, but expected: %llu",
							   llu(BlockBuffer->Size()),
							   llu(CompletedBatch->SizeBytes));

				ErrorsSinceLastSuccess = 0;

				CompleteBatch(*CompletedBatch, BlockBuffer);
			}
			else
			{
				++ErrorsSinceLastSuccess;
				++TotalErrors;

				const FError& Error = DownloadResult.GetError();
				if (Error.Kind == EErrorKind::Http)
				{
					const char* StatusStr = HttpStatusToString(Error.Code);
					UNSYNC_WARNING(L"Received HTTP code %d - %hs", Error.Code, StatusStr);

					if (Error.Code == 401				// If access is denied, further requests are pointless
						|| ErrorsSinceLastSuccess > 10	// Give up after some arbitrary excessive number of errors
					)
					{
						return FDownloadResult(EDownloadRetryMode::Abort);
					}
				}
			}
		}
	}

	UNSYNC_ASSERT(Connection.NumActiveRequests == 0);
	UNSYNC_ASSERT(Connection.ResponseQueue.empty());

	return ResultOk<FDownloadError>();
}

void
FJupiterProtocolImpl::Invalidate()
{
	bConnected = false;
	Connection.Close();
}

bool  // TODO: return a TResult
FJupiterProtocolImpl::Contains(const FDirectoryManifest& Manifest)
{
	// TODO: share a pool between all jupiter connections
	auto CloneConnection = [this] { return new FHttpConnection(this->Connection); };

	TObjectPool<FHttpConnection> ConnectionPool(CloneConnection);

	// POST api/v1/blobs/my-namespace/exists?id=<hash1>&id=<hash2>&id=...

	std::vector<std::string> RequestBatches;

	std::string RequestBaseUrl = std::string("/api/v1/blobs/") + RemoteDesc.StorageNamespace + "/exists?";

	const uint64 MaxRequestIds = 160;  // ~7KB of IDs per batch (max Jupiter request line is 8KB)
	uint64		 NumRequestIds = MaxRequestIds;

	for (const auto& It : Manifest.Files)
	{
		const FFileManifest& File = It.second;
		for (const FGenericBlock& MacroBlock : File.MacroBlocks)
		{
			if (NumRequestIds >= MaxRequestIds)
			{
				RequestBatches.push_back(RequestBaseUrl);
				NumRequestIds = 0;
			}

			std::string& RequestStr = RequestBatches.back();

			RequestStr += "id=";
			AppendHash(RequestStr, MacroBlock.HashStrong.ToHash160());
			RequestStr += "&";

			++NumRequestIds;
		}
	}

	UNSYNC_VERBOSE(L"Sending block IDs for validation ...")

	for (const std::string& RequestStr : RequestBatches)
	{
		FHttpRequest Request;
		Request.Method			  = EHttpMethod::POST;
		Request.Url				  = RequestStr;
		Request.CustomHeaders	  = RemoteDesc.HttpHeaders;
		Request.AcceptContentType = EHttpContentType::Application_UECB;

		bool bRequestAdded = HttpRequestBegin(Connection, Request);
		if (!bRequestAdded)
		{
			UNSYNC_BREAK_ON_ERROR;
			return false;
		}
	}

	uint64 NumMissingBlocks = 0;
	uint64 RequestIndex		= 0;

	for (const std::string& RequestStr : RequestBatches)
	{
		UNSYNC_UNUSED(RequestStr);

		LogPrintf(ELogLevel::Debug, L"Receiving validation results: %llu / %llu\r", llu(RequestIndex) + 1, llu(RequestBatches.size()));

		// TODO: batch requests
		FHttpResponse Response = HttpRequestEnd(Connection);

		if (!Response.Success())
		{
			LogError(HttpError(Response.Code));
			UNSYNC_BREAK_ON_ERROR;
			return false;
		}

		if (Response.ContentType != EHttpContentType::Application_UECB)
		{
			LogError(AppError(L"Expected to receive a CompactBinary object from Jupiter"));
			UNSYNC_BREAK_ON_ERROR;
			return false;
		}

		FMiniCbReader	 RootReader	 = FMiniCbReader(Response.Buffer.Data(), Response.Buffer.Size());
		FMiniCbFieldView ElementView = RootReader.Child().Child();

		if (ElementView.Name != "needs" || ElementView.Type != EMiniCbFieldType::UniformArray ||
			ElementView.UniformArrayItemType != EMiniCbFieldType::BinaryAttachment)
		{
			LogError(AppError(L"Expected to receive a CompactBinary object with a 'needs' uniform array of binary attachments"));
			UNSYNC_BREAK_ON_ERROR;
			return false;
		}

		if (ElementView.UniformArrayItemCount)
		{
			NumMissingBlocks += ElementView.UniformArrayItemCount;
		}

		RequestIndex++;
	}
	LogPrintf(ELogLevel::Debug, L"\n");

	if (NumMissingBlocks != 0)
	{
		UNSYNC_WARNING(L"Jupiter server is missing %llu block(s) referenced by the manifest", llu(NumMissingBlocks));
	}

	return NumMissingBlocks == 0;
}

}  // namespace unsync
