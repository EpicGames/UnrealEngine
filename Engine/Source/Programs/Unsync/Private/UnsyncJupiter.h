// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncBuffer.h"
#include "UnsyncCommon.h"
#include "UnsyncError.h"
#include "UnsyncHttp.h"
#include "UnsyncProxy.h"
#include "UnsyncRemote.h"
#include "UnsyncSocket.h"

#include <string>

namespace unsync {

struct FDirectoryManifest;
struct FTlsClientSettings;

// Returns number of blocks pushed
TResult<uint64> JupiterPush(const FDirectoryManifest& Manifest, const FRemoteDesc& RemoteDesc, const FTlsClientSettings& TlsSettings);

TResult<> JupiterPutRawBlob(FHttpConnection&	   Connection,
							const std::string_view BaseUrl,
							const std::string_view HttpHeaders,
							FBufferView			   Blob,
							const FHash160&		   Hash);

TResult<FBuffer> JupiterGetRawBlob(FHttpConnection& Connection,
								   const std::string_view,
								   const std::string_view HttpHeaders,
								   const FHash160&		  Hash);

TResult<> JupiterCheckAccess(FHttpConnection& Connection, std::string_view JupiterNamespace, std::string_view HttpHeaders);

struct FJupiterProtocolImpl : FRemoteProtocolBase
{
	FJupiterProtocolImpl(const FRemoteDesc& InSettings, const FBlockRequestMap* InRequestMap, std::string_view HttpHeaders);
	virtual bool						IsValid() const override;
	virtual TResult<FDirectoryManifest> DownloadManifest(std::string_view ManifestName) override;
	virtual FDownloadResult Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback) override;
	virtual void			Invalidate() override;
	virtual bool			Contains(const FDirectoryManifest& Manifest) override;

	FHttpConnection Connection;
	std::string		HttpHeaders;  // TODO: store this in the HttpConnection instead
	FRemoteDesc		RemoteDesc;

	bool   bConnected			  = false;
	uint64 ErrorsSinceLastSuccess = 0;
	uint64 TotalErrors			  = 0;
	uint64 TotalRequests		  = 0;
};

}  // namespace unsync
