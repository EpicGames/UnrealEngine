// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncLog.h"
#include "UnsyncCore.h"
#include "UnsyncError.h"
#include "UnsyncFile.h"
#include "UnsyncUtil.h"
#include "UnsyncVersion.h"

#include <stdarg.h>
#include <stdio.h>
#include <stdlib.h>
#include <atomic>
#include <mutex>
#include <vector>
#include <chrono>

UNSYNC_THIRD_PARTY_INCLUDES_START
#include <fmt/format.h>
#include <fmt/chrono.h>
UNSYNC_THIRD_PARTY_INCLUDES_END

#if UNSYNC_PLATFORM_WINDOWS
#	include <Dbghelp.h>
#	include <Windows.h>
#	pragma comment(lib, "Dbghelp.lib")
#endif	// UNSYNC_PLATFORM_WINDOWS

namespace unsync {

static constexpr size_t TimestampStringSize = sizeof("YYYY-MM-DDTHH:MM:SS.sssZ");
using FTimestampString = fmt::basic_memory_buffer<char, TimestampStringSize>;
static FTimestampString FormatTimestamp(std::chrono::system_clock::time_point Timestamp);
static FTimestampString FormatWindowsFileTime(uint64 Ticks);

#if UNSYNC_PLATFORM_WINDOWS
bool
IsDebuggerPresent()
{
	return ::IsDebuggerPresent();
}
#else
bool
IsDebuggerPresent()
{
	return false;  // TODO
}
#endif

thread_local bool	GLogVerbose			= false;
bool				GLogVeryVerbose		= false;
bool				GLogSilent			= false;
bool				GBreakOnError		= IsDebuggerPresent();
bool				GBreakOnWarning		= false;
thread_local uint32 GLogIndent			= 0;
bool				GLogProgress		= false;
bool				GLogMachineReadable = false;
bool				GLogStdOut		= false;

std::mutex GLogMutex;

std::atomic<uint32> GLogThreadIndexCounter;
thread_local uint32 GLogThreadIndex = ~0u;

FTimePoint GNextFlushTime = TimePointNow();

std::vector<std::wstring> GCommandLine;

void
LogSaveCommandLineUtf8(int Argc, char** Argv)
{
	std::lock_guard<std::mutex> LockGuard(GLogMutex);
	for (int i = 0; i < Argc; ++i)
	{
		GCommandLine.push_back(ConvertUtf8ToWide(Argv[i]));
	}
}

static FILE*
GetLogStream(ELogLevel LogLevel)
{
	// When --stdout is set, all output goes to stdout (no stderr).
	if (GLogStdOut)
	{
		return stdout;
	}

	// If machine readable log format is enabled, *all* diagnostic logging is sent to stderr.
	// Otherwise, only errors and warnings are sent to stderr, while normal diagnostics are sent to stdout.

	if (GLogMachineReadable)
	{
		return (LogLevel == ELogLevel::MachineReadable) ? stdout : stderr;
	}
	else
	{
		return (LogLevel >= ELogLevel::Info) ? stdout : stderr;
	}
}

static void
LogConditionalFlush(FILE* Stream)
{
	FTimePoint NextFlushTime = GNextFlushTime;
	FTimePoint CurrentTime	 = TimePointNow();
	if (CurrentTime > NextFlushTime || GLogProgress)
	{
		GNextFlushTime = CurrentTime + std::chrono::milliseconds(1000);
		fflush(Stream);
	}
}

static void
LogConditionalFlush(ELogLevel LogLevel)
{
	LogConditionalFlush(GetLogStream(LogLevel));
}

static uint32
GetLogThreadIndex()
{
	if (GLogThreadIndex == ~0u)
	{
		GLogThreadIndex = GLogThreadIndexCounter++;
	}
	return GLogThreadIndex;
}

struct FLogFile
{
	FLogFile(const wchar_t* InFilename) : Filename(InFilename)
	{
#if UNSYNC_PLATFORM_WINDOWS
		Handle = _wfopen(InFilename, L"wt, ccs=UNICODE");
#else
		std::string FilenameUtf8 = ConvertWideToUtf8(std::wstring_view(InFilename));
		Handle					 = fopen(FilenameUtf8.c_str(), "wt");
#endif
	}

	~FLogFile()
	{
		if (Handle)
		{
			fclose(Handle);
		}
	}

	std::wstring Filename;
	FILE*		 Handle = nullptr;
};

std::unique_ptr<FLogFile> GLogFile;

static void
RotateLogs(const FPath& LogFilename)
{
	static const FPath ExpectedSuffix(".log");

	std::error_code ErrorCode;
	FFileAttributes ExistingAttrib = GetFileAttrib(LogFilename);

	if (!LogFilename.has_parent_path() || !ExistingAttrib.bValid)
	{
		return;
	}

	FTimestampString TimestampString	= FormatWindowsFileTime(ExistingAttrib.Mtime);
	std::string		 BackupExtensionStr = TimestampString.data();
	std::replace_if(
		BackupExtensionStr.begin(),
		BackupExtensionStr.end(),
		[](char C) { return C == ':' || C == '.'; },
		'-');
	BackupExtensionStr += ".log";

	FPath BackupExtension	= FPath(BackupExtensionStr);
	FPath BackupLogFilename = LogFilename;
	BackupLogFilename.replace_extension(BackupExtension);

	FileRename(LogFilename, BackupLogFilename, ErrorCode);

	FPath LogDirectory = LogFilename.parent_path();

	FPathFilterCallback Filter = [](const FPath& CandidatePath)
	{
		return CandidatePath.has_extension() && CandidatePath.native().ends_with(ExpectedSuffix.native());
	};

	DeleteOldFilesInDirectory(LogDirectory, 10 /*files to keep*/, true /*allow in dry run*/, Filter);
}

void
LogSetFileInternal(const wchar_t* Filename)
{
	GLogFile = {};

	if (Filename)
	{
		GLogFile = std::make_unique<FLogFile>(Filename);
	}
}

std::vector<std::unique_ptr<FLogFile>> GLogFileStack;

FLogFileScope::FLogFileScope(const wchar_t* Filename)
{
	std::wstring CommandLine;

	RotateLogs(FPath(Filename));

	{
		std::lock_guard<std::mutex> LockGuard(GLogMutex);
		GLogFileStack.push_back(std::move(GLogFile));
		LogSetFileInternal(Filename);

		for (const std::wstring& Arg : GCommandLine)
		{
			if (!CommandLine.empty())
			{
				CommandLine += ' ';
			}
			CommandLine += Arg;
		}
	}

	UNSYNC_VERBOSE2(L"UNSYNC v%hs started logging to file '%ls'", GetVersionString().c_str(), Filename);
	
	if (!CommandLine.empty())
	{
		UNSYNC_VERBOSE2(L"Command line: %ls", CommandLine.c_str());
	}
}

FLogFileScope::~FLogFileScope()
{
	UNSYNC_VERBOSE2(L"Finished logging to file");

	std::lock_guard<std::mutex> LockGuard(GLogMutex);
	std::swap(GLogFile, GLogFileStack.back());
	GLogFileStack.pop_back();
}

void
LogProgress(const wchar_t* ItemName, uint64 Current, uint64 Total)
{
	if (!GLogProgress)
	{
		return;
	}

	std::lock_guard<std::mutex> LockGuard(GLogMutex);
	if (ItemName == nullptr)
	{
		ItemName = L"*";
	}
	wprintf(L"@progress [%ls] %llu / %llu\n", ItemName, Current, Total);

	LogConditionalFlush(stdout); // progress is always reported to stdout
}

void
LogStatus(const wchar_t* InItemName, const wchar_t* Status)
{
	if (!GLogProgress)
	{
		return;
	}

	std::lock_guard<std::mutex> LockGuard(GLogMutex);

	const wchar_t* ItemName = InItemName ? InItemName : L"*";
	wprintf(L"@status [%ls] %ls\n", ItemName, Status);

	if (InItemName)
	{
		LogConditionalFlush(stdout); // status is always reported to stdout
	}
	else
	{
		fflush(stdout);
	}
}

void
LogFlush()
{
	std::lock_guard<std::mutex> LockGuard(GLogMutex);
	fflush(stdout);
	fflush(stderr);
	if (GLogFile && GLogFile->Handle)
	{
		FILE* F = GLogFile->Handle;
		fflush(F);
	}
}

static FTimestampString
FormatTimestamp(std::chrono::system_clock::time_point Timestamp)
{
	auto	TimestampMilliseconds = std::chrono::time_point_cast<std::chrono::milliseconds>(Timestamp);
	auto	TimestampSeconds	  = std::chrono::time_point_cast<std::chrono::seconds>(TimestampMilliseconds);
	auto	DeltaMilliseconds	  = (TimestampMilliseconds - TimestampSeconds).count();
	std::tm UtcTime				  = fmt::gmtime(std::chrono::system_clock::to_time_t(TimestampSeconds));

	FTimestampString Result;
	fmt::format_to(std::back_inserter(Result), "{:%FT%T}.{:03}Z", UtcTime, DeltaMilliseconds);
	Result.push_back(0);

	UNSYNC_ASSERT(Result.size() == TimestampStringSize);

	return Result;
}

static FTimestampString
FormatWindowsFileTime(uint64 Ticks)
{
	std::filesystem::file_time_type FileTime = FromWindowsFileTime(Ticks);
	return FormatTimestamp(SystemTimeFromFileTime(FileTime));
}

void
LogPrintf(ELogLevel Level, const wchar_t* Str, ...)
{
	std::lock_guard<std::mutex> LockGuard(GLogMutex);

	const auto Timestamp = std::chrono::system_clock::now();

	bool		   bShouldIndent = false;
	const wchar_t* Prefix		 = nullptr;

	const uint32 ThreadIndex = GetLogThreadIndex();

	bool bShouldOutputThreadIndex = false;

	if (Level == ELogLevel::Error)
	{
		Prefix = L"ERROR: ";
		bShouldOutputThreadIndex = ThreadIndex != 0;
	}
	else if (Level == ELogLevel::Warning)
	{
		Prefix = L"WARNING: ";
		bShouldOutputThreadIndex = ThreadIndex != 0;
	}
	else if (GLogIndent)
	{
		// Indent the log only for info/debug/trace levels
		bShouldIndent = true;
	}

	ELogLevel MaxDisplayLevel = ELogLevel::Info;

	if (GLogSilent)
	{
		MaxDisplayLevel = ELogLevel::Warning;
	}
	else if (GLogVerbose)
	{
		if (GLogVeryVerbose)
		{
			MaxDisplayLevel = ELogLevel::Trace;
		}
		else
		{
			MaxDisplayLevel = ELogLevel::Debug;
		}
	}

	FILE* LogStream = GetLogStream(Level);

	if (Level <= MaxDisplayLevel || Level == ELogLevel::MachineReadable)
	{
		if (Prefix)
		{
			fwprintf(LogStream, Prefix);
		}

		if (bShouldOutputThreadIndex)
		{
			fwprintf(LogStream, L"[Thread %u] ", ThreadIndex);
		}

		if (bShouldIndent)
		{
			fwprintf(LogStream, L"%*c", GLogIndent, L' ');
		}

		va_list Va;
		va_start(Va, Str);
		vfwprintf(LogStream, Str, Va);
		va_end(Va);

		LogConditionalFlush(Level);
	}

	if (GLogFile && GLogFile->Handle)
	{
		FILE* LogFileStream = GLogFile->Handle;

		FTimestampString TimestampString = FormatTimestamp(Timestamp);

		fwprintf(LogFileStream, L"[%hs] ", TimestampString.data());

		fwprintf(LogFileStream, L"[%3u] ", ThreadIndex);

		switch (Level)
		{
			case ELogLevel::Error:
				fwprintf(LogFileStream, L"[ERROR] ");
				break;
			case ELogLevel::Warning:
				fwprintf(LogFileStream, L"[WARN] ");
				break;
			case ELogLevel::Info:
				fwprintf(LogFileStream, L"[INFO] ");
				break;
			case ELogLevel::Debug:
				fwprintf(LogFileStream, L"[DEBUG] ");
				break;
			case ELogLevel::Trace:
				fwprintf(LogFileStream, L"[TRACE] ");
				break;
			default:
				break;
		}

		va_list Va;
		va_start(Va, Str);
		vfwprintf(LogFileStream, Str, Va);
		va_end(Va);

		if (Level == ELogLevel::Error || Level == ELogLevel::Warning)
		{
			fflush(LogFileStream);
		}
	}
}

extern const char* HttpStatusToString(int32 Code);
void
LogError(const FError& E, std::wstring ExtraContext)
{
	const char* ErrorKindStr = nullptr;
	const char* ErrorDescStr = nullptr;
	switch (E.Kind)
	{
		default:
		case EErrorKind::Unknown:
			ErrorKindStr = "Unknown";
			break;
		case EErrorKind::Http:
			ErrorDescStr = HttpStatusToString(E.Code);
			ErrorKindStr = "HTTP";
			break;
		case EErrorKind::System:
			ErrorKindStr = "System";
			break;
		case EErrorKind::App:
			ErrorKindStr = "Application";
			break;
	}

	const bool bHaveContext = !E.Context.empty();
	const bool bHaveExtraContext = !ExtraContext.empty();

	if (ErrorDescStr)
	{
		LogPrintf(ELogLevel::Error,
				  L"%ls%hs%hs code: %d (%hs).%ls%ls\n",
				  bHaveExtraContext ? ExtraContext.c_str() : L"",
				  bHaveExtraContext ? ": " : "",
				  ErrorKindStr,
				  E.Code,
				  ErrorDescStr,
				  bHaveContext ? L" Context: " : L"",
				  E.Context.empty() ? L"" : E.Context.c_str());
	}
	else
	{
		LogPrintf(ELogLevel::Error,
				  L"%ls%hs%hs code: %d.%ls%ls\n",
				  bHaveExtraContext ? ExtraContext.c_str() : L"",
				  bHaveExtraContext ? ": " : "",
				  ErrorKindStr,
				  E.Code,
				  bHaveContext ? L" Context: " : L"",
				  E.Context.empty() ? L"" : E.Context.c_str());
	}
}

static FPath GCrashDumpPath;
void
SetCrashDumpPath(const FPath& Path)
{
	std::lock_guard<std::mutex> LockGuard(GLogMutex);
	GCrashDumpPath = Path;
}

bool
get_crash_dump_path(FPath& OutPath)
{
	std::lock_guard<std::mutex> LockGuard(GLogMutex);
	if (GCrashDumpPath.empty())
	{
		return false;
	}
	else
	{
		OutPath = GCrashDumpPath;
		return true;
	}
}

bool
LogWriteCrashDump(void* InExceptionPointers)
{
#if UNSYNC_PLATFORM_WINDOWS

	_EXCEPTION_POINTERS* ExceptionPointers = (_EXCEPTION_POINTERS*)InExceptionPointers;

	FPath CrashDumpPath;
	if (!get_crash_dump_path(CrashDumpPath))
	{
		return false;
	}

	wchar_t DumpFilename[4096] = {};
	uint64	Timestamp		   = TimePointNow().time_since_epoch().count();
	swprintf_s(DumpFilename, L"unsync-%016llX.dmp", Timestamp);
	FPath CrashDumpFilename = CrashDumpPath / FPath(DumpFilename);

	LogPrintf(ELogLevel::Info, L"!!! Writing crash dump to '%ls'\n", CrashDumpFilename.wstring().c_str());

	MINIDUMP_EXCEPTION_INFORMATION ExceptionInfo;
	ExceptionInfo.ThreadId			= GetCurrentThreadId();
	ExceptionInfo.ExceptionPointers = ExceptionPointers;
	ExceptionInfo.ClientPointers	= true;

	HANDLE DumpFile =
		CreateFileW(CrashDumpFilename.c_str(), GENERIC_READ | GENERIC_WRITE, FILE_SHARE_WRITE | FILE_SHARE_READ, 0, CREATE_ALWAYS, 0, 0);
	if (DumpFile != INVALID_HANDLE_VALUE)
	{
		const BOOL Ok =
			MiniDumpWriteDump(GetCurrentProcess(), GetCurrentProcessId(), DumpFile, MiniDumpWithDataSegs, &ExceptionInfo, nullptr, nullptr);

		if (Ok)
		{
			LogPrintf(ELogLevel::Info, L"!!! Crash dump file written.\n");
			return true;
		}
		else
		{
			LogPrintf(ELogLevel::Error, L"!!! Failed to generate crash dump. %hs\n", FormatSystemErrorMessage(GetLastError()).c_str());
		}
	}
	else
	{
		LogPrintf(ELogLevel::Error, L"!!! Failed to open output file. %hs\n", FormatSystemErrorMessage(GetLastError()).c_str());
	}

	LogPrintf(ELogLevel::Error, L"!!! Failed to write dump file.\n");
#endif	// UNSYNC_PLATFORM_WINDOWS

	return false;
}

FLogFlushScope::~FLogFlushScope()
{
	LogFlush();
}

}  // namespace unsync
