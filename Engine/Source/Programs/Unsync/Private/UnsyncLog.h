// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"

namespace unsync {

extern thread_local bool   GLogVerbose;
extern bool				   GLogVeryVerbose;
extern bool				   GLogSilent;
extern bool				   GBreakOnError;
extern bool				   GBreakOnWarning;
extern thread_local uint32 GLogIndent;
extern bool				   GLogProgress;		 // Whether to output @progress and @status markers to stdout
extern bool				   GLogMachineReadable;	 // Whether output is intended for other programs rather than humans
extern bool				   GLogStdOut;           // Whether to send all output to stdout (disabling stderr)

struct FError;

struct FLogIndentScope
{
	uint32 Next = 0;
	uint32 Prev = 0;
	FLogIndentScope(uint32 InN = 2, bool bInOverride = false) : Next(InN), Prev(GLogIndent)
	{
		if (bInOverride)
		{
			GLogIndent = InN;
		}
		else
		{
			GLogIndent += Next;
		}
	}
	~FLogIndentScope() { GLogIndent = Prev; }
	FLogIndentScope(const FLogIndentScope&) = delete;
	FLogIndentScope& operator=(const FLogIndentScope&) = delete;
};

struct FLogVerbosityScope
{
	bool bNext = false;
	bool bPrev = false;
	FLogVerbosityScope(bool InNext) : bNext(InNext), bPrev(GLogVerbose) { GLogVerbose = bNext; }
	~FLogVerbosityScope() { GLogVerbose = bPrev; }
	FLogVerbosityScope(const FLogVerbosityScope&) = delete;
	FLogVerbosityScope& operator=(const FLogVerbosityScope&) = delete;
};

struct FLogFileScope
{
	FLogFileScope(const wchar_t* Filename);
	~FLogFileScope();
};

struct FLogFlushScope
{
	~FLogFlushScope();
};

enum class ELogLevel
{
	Error	= 0,
	Warning = 1,
	Info	= 2,
	Debug	= 3,
	Trace	= 4,

	// Special log level to signify output that's intended to be machine-readable.
	// This is always written to stdout, while other modes may be written to stderr in some cases.
	MachineReadable = 5,
};

void LogFlush();

void LogPrintf(ELogLevel Level, const wchar_t* Str, ...);
void LogError(const FError& E, std::wstring ExtraContext = {});

void LogProgress(const wchar_t* ItemName, uint64 Current, uint64 Total);
void LogStatus(const wchar_t* ItemName, const wchar_t* Status);

inline void
LogGlobalProgress(uint64 Current, uint64 Total)
{
	LogProgress(nullptr, Current, Total);
}
inline void
LogGlobalStatus(const wchar_t* Status)
{
	LogStatus(nullptr, Status);
}

void SetCrashDumpPath(const FPath& Path);
bool GetCrashDumpPath(FPath& OutPath);
bool LogWriteCrashDump(void* ExceptionPointers);

void LogSaveCommandLineUtf8(int Argc, char** Argv);

#define UNSYNC_LOG_INDENT		FLogIndentScope UNSYNC_CONCAT(log_indent_, __LINE__);
#define UNSYNC_LOG_INDENT_N(N)  FLogIndentScope UNSYNC_CONCAT(log_indent_, __LINE__)(N);

#define UNSYNC_LOG(text, ...)                                  \
	{                                                          \
		LogPrintf(ELogLevel::Info, text L"\n", ##__VA_ARGS__); \
	}
#define UNSYNC_WARNING(text, ...)                                 \
	{                                                             \
		LogPrintf(ELogLevel::Warning, text L"\n", ##__VA_ARGS__); \
		if (GBreakOnWarning)                                      \
			UNSYNC_BREAK();                                       \
	}
#define UNSYNC_ERROR(text, ...)                                 \
	{                                                           \
		LogPrintf(ELogLevel::Error, text L"\n", ##__VA_ARGS__); \
		if (GBreakOnError)                                      \
			UNSYNC_BREAK();                                     \
	}
#define UNSYNC_FATAL(text, ...)                                 \
	{                                                           \
		LogPrintf(ELogLevel::Error, text L"\n", ##__VA_ARGS__); \
		UNSYNC_BREAK();                                         \
	}
#define UNSYNC_VERBOSE(text, ...)                               \
	{                                                           \
		LogPrintf(ELogLevel::Debug, text L"\n", ##__VA_ARGS__); \
	}
#define UNSYNC_VERBOSE2(text, ...)                              \
	{                                                           \
		LogPrintf(ELogLevel::Trace, text L"\n", ##__VA_ARGS__); \
	}

#define UNSYNC_ASSERT(cond)                                                             \
	{                                                                                   \
		if (!(cond))                                                                    \
		{                                                                               \
			LogPrintf(ELogLevel::Error, L"Assertion failed: " UNSYNC_WSTR(cond) L"\n"); \
			UNSYNC_BREAK();                                                             \
		}                                                                               \
	}
#define UNSYNC_ASSERTF(cond, text, ...)                                                                           \
	{                                                                                                             \
		if (!(cond))                                                                                              \
		{                                                                                                         \
			LogPrintf(ELogLevel::Error, L"Assertion failed: " UNSYNC_WSTR(cond) L". " text L"\n", ##__VA_ARGS__); \
			UNSYNC_BREAK();                                                                                       \
		}                                                                                                         \
	}

#define UNSYNC_BREAK_ON_ERROR \
	{                         \
		if (GBreakOnError)    \
			UNSYNC_BREAK();   \
	}

#define UNSYNC_BREAK_ON_WARNING \
	{                           \
		if (GBreakOnWarning)    \
			UNSYNC_BREAK();     \
	}

#define UNSYNC_BREAK_ON_ERROR_LEVEL(LogLevel)              \
	{                                                      \
		if (LogLevel == ELogLevel::Error && GBreakOnError) \
			UNSYNC_BREAK();                                \
	}

#define UNSYNC_BREAK_ON_WARNING_LEVEL(LogLevel)                \
	{                                                          \
		if (LogLevel == ELogLevel::Warning && GBreakOnWarning) \
			UNSYNC_BREAK();                                    \
	}


}  // namespace unsync
