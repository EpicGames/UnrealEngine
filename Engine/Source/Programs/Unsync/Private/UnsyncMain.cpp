// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncAuth.h"
#include "UnsyncCmdDiff.h"
#include "UnsyncCmdHash.h"
#include "UnsyncCmdLogin.h"
#include "UnsyncCmdMount.h"
#include "UnsyncCmdPack.h"
#include "UnsyncCmdPatch.h"
#include "UnsyncCmdPush.h"
#include "UnsyncCmdQuery.h"
#include "UnsyncCmdSync.h"
#include "UnsyncCmdInfo.h"
#include "UnsyncCmdVerify.h"
#include "UnsyncCore.h"
#include "UnsyncFile.h"
#include "UnsyncMemory.h"
#include "UnsyncProxy.h"
#include "UnsyncTest.h"
#include "UnsyncThread.h"
#include "UnsyncUtil.h"
#include "UnsyncScheduler.h"
#include "UnsyncVersion.h"
#include "UnsyncSource.h"
#include "UnsyncFilter.h"
#include "UnsyncHorde.h"

UNSYNC_THIRD_PARTY_INCLUDES_START
#if UNSYNC_PLATFORM_WINDOWS
#	include <io.h>
#	include <shellapi.h>
#endif	// UNSYNC_PLATFORM_WINDOWS
#include <fcntl.h>
#include <CLI/CLI.hpp>
#include <filesystem>
#include <fstream>
#include <iostream>
#include <system_error>
UNSYNC_THIRD_PARTY_INCLUDES_END

namespace unsync {

static FPath GExePath;

int
InnerMain(int Argc, char** Argv)
{
	LogSaveCommandLineUtf8(Argc, Argv);

	std::string AppDescription = "UNSYNC v";
	AppDescription += GetVersionString();
	AppDescription +=
		" -- Differential binary synchronization tool.\n"
		"Copyright Epic Games, Inc. All Rights Reserved.\n";

	CLI::App Cli(AppDescription, "unsync");

	Cli.allow_windows_style_options(false); // never allow /flag syntax, only --flag
	Cli.set_version_flag("--version", GetVersionString());

	std::vector<CLI::App*> SubCommands;

	std::string				 InputFilenameUtf8;
	std::string				 OutputFilenameUtf8;
	std::string				 BaseFilenameUtf8;
	std::string				 SourceFilenameUtf8;
	std::string				 TargetFilenameUtf8;
	std::string				 PatchFilenameUtf8;
	std::string				 InputFilename2Utf8;
	std::string				 SourceManifestFilenameUtf8;
	std::vector<std::string> IncludeFilterArrayUtf8;
	std::vector<std::string> ExcludeFilterArrayUtf8;
	std::vector<std::string> CleanupExcludeFilterArrayUtf8;
	std::vector<std::string> OverlayArrayUtf8;
	std::string				 RemoteAddressUtf8;
	std::string				 PreferredDfsUtf8;
	std::string				 WeakHashUtf8	= "buzhash";
	std::string				 StrongHashUtf8 = "blake3.128";
	std::string				 PresetUtf8		= "all";
	std::string				 ChunkModeUtf8;
	std::string				 CacertFilenameUtf8;
	std::string				 ProtocolName;
	std::string				 HttpHeaderFilenameUtf8;
	std::string				 QueryStringUtf8;
	std::vector<std::string> QueryArgsUtf8;
	std::string				 ScavengeRootUtf8;
	std::string				 P4HavePathUtf8;
	std::string				 StorePathUtf8;
	std::string				 SnapshotNameUtf8;
	std::string				 AuthTokenPathUtf8;
	bool					 bRunP4Have			 = false;
	bool					 bForceOperation	 = false;
	bool					 bAllowInsecureTls	 = false;
	bool					 bRequireTls		 = false;
	bool					 bUseDebugMode		 = false;
	bool					 bIncrementalMode	 = false;
	bool					 bNoOutputValidation = false;
	bool					 bNoCleanupAfterSync = false;
	bool					 bNoSpaceValidation	 = false;
	bool					 bFullSourceScan	 = false;
	bool					 bFullDifference	 = false;
	bool					 bInfoFiles			 = false;
	bool					 bNoProxySelect		 = false;
	bool					 bInteractive		 = false;
	bool					 bDecode			 = false;
	bool					 bPrint				 = false;
	bool					 bPrintHttpHeader	 = false;
	bool					 bShouldLogin		 = false;
	bool					 bQuickLogin		 = false;
	bool					 bForceRefreshAuth	 = false;
	bool					 bNoSocketTimeout	 = false;
	bool					 bNoOutputFiles  	 = false;
	bool					 bNoOutputRevisions  = false;
	bool					 bPackOnlySmallFiles = false;
	bool					 bPackFiles		     = false;
	bool					 bNoCompression		 = false;
	int32					 CompressionLevel	 = 3;
	uint32					 DiffBlockSize		 = uint32(4_KB);
	uint32					 HashOrSyncBlockSize = uint32(64_KB);
	uint32					 BackgroundTaskMemoryBudgetGB = 2;

	const std::string HiddenGroupId;  // CLI11 uses an empty string group name to mark arguments that should be hidden
	const std::string ExperimentalGroupId = "Experimental";
	const std::string DangerousGroupId	  = "Dangerous";

	struct FDeprecatedOptions
	{
		bool bQuickSyncMode			= false;
		bool bQuickDifference		= false;
		bool bQuickSourceValidation = false;
	} DeprecatedOptions;

	auto AddTlsOptions = [&CacertFilenameUtf8, &bRequireTls, &bAllowInsecureTls, &DangerousGroupId](CLI::App* App)
	{
		App->add_option("--cacert", CacertFilenameUtf8, "Certificate authority file to use for TLS validation (.pem)");
		App->add_flag("--tls", bRequireTls, "Force TLS when connecting to remote server");
		App->add_flag("--insecure", bAllowInsecureTls, "Skip remote server TLS certificate validation")->group(DangerousGroupId);
	};

	auto AddProxyOptions = [&RemoteAddressUtf8, &ProtocolName, &bNoProxySelect, &bNoCompression](CLI::App* App)
	{
		App->add_option("--proxy, --remote, --server",
						RemoteAddressUtf8,
						"Download server address ([protocol+][transport://]address[:port][/request][#namespace])");
		App->add_flag("--no-proxy-select",
					  bNoProxySelect,
					  "Skip automatic server selection and use the exact one specified by command line or environment variable");
		App->add_flag("--no-compression",
					  bNoCompression,
					  "Disable compression when downloading blocks from the server, if possible (intended for debugging)");
		App->add_option("--protocol", ProtocolName, "Explicitly specify server protocol instead of inferring it from URL")
			->required(false)
			->check(CLI::IsMember({"unsync", "jupiter", "horde"}));
		App->add_flag("--no-server-announcements", GNoServerAnnouncements, "Suppress server announcement messages");
	};

	// Configure hash

	CLI::App* SubHash = Cli.add_subcommand("hash", "Generate hash manifest for a file or directory");
	SubHash->add_option("Input", InputFilenameUtf8, "Input file or directory path")->required();
	SubHash->add_flag("-f, --force", bForceOperation, "Force the operation even if hash is already computed for the input");
	SubHash
		->add_option("--mode",
					 ChunkModeUtf8,
					 "Specify chunking mode. Fixed chunking is faster and may produce smaller patches. Variable chunking allows block "
					 "reuse between files.")
		->check(CLI::IsMember({"fixed", "variable"}))
		->default_str("variable");
	SubHash->add_option("-o", OutputFilenameUtf8, "Output file name");
	SubHash->add_option("--strong", StrongHashUtf8, "Specify strong hash algorithm instead)")
		->check(CLI::IsMember({"blake3.128", "blake3.160", "iohash", "blake3.256", "md5"}))
		->default_str(StrongHashUtf8);
	SubHash->add_option("--weak", WeakHashUtf8, "Specify weak hash algorithm instead)")
		->check(CLI::IsMember({"naive", "buzhash"}))
		->default_str(WeakHashUtf8);
	SubHash->add_option("-b, --block", HashOrSyncBlockSize, "Block size in bytes (default=64KB)");
	SubHash->add_flag(
		"--update",
		bIncrementalMode,
		"Create a directory manifest incrementally, by updating an existing manifest if one exists (only process changed files)");
	SubHash->add_flag(
		"--pack-small-files",
		bPackOnlySmallFiles,
		"Small files will be copied to compressed pack files during manifest generation and stored next to the manifest. "
		"This can make syncing more efficient by reducing the number of remote file handles that must be opened. Implies --pack. "
		"Files less than 4MB in size are considered small.");
	SubHash->add_flag(
		"--pack",
		bPackFiles,
		"Input files will be copied to compressed pack files during manifest generation and stored next to the manifest file.");
	SubCommands.push_back(SubHash);

	// Configure verify

	CLI::App* SubVerify = Cli.add_subcommand("verify", "Verify directory contents against its manifest");
	SubVerify->add_option("Input", InputFilenameUtf8, "Input directory path")->required();
	SubCommands.push_back(SubVerify);

	// Configure pack

	CLI::App* SubPack = nullptr;
	{
		SubPack =
			Cli.add_subcommand("pack",
							   "EXPERIMENTAL: Generate manifest for a directory and store all referenced data in a compressed pack file")->group(HiddenGroupId);

		SubPack->add_option("Input", InputFilenameUtf8, "Input directory path")->required();

		auto P4HaveFileOpt =
			SubPack->add_option("--p4havefile",
								P4HavePathUtf8,
								"Use `p4 have` output from a given file to explicitly specify files included in the manifest");

		auto RunP4HaveOpt = SubPack->add_flag("--p4have", bRunP4Have, "Run `p4 have` when generating the dirctory pack");

		SubPack->add_option("--store", StorePathUtf8, "Use this location to store pack data (default: <Input>/.unsync/pack)");
		SubPack->add_option("--snapshot", SnapshotNameUtf8, "Custom name for the snapshot (will overwrite an existing tag)");

		RunP4HaveOpt->excludes(P4HaveFileOpt);

		SubCommands.push_back(SubPack);
	}

	CLI::App* SubUnpack = nullptr;
	{
		SubUnpack = Cli.add_subcommand("unpack", "EXPERIMENTAL: Sync directory based on package snapshot")->group(HiddenGroupId);

		SubUnpack->add_option("Output", OutputFilenameUtf8, "Output directory path")->required();

		SubUnpack->add_option("--store", StorePathUtf8, "Pack storage path")->required();
		SubUnpack->add_option("--snapshot", SnapshotNameUtf8, "Directory snapshot ID")->required();
		SubUnpack->add_option("--p4havefile", P4HavePathUtf8, "Write revision control data in `p4 have` format into this file");
		SubUnpack->add_flag("--no-revisions", bNoOutputRevisions, "Skip writing revision control data to <output>/.unsync/revisions.txt");
		SubUnpack->add_flag("--no-files",
							bNoOutputFiles,
							"Skip actually unpacking the snapshot files, but attempt to reconstruct and verify the manifest. "
							"Can be used in combination with --p4havefile option to only extract the `p4 have` list.");


		SubCommands.push_back(SubUnpack);
	}

	// Configure push

	CLI::App* SubPush = Cli.add_subcommand("push", "Loads a manifest from a directory and uploads referenced blocks to the remote server");
	SubPush->add_option("Input", InputFilenameUtf8, "Input file or directory path")->required();
	SubPush
		->add_option("Remote",
					 RemoteAddressUtf8,
					 "Remote storage that will receive blocks ([transport://]address[:port][/request][#namespace])")
		->required();
	SubPush->add_option("--http-header-file",
						HttpHeaderFilenameUtf8,
						"Text file that contains any extra HTTP headers to pass to the remote server (auth tokens, etc.)");
	SubPush->add_flag("--insecure", bAllowInsecureTls, "Skip remote server TLS certificate validation");
	SubCommands.push_back(SubPush);

	CLI::App* SubInfo = Cli.add_subcommand("info", "Display information about a manifest file or diff two manifests");
	SubInfo->add_option("InputA", InputFilenameUtf8, "Input manifest file or root directory")->required();
	SubInfo->add_option("InputB", InputFilename2Utf8, "Optional input manifest file or root directory");
	SubInfo->add_flag("--files", bInfoFiles, "List all files in the manifest");
	SubInfo->add_option(
		"--include",
		IncludeFilterArrayUtf8,
		"Include filenames that contain specified words (comma separated). If this is not present, all files will be included.");
	SubInfo->add_option("--exclude",
						ExcludeFilterArrayUtf8,
						"Exclude filenames that contain specified words (comma separated). Filter is run after --include.");
	SubInfo->add_flag("--decode", bDecode, "Decode binary manifest into json");
	SubCommands.push_back(SubInfo);

	// Configure diff

	CLI::App* SubDiff = Cli.add_subcommand("diff", "Compute difference required to transform BaseFile into SourceFile");
	SubDiff->add_option("Base", BaseFilenameUtf8, "Base file name (local data)")->required();
	SubDiff->add_option("Source", SourceFilenameUtf8, "Source file name (remote data)")->required();
	SubDiff->add_option("-o", OutputFilenameUtf8, "Output patch file name (data required to transform base into source)");
	SubDiff->add_option("--level", CompressionLevel, "ZSTD compression level (default=3)");
	SubDiff->add_option("-b, --block", DiffBlockSize, "Block size in bytes (default=4KB)");
	SubCommands.push_back(SubDiff);

	// Configure sync

	CLI::App* SubSync = Cli.add_subcommand("sync", "Synchronize files, transforming target file/directory into source");
	SubSync
		->add_option("Source",
					 SourceFilenameUtf8,
					 "Source path, object name, hash or full URL ([transport://]address[:port]#namespace/object)")
		->required();
	SubSync->add_option("Target", TargetFilenameUtf8, "Target path")->required();
	SubSync->add_option("-m, --manifest", SourceManifestFilenameUtf8, "Override manifest path for Source");
	AddProxyOptions(SubSync);
	SubSync->add_option("--dfs", PreferredDfsUtf8, "DEPRECATED: Preferred DFS mirror (matched by sub-string)")->group(HiddenGroupId);
	SubSync->add_option(
		"--overlay",
		OverlayArrayUtf8,
		"Additional source directory to sync (keep unique files from all sources, overwrite conflicting files with overlay source)");
	SubSync->add_option(
		"--include",
		IncludeFilterArrayUtf8,
		"Include filenames that contain specified words (comma separated). If this is not present, all files will be included.");
	SubSync->add_option("--exclude",
						ExcludeFilterArrayUtf8,
						"Exclude filenames that contain specified words (comma separated). Filter is run after --include.");
	AddTlsOptions(SubSync);

	SubSync->add_option("--http-header-file",
						HttpHeaderFilenameUtf8,
						"Text file that contains any extra HTTP headers to pass to the remote server (auth tokens, etc.)");
	SubSync->add_flag("--no-cleanup", bNoCleanupAfterSync, "Do not delete local files that aren't in the manifest after a successful sync");
	SubSync->add_option("--cleanup-exclude",
						CleanupExcludeFilterArrayUtf8,
						"Exclude filenames that contain specified words from cleanup process (comma separated)");

	// Deprecated --quick flag
	SubSync
		->add_flag("--quick",
				   DeprecatedOptions.bQuickSyncMode,
				   "Quick sync mode that skips some of the validation steps (enables all '--quick-*' options)")
		->group(HiddenGroupId);

	// Deprecated --quick-source-validation flag
	SubSync
		->add_flag("--quick-source-validation",
				   DeprecatedOptions.bQuickSourceValidation,
				   "Skip checking if all source files are present before starting a sync")
		->group(HiddenGroupId);

	// Deprecatred --quick-difference flag
	SubSync
		->add_flag("--quick-difference",
				   DeprecatedOptions.bQuickDifference,
				   "Allow computing file difference based on previous sync manifest and file timestamps")
		->group(HiddenGroupId);

	SubSync->add_flag("--full-diff",
					  bFullDifference,
					  "Run the full binary differencing algorithm on local files, even if there is a compatible local directory "
					  "manifest and file timestamps/sizes match. This is an extra precaution that will handle any unexpected local file "
					  "modifications, but it should not be needed in a common case.");

	SubSync->add_flag("--full-source-scan",
					  bFullSourceScan,
					  "Perform a full scan of the source directory to check that all files are present and their timestamps/sizes match "
					  "the manifest. This is an extra precaution that will detect any missing or invalid remote files before running the "
					  "sync process, however this can be very slow when dealing with large numbers of files and directories.");

	SubSync->add_flag("--no-output-validation", bNoOutputValidation, "Skip final patched file block hash validation (DANGEROUS)")
		->group(DangerousGroupId);
	SubSync->add_flag("--no-space-validation", bNoSpaceValidation, "Skip checking available disk space before sync (DANGEROUS)")
		->group(DangerousGroupId);
	SubSync->add_option("--scavenge", ScavengeRootUtf8, "Search for unsync manifests and reusable blocks in this directory (EXPERIMENTAL)")
		->group(ExperimentalGroupId);
	SubSync->add_flag("--login", bShouldLogin, "Use user authentication when accessing unsync server");
	SubSync->add_option("--token", AuthTokenPathUtf8, "Explicit path to the authentication token file to use");
	SubSync->add_flag("--no-timeout", bNoSocketTimeout, "Disable the default 15 minute timeout on network socket operations");

	CLI::Option* BackgroundMemoryBudgetOption = SubSync->add_option("--background-task-memory",
															BackgroundTaskMemoryBudgetGB,
															"Set memory budget that background tasks in gigabytes (default: 2 GB)");

	SubCommands.push_back(SubSync);

	CLI::App* SubPatch = Cli.add_subcommand("patch", "Applies a patch generated with 'diff' on top of base file");
	SubPatch->add_option("Base", BaseFilenameUtf8, "Base file name")->required();
	SubPatch->add_option("Patch", PatchFilenameUtf8, "Patch file name")->required();
	SubPatch->add_option("-o", OutputFilenameUtf8, "Output file name")->required();
	SubCommands.push_back(SubPatch);

	CLI::App* SubTest = Cli.add_subcommand("test", "Run internal tests");
	SubTest->add_option("Preset, --preset", PresetUtf8, "Test preset")->default_str(PresetUtf8);
	SubCommands.push_back(SubTest);

	// Configure query

	CLI::App* SubQuery = Cli.add_subcommand("query", "Run a query command on the remote server");
	SubQuery->add_option("QueryString", QueryStringUtf8, "Query to run: mirrors, login, list")->required();
	SubQuery->add_option("QueryArgs", QueryArgsUtf8, "Query arguments");
	SubQuery->add_option("-o", OutputFilenameUtf8, "Output file name");
	AddProxyOptions(SubQuery);

	AddTlsOptions(SubQuery);
	SubCommands.push_back(SubQuery);

	// Configure login

	CLI::App* SubLogin = Cli.add_subcommand("login", "Authenticate with the remote server (acquire access and refresh tokens)");
	SubLogin->add_flag("--interactive", bInteractive, "Allow user interaction through modal dialogs");
	SubLogin->add_flag("--decode", bDecode, "Decode authentication token (implies --print)");
	SubLogin->add_flag("--print", bPrint, "Print authentication token to standard output");
	SubLogin->add_flag("--print-http-header", bPrintHttpHeader, "Print authentication token to standard output as HTTP Authorization header that could be used with curl, etc.");
	SubLogin->add_flag("--refresh", bForceRefreshAuth, "Force authentication refresh even if access token has not yet expired");
	SubLogin->add_flag("--quick", bQuickLogin, "Skip token validation using remote server (fast path when cached acess token is expected to be valid)");
	AddTlsOptions(SubLogin);
	AddProxyOptions(SubLogin);
	SubCommands.push_back(SubLogin);

	// Configure mount

	CLI::App* SubMount = Cli.add_subcommand("mount", "Mount directory manifest as a virtual file system (EXPERIMENTAL)")->group(HiddenGroupId);
	SubMount
		->add_option("Source",
					 SourceFilenameUtf8,
					 "Source path, object name, hash or full URL ([transport://]address[:port]#namespace/object)")
		->required();
	AddProxyOptions(SubMount);
	SubCommands.push_back(SubMount);

	for (CLI::App* Subcommand : SubCommands)
	{
		Subcommand->add_flag("-d, --dry, --dry-run", GDryRun, "Don't write any outputs to disk");
		auto VerboseFlag	 = Subcommand->add_flag("-v, --verbose", GLogVerbose, "Verbose logging");
		auto VeryVerboseFlag = Subcommand->add_flag("--very-verbose", GLogVeryVerbose, "Very verbose logging");
		auto SilentFlag		 = Subcommand->add_flag("--silent", GLogSilent, "Skip all console logging except errors and warnings");
		Subcommand->add_flag("--progress", GLogProgress, "Output @progress and @status markers");
		Subcommand->add_flag("--stdout", GLogStdOut, "Send all output to stdout (no stderr)");
		Subcommand->add_option("--threads", GMaxThreads, "Limit worker threads to specified number");
		Subcommand->add_flag("--buffered-files", GForceBufferedFiles, "Always use buffered file IO");
		Subcommand->add_flag("--debug", bUseDebugMode, "Enable extra debugging features, such as extra memory safety validation");
		Subcommand->add_flag("--experimental", GExperimental, "Enable experimental code paths")->group(HiddenGroupId);
		Subcommand->add_flag("--streaming", GExperimentalStreaming, "Enable experimental file streaming code path")->group(HiddenGroupId);

		SilentFlag->excludes(VerboseFlag);
		SilentFlag->excludes(VeryVerboseFlag);
		VeryVerboseFlag->excludes(VerboseFlag);
	}

	// Run the command

	try
	{
		Cli.parse(Argc, Argv);
	}
	catch (CLI::Error& E)
	{
		std::stringstream OutputStream;
		const int32		  ReturnCode = Cli.exit(E, OutputStream, OutputStream);
		std::wstring	  Output	 = ConvertUtf8ToWide(OutputStream.str());
		wprintf(L"%ls", Output.c_str());
		return ReturnCode;
	}

	if (bPackOnlySmallFiles)
	{
		bPackFiles = true;
	}

	if (Cli.get_subcommands().size() == 0)
	{
		wprintf(L"%hs", Cli.help().c_str());
	}

	FTimingLogger TimingLogger("Total time", ELogLevel::Info);

	// Configure default output mehtod based on subcommand.
	// In machine-readable mode, all verbose logging is directed to stderr.

	if (Cli.got_subcommand(SubQuery) || Cli.got_subcommand(SubLogin))
	{
		GLogMachineReadable = true;
	}
	else if (Cli.got_subcommand(SubInfo))
	{
		GLogMachineReadable = bDecode;
	}

	UNSYNC_VERBOSE(L"UNSYNC v%hs", GetVersionString().c_str());

	UnsyncMallocInit(bUseDebugMode ? EMallocType::Debug : EMallocType::Default);

	if (bUseDebugMode)
	{
		UNSYNC_LOG(L"*** Debug mode enabled ***");
	}

	// Augment configuration based on environment variables if corresponding command line arguments are missing.

	if (const char* EnvCleanupExclude = getenv("UNSYNC_CLEANUP_EXCLUDE"))
	{
		UNSYNC_LOG(L"Using UNSYNC_CLEANUP_EXCLUDE environment: '%hs'", EnvCleanupExclude);
		CleanupExcludeFilterArrayUtf8.push_back(EnvCleanupExclude);
	}

	if (PreferredDfsUtf8.empty())
	{
		const char* EnvDfs = getenv("UNSYNC_DFS");
		if (EnvDfs)
		{
			UNSYNC_LOG(L"Using UNSYNC_DFS environment: '%hs'", EnvDfs);
			PreferredDfsUtf8 = std::string(EnvDfs);
		}
	}

	if (RemoteAddressUtf8.empty())
	{
		const char* EnvProxy = getenv("UNSYNC_PROXY");
		if (EnvProxy)
		{
			UNSYNC_LOG(L"Using UNSYNC_PROXY environment: '%hs'", EnvProxy);
			RemoteAddressUtf8 = std::string(EnvProxy);
		}
	}

	if (CacertFilenameUtf8.empty())
	{
		const char* EnvCacert = getenv("UNSYNC_CACERT");
		if (EnvCacert)
		{
			UNSYNC_LOG(L"Using UNSYNC_CACERT environment: '%hs'", EnvCacert);
			CacertFilenameUtf8 = std::string(EnvCacert);
		}
	}

	if (HttpHeaderFilenameUtf8.empty())
	{
		const char* EnvHttpHeaderFile = getenv("UNSYNC_HTTP_HEADER_FILE");
		if (EnvHttpHeaderFile)
		{
			UNSYNC_LOG(L"Using UNSYNC_HTTP_HEADER_FILE environment: '%hs'", EnvHttpHeaderFile);
			HttpHeaderFilenameUtf8 = std::string(EnvHttpHeaderFile);
		}
	}

	if (GLogVeryVerbose)
	{
		// Force verbose mode if very-verbose flag is present
		GLogVerbose = true;
	}

	if (DeprecatedOptions.bQuickSyncMode)
	{
		UNSYNC_WARNING(
			L"Quick mode is now the default and --quick flag is deprecated. Use --full-source-scan and --full-diff options to enable legacy "
			L"default behavior.");
	}

	if (DeprecatedOptions.bQuickSourceValidation)
	{
		UNSYNC_WARNING(
			L"Quick mode is now the default and --quick-source-validation flag is deprecated. Use --full-source-scan to enable legacy behavior "
			L"that scans source directory.");
	}

	if (DeprecatedOptions.bQuickDifference)
	{
		UNSYNC_WARNING(
			L"Quick mode is now the default and --quick-difference flag is deprecated. Use --full-diff to enable legacy behavior performs "
			L"full binary difference of local files even if timestamps and sizes match.");
	}

	EWeakHashAlgorithmID DefaultWeakHasher = EWeakHashAlgorithmID::BuzHash;
	if (WeakHashUtf8 == "naive")
	{
		DefaultWeakHasher = EWeakHashAlgorithmID::Naive;
	}
	else if (WeakHashUtf8 == "buzhash")
	{
		DefaultWeakHasher = EWeakHashAlgorithmID::BuzHash;
	}

	EStrongHashAlgorithmID DefaultStrongHasher = EStrongHashAlgorithmID::Blake3_128;
	if (StrongHashUtf8 == "md5")
	{
		DefaultStrongHasher = EStrongHashAlgorithmID::MD5;
	}
	else if (StrongHashUtf8 == "blake3.128")
	{
		DefaultStrongHasher = EStrongHashAlgorithmID::Blake3_128;
	}
	else if (StrongHashUtf8 == "blake3.160" || StrongHashUtf8 == "iohash")
	{
		DefaultStrongHasher = EStrongHashAlgorithmID::Blake3_160;
	}
	else if (StrongHashUtf8 == "blake3.256")
	{
		DefaultStrongHasher = EStrongHashAlgorithmID::Blake3_256;
	}

	EChunkingAlgorithmID DefaultChunkingAlgorithm = EChunkingAlgorithmID::VariableBlocks;
	if (ChunkModeUtf8 == "fixed")
	{
		DefaultChunkingAlgorithm = EChunkingAlgorithmID::FixedBlocks;
	}
	else if (ChunkModeUtf8 == "variable")
	{
		DefaultChunkingAlgorithm = EChunkingAlgorithmID::VariableBlocks;
	}

	FRemoteDesc RemoteDesc;
	FAuthDesc	AuthDesc;

	bool bFilesystemSource = true;

	std::string_view PossibleUrl;

	if (Cli.got_subcommand(SubSync))
	{
		PossibleUrl = SourceFilenameUtf8;
	}
	else if (Cli.got_subcommand(SubQuery) && !QueryArgsUtf8.empty())
	{
		PossibleUrl = QueryArgsUtf8[0];
	}

	
	EProtocolFlavor ProtocolFlavorHint = EProtocolFlavor::Unknown;
	if (!ProtocolName.empty())
	{
		ProtocolFlavorHint = ProtocolFlavorFromString(ProtocolName);
	}

	if (RemoteAddressUtf8.empty() && LooksLikeUrl(PossibleUrl) && (Cli.got_subcommand(SubSync) || Cli.got_subcommand(SubQuery)))
	{
		// Derive remote server address from source name if explicit --proxy or --remote option is not provided for sync or query

		TResult<FRemoteDesc> ParsedRemoteDesc = FRemoteDesc::FromUrl(PossibleUrl, ProtocolFlavorHint);

		if (ParsedRemoteDesc.IsOk())
		{
			RemoteDesc = *ParsedRemoteDesc;
			bFilesystemSource = false;

			if (RemoteDesc.Protocol == EProtocolFlavor::Jupiter)
			{
				size_t SlashPos = RemoteDesc.StorageNamespace.find_first_of('/');
				if (SlashPos == std::string::npos)
				{
					UNSYNC_ERROR(L"Jupiter URL source is expected to follow [transport://]address[:port]#namespace/object format");
					return 1;
				}
				else
				{
					SourceFilenameUtf8			= RemoteDesc.StorageNamespace.substr(SlashPos + 1);
					RemoteDesc.StorageNamespace = RemoteDesc.StorageNamespace.substr(0, SlashPos);
				}
			}
			else if (RemoteDesc.Protocol == EProtocolFlavor::Unsync || RemoteDesc.Protocol == EProtocolFlavor::Horde)
			{
				bShouldLogin = true; // Try to authenticate by default when source is a valid URL

				if (Cli.got_subcommand(SubQuery))
				{
					QueryArgsUtf8[0] = RemoteDesc.RequestPath;
				}
				else if (Cli.got_subcommand(SubSync))
				{
					SourceFilenameUtf8 = RemoteDesc.RequestPath;
				}
			}
		}
		else
		{
			UNSYNC_ERROR(L"Failed to parse remote address '%hs': %ls",
							RemoteAddressUtf8.c_str(),
							ParsedRemoteDesc.TryError()->Context.c_str());
			return 1;
		}
	}
	else
	{
		TResult<FRemoteDesc> ParsedRemoteDesc = FRemoteDesc::FromUrl(RemoteAddressUtf8, ProtocolFlavorHint);

		if (ParsedRemoteDesc.IsOk())
		{
			RemoteDesc = *ParsedRemoteDesc;
		}
		else
		{
			UNSYNC_ERROR(L"Failed to parse remote address '%hs': %ls",
						 RemoteAddressUtf8.c_str(),
						 ParsedRemoteDesc.TryError()->Context.c_str());
			return 1;
		}
	}

	// Derive artifact request path when syncing from Horde, if it wasn't specified via URL source syntax
	if (RemoteDesc.Protocol == EProtocolFlavor::Horde && Cli.got_subcommand(SubSync))
	{
		bFilesystemSource = false;

		auto ResolveHordeArtifactPath = [](const std::string& PathUtf8)
		{
			TResult<FHordeArtifactQuery> Query = FHordeArtifactQuery::FromString(PathUtf8);
			if (Query.IsError())
			{
				LogError(Query.GetError(), L"Could not parse sync source path");
				return std::string();
			}

			if (Query->Id.empty())
			{
				UNSYNC_ERROR(L"Could not parse sync source path. Artifact ID is expected, i.e. '#123456abcdef'.");
				return std::string();
			}

			return std::string("api/v2/artifacts/") + Query->Id;
		};

		if (RemoteDesc.RequestPath.empty())
		{
			TResult<FHordeArtifactQuery> Query = FHordeArtifactQuery::FromString(SourceFilenameUtf8);
			if (Query.IsError())
			{
				LogError(Query.GetError(), L"Could not parse sync source path");
				return 1;
			}

			if (Query->Id.empty())
			{
				UNSYNC_ERROR(L"Could not parse sync source path. Artifact ID is expected, i.e. '#123456abcdef'.");
				return 1;
			}

			SourceFilenameUtf8	   = ResolveHordeArtifactPath(SourceFilenameUtf8);
			RemoteDesc.RequestPath = SourceFilenameUtf8;
		}

		for (std::string& OverlayPath : OverlayArrayUtf8)
		{
			OverlayPath = ResolveHordeArtifactPath(OverlayPath);
		}
	}

	if (bNoCompression)
	{
		UNSYNC_VERBOSE(L"Uncompressed data transfer is preferred");
		RemoteDesc.bPreferCompression = false;
	}

	FPath InputFilename			 = NormalizeFilenameUtf8(InputFilenameUtf8);
	FPath InputFilename2		 = NormalizeFilenameUtf8(InputFilename2Utf8);
	FPath OutputFilename		 = NormalizeFilenameUtf8(OutputFilenameUtf8);
	FPath BaseFilename			 = NormalizeFilenameUtf8(BaseFilenameUtf8);
	FPath TargetFilename		 = NormalizeFilenameUtf8(TargetFilenameUtf8);
	FPath PatchFilename			 = NormalizeFilenameUtf8(PatchFilenameUtf8);
	FPath ScavengeRoot			 = NormalizeFilenameUtf8(ScavengeRootUtf8);
	FPath SourceManifestFilename = NormalizeFilenameUtf8(SourceManifestFilenameUtf8);

	FPath SourceFilename = bFilesystemSource ? NormalizeFilenameUtf8(SourceFilenameUtf8) : FPath(SourceFilenameUtf8);

	if (GDryRun)
	{
		UNSYNC_LOG(L">>> DRY RUN <<<");
	}

	if (GExperimental)
	{
		UNSYNC_LOG(L">>> EXPERIMENTAL MODE <<<");
	}

	if (GLogVeryVerbose)
	{
		UNSYNC_LOG(L"Very verbose logging is enabled");
	}
	else if (GLogVerbose)
	{
		UNSYNC_LOG(L"Verbose logging is enabled");
	}

	if (GForceBufferedFiles)
	{
		UNSYNC_VERBOSE(L"Using buffered file IO");
	}

	GMaxThreads = std::max(1u, GMaxThreads);
	UNSYNC_VERBOSE(L"Using threads: %d", GMaxThreads);

	// Don't count the main thread when starting the thread pool
	const uint32 NumWorkerThreads = GMaxThreads - 1;

	static FScheduler MainScheduler(NumWorkerThreads);

	UNSYNC_ASSERT(GScheduler == nullptr);
	GScheduler = &MainScheduler;

	if (Cli.got_subcommand(SubHash) || Cli.got_subcommand(SubPack))
	{
		UNSYNC_VERBOSE(L"Using block size: %d KB", HashOrSyncBlockSize / 1024);
	}

	if (Cli.got_subcommand(SubDiff))
	{
		UNSYNC_VERBOSE(L"Using block size: %d KB", DiffBlockSize / 1024);
	}

	if (Cli.got_subcommand(SubDiff))
	{
		DefaultWeakHasher		 = EWeakHashAlgorithmID::Naive;
		DefaultChunkingAlgorithm = EChunkingAlgorithmID::FixedBlocks;
	}

	if (Cli.got_subcommand(SubHash) || Cli.got_subcommand(SubDiff))
	{
		if (!bIncrementalMode)
		{
			UNSYNC_VERBOSE(L"Using weak hash: %hs", ToString(DefaultWeakHasher));
			UNSYNC_VERBOSE(L"Using strong hash: %hs", ToString(DefaultStrongHasher));
			UNSYNC_VERBOSE(L"Using chunking mode: %hs", ToString(DefaultChunkingAlgorithm));
		}
	}

	FAlgorithmOptions Algorithm;
	Algorithm.ChunkingAlgorithmId	= DefaultChunkingAlgorithm;
	Algorithm.StrongHashAlgorithmId = DefaultStrongHasher;
	Algorithm.WeakHashAlgorithmId	= DefaultWeakHasher;

	FSyncFilter SyncFilter;

	for (const std::string& Str : ExcludeFilterArrayUtf8)
	{
		SyncFilter.ExcludeFromSync(ConvertUtf8ToWide(Str));
	}
	for (const std::string& Str : IncludeFilterArrayUtf8)
	{
		SyncFilter.IncludeInSync(ConvertUtf8ToWide(Str));
	}

	for (const std::string& Str : CleanupExcludeFilterArrayUtf8)
	{
		SyncFilter.ExcludeFromCleanup(ConvertUtf8ToWide(Str));
	}

	if (!PreferredDfsUtf8.empty() && !SourceFilenameUtf8.empty())
	{
		LogGlobalStatus(L"Enumerating DFS");
		UNSYNC_LOG(L"Enumerating DFS");
		std::wstring PreferredDfs = ConvertUtf8ToWide(PreferredDfsUtf8);
		auto		 DfsEntries	  = DfsEnumerate(SourceFilename);

		const FDfsStorageInfo* FoundDfsStorage		= nullptr;
		size_t				   FoundDfsSubstringPos = std::numeric_limits<size_t>::max();
		for (const FDfsStorageInfo& DfsStorage : DfsEntries.Storages)
		{
			size_t Pos = DfsStorage.Server.find(PreferredDfs);
			if (Pos < FoundDfsSubstringPos)
			{
				FoundDfsSubstringPos = Pos;
				FoundDfsStorage		 = &DfsStorage;
			}
		}

		if (FoundDfsStorage)
		{
			UNSYNC_LOG(L"Found preferred DFS storage server '%ls' with share '%ls'",
						   FoundDfsStorage->Server.c_str(),
						   FoundDfsStorage->Share.c_str());

			FDfsAlias DfsAlias;
			DfsAlias.Source = DfsEntries.Root;
			DfsAlias.Target = FPath(L"\\\\") / FoundDfsStorage->Server / FoundDfsStorage->Share;

			UNSYNC_LOG(L"Using DFS alias '%ls' -> '%ls'", DfsAlias.Source.wstring().c_str(), DfsAlias.Target.wstring().c_str());

			if (!DfsAlias.Source.empty())
			{
				SyncFilter.DfsAliases.push_back(std::move(DfsAlias));
			}
		}
	}

	if (!HttpHeaderFilenameUtf8.empty())
	{
		FPath	Filename		  = NormalizeFilenameUtf8(HttpHeaderFilenameUtf8);
		FBuffer HttpHeadersBuffer = ReadFileToBuffer(Filename);

		const uint8 Bom[2] = {0xFF, 0xFE};
		if (HttpHeadersBuffer.Size() > 2 && !memcmp(HttpHeadersBuffer.Data(), Bom, 2))
		{
			std::wstring_view View((const wchar_t*)(HttpHeadersBuffer.Data() + 2), (HttpHeadersBuffer.Size() - 2) / 2);
			RemoteDesc.HttpHeaders = ConvertWideToUtf8(View);
		}
		else
		{
			RemoteDesc.HttpHeaders = std::string((const char*)HttpHeadersBuffer.Data(), HttpHeadersBuffer.Size());
		}
	}

	if (bRequireTls)
	{
		RemoteDesc.TlsRequirement = ETlsRequirement::Required;
	}

	if (bAllowInsecureTls)
	{
		RemoteDesc.bTlsVerifyCertificate = false;
		RemoteDesc.bTlsVerifySubject	 = false;
		UNSYNC_WARNING(L"Remote server certificate verification is disabled.");
	}
	else
	{
		RemoteDesc.bTlsVerifyCertificate = true;
		RemoteDesc.bTlsVerifySubject	 = true;
	}

	if (bNoOutputValidation)
	{
		UNSYNC_WARNING(L"Final file validation is disabled. Data corruptions will not be detected or reported!");
	}

	if (!CacertFilenameUtf8.empty())
	{
		FPath	CacertPath	 = NormalizeFilenameUtf8(CacertFilenameUtf8);
		FBuffer CacertBuffer = ReadFileToBuffer(CacertPath);

		RemoteDesc.TlsCacert = std::make_shared<FBuffer>(std::move(CacertBuffer));
		RemoteDesc.TlsCacert->PushBack('\n');
	}

	{
		FPath	ExtraCertPath = GExePath.parent_path() / "unsync.cer";
		FBuffer CertBuffer	  = ReadFileToBuffer(ExtraCertPath);
		if (!CertBuffer.Empty())
		{
			UNSYNC_LOG(L"Using trusted certificates from '%ls'", ExtraCertPath.wstring().c_str());
			if (!RemoteDesc.TlsCacert)
			{
				RemoteDesc.TlsCacert = std::make_shared<FBuffer>(std::move(CertBuffer));
			}
			else
			{
				RemoteDesc.TlsCacert->Append(CertBuffer);
			}
			RemoteDesc.TlsCacert->PushBack('\n');
		}
	}

	if (bShouldLogin)
	{
		RemoteDesc.PrimaryHost			   = RemoteDesc.Host;
		RemoteDesc.bAuthenticationRequired = true;
	}

	if (bNoSocketTimeout)
	{
		RemoteDesc.RecvTimeoutSeconds = 0;
	}
	else
	{
		RemoteDesc.RecvTimeoutSeconds = 15 * 60;
	}

	if (!bNoProxySelect
		&& Cli.got_subcommand(SubSync)
		&& RemoteDesc.IsValid() && RemoteDesc.Protocol == EProtocolFlavor::Unsync)
	{
		UNSYNC_LOG(L"Selecting server using root '%hs'", RemoteDesc.Host.Address.c_str());
		TResult<FMirrorInfo> MirrorResult = FindClosestMirror(RemoteDesc);
		if (const FMirrorInfo* Mirror = MirrorResult.TryData())
		{
			UNSYNC_LOG(L"Closest server: '%hs', ping: %.2f ms", Mirror->Address.c_str(), Mirror->Ping * 1000.0);

			RemoteDesc.Host.Address = Mirror->Address;
			RemoteDesc.Host.Port	= Mirror->Port;
			
			if (Mirror->Port == 443)
			{
				RemoteDesc.TlsRequirement = ETlsRequirement::Required;
			}
			else if (RemoteDesc.TlsRequirement < ETlsRequirement::Required)
			{
				RemoteDesc.TlsRequirement = ETlsRequirement::Preferred;
			}
		}
		else
		{
			UNSYNC_WARNING(L"Failed to find closest proxy using root server '%hs': %ls",
						   RemoteAddressUtf8.c_str(),
						   MirrorResult.TryError()->Context.c_str());
		}
	}

	if (Cli.got_subcommand(SubHash))
	{
		FCmdHashOptions HashOptions;

		HashOptions.Input			= InputFilename;
		HashOptions.Output			= OutputFilename;
		HashOptions.BlockSize		= HashOrSyncBlockSize;
		HashOptions.Algorithm		= Algorithm;
		HashOptions.bForce			= bForceOperation;
		HashOptions.bIncremental	= bIncrementalMode;
		HashOptions.bPackFiles		= bPackFiles;

		if (bPackOnlySmallFiles)
		{
			HashOptions.MaxFileSizeToPack = 4_MB;
		}

		return CmdHash(HashOptions);
	}
	else if (Cli.got_subcommand(SubVerify))
	{
		FCmdVerifyOptions Options;
		Options.Input = InputFilename;

		return CmdVerify(Options);
	}
	else if (Cli.got_subcommand(SubPack))
	{
		FCmdPackOptions PackOptions;

		PackOptions.RootPath	 = InputFilename;
		PackOptions.P4HavePath	 = NormalizeFilenameUtf8(P4HavePathUtf8);
		PackOptions.StorePath	 = NormalizeFilenameUtf8(StorePathUtf8);
		PackOptions.bRunP4Have	 = bRunP4Have;
		PackOptions.BlockSize	 = HashOrSyncBlockSize;
		PackOptions.Algorithm	 = Algorithm;
		PackOptions.SnapshotName = SnapshotNameUtf8;

		return CmdPack(PackOptions);
	}
	else if (Cli.got_subcommand(SubUnpack))
	{
		FCmdUnpackOptions UnpackOptions;

		UnpackOptions.OutputPath	   = OutputFilename;
		UnpackOptions.SnapshotName	   = SnapshotNameUtf8;
		UnpackOptions.P4HaveOutputPath = NormalizeFilenameUtf8(P4HavePathUtf8);
		UnpackOptions.StorePath		   = NormalizeFilenameUtf8(StorePathUtf8);
		UnpackOptions.bOutputFiles	   = !bNoOutputFiles;
		UnpackOptions.bOutputRevisions = !bNoOutputRevisions;

		return CmdUnpack(UnpackOptions);
	}
	else if (Cli.got_subcommand(SubDiff))
	{
		FCmdDiffOptions DiffOptions;

		DiffOptions.Source			 = SourceFilename;
		DiffOptions.Base			 = BaseFilename;
		DiffOptions.Output			 = OutputFilename;
		DiffOptions.BlockSize		 = DiffBlockSize;
		DiffOptions.WeakHasher		 = DefaultWeakHasher;
		DiffOptions.StrongHasher	 = DefaultStrongHasher;
		DiffOptions.CompressionLevel = CompressionLevel;

		return CmdDiff(DiffOptions);
	}
	else if (Cli.got_subcommand(SubSync))
	{
		if (!ScavengeRoot.empty() && !IsDirectory(ScavengeRoot))
		{
			UNSYNC_WARNING(L"Scavenge directory '%ls' does not exist", ScavengeRoot.wstring().c_str());
			ScavengeRoot = FPath{};
		}

		if (bShouldLogin)
		{
			UNSYNC_LOG(L"Attempting to authenticate");
			UNSYNC_LOG_INDENT;

			TResult<FAuthDesc> AuthDescResult = GetRemoteAuthDesc(RemoteDesc);

			if (AuthDescResult.IsOk())
			{
				AuthDesc = AuthDescResult.GetData();

				AuthDesc.TokenPath = NormalizeFilenameUtf8(AuthTokenPathUtf8);

				// Note: since tokens can expire during a long operation,
				// we can only save the auth descriptor and re-authenticate later if necessary
				TResult<FAuthToken> AuthTokenResult = Authenticate(AuthDesc, RemoteDesc.RecvTimeoutSeconds);

				if (AuthTokenResult.IsError())
				{
					UNSYNC_ERROR("Failed to authenticate with server '%hs'", RemoteDesc.Host.Address.c_str());
					LogError(AuthTokenResult.GetError());
					return -1;
				}

				// Authentication requires encrypted connection
				RemoteDesc.TlsRequirement		   = ETlsRequirement::Required;
				RemoteDesc.bAuthenticationRequired = true;

				UNSYNC_LOG(L"Authentication enabled")
			}
		}

		// Try to derive default memory budget

		if (BackgroundMemoryBudgetOption->empty())
		{
			FSystemMemoryInfo MemoryInfo;
			if (QueryMemoryInfo(MemoryInfo))
			{
				uint32 InstalledMemoryGB = CheckedNarrow(MemoryInfo.InstalledPhysicalMemory >> 30);
				UNSYNC_VERBOSE2(L"Detected memory: %llu GB", InstalledMemoryGB);
				BackgroundTaskMemoryBudgetGB = std::max<uint32>(2, InstalledMemoryGB / 4);
			}
			else
			{
				UNSYNC_VERBOSE2(L"Could not detect system memory size");
			}

			UNSYNC_VERBOSE2(L"Using automatic background task memory budget: %llu GB", BackgroundTaskMemoryBudgetGB);
		}
		else
		{
			UNSYNC_VERBOSE2(L"Using explicit background task memory budget: %llu GB", BackgroundTaskMemoryBudgetGB);
		}
		//

		FCmdSyncOptions SyncOptions;

		SyncOptions.Algorithm				   = Algorithm;
		SyncOptions.Source					   = SourceFilename;
		SyncOptions.Target					   = TargetFilename;
		SyncOptions.SourceManifestOverride	   = SourceManifestFilename;
		SyncOptions.Remote					   = RemoteDesc;
		SyncOptions.AuthDesc				   = AuthDesc.IsValid() ? &AuthDesc : nullptr;
		SyncOptions.bFullDifference			   = bFullDifference;
		SyncOptions.bFullSourceScan			   = bFullSourceScan;
		SyncOptions.bCleanup				   = !bNoCleanupAfterSync;
		SyncOptions.Filter					   = &SyncFilter;
		SyncOptions.bValidateTargetFiles	   = !bNoOutputValidation;
		SyncOptions.bCheckAvailableSpace	   = !bNoSpaceValidation;
		SyncOptions.ScavengeRoot			   = ScavengeRoot;
		SyncOptions.BackgroundTaskMemoryBudget = uint64(BackgroundTaskMemoryBudgetGB) << 30ull;

		for (const std::string& Entry : OverlayArrayUtf8)
		{
			if (bFilesystemSource)
			{
				SyncOptions.Overlays.push_back(NormalizeFilenameUtf8(Entry));
			}
			else
			{
				SyncOptions.Overlays.push_back(FPath(Entry));
			}
		}

		return CmdSync(SyncOptions);
	}
	else if (Cli.got_subcommand(SubPatch))
	{
		FCmdPatchOptions PatchOptions;

		PatchOptions.Base	= BaseFilename;
		PatchOptions.Output = OutputFilename;
		PatchOptions.Patch	= PatchFilename;

		return CmdPatch(PatchOptions);
	}
	else if (Cli.got_subcommand(SubPush))
	{
		FCmdPushOptions PushOptions;
		PushOptions.Input  = InputFilename;
		PushOptions.Remote = RemoteDesc;

		return CmdPush(PushOptions);
	}
	else if (Cli.got_subcommand(SubTest))
	{
		UNSYNC_LOG(L"Running internal tests ...");
		RunTests(PresetUtf8);
	}
	else if (Cli.got_subcommand(SubInfo))
	{
		FCmdInfoOptions Options;
		Options.InputA	   = InputFilename;
		Options.InputB	   = InputFilename2;
		Options.bListFiles = bInfoFiles;
		Options.SyncFilter = &SyncFilter;
		Options.bDecode	   = bDecode;
		return CmdInfo(Options);
	}
	else if (Cli.got_subcommand(SubQuery))
	{
		FCmdQueryOptions QueryOptions;
		QueryOptions.Query		= QueryStringUtf8;
		QueryOptions.Args		= QueryArgsUtf8;
		QueryOptions.Remote		= RemoteDesc;
		QueryOptions.OutputPath = OutputFilename;
		return CmdQuery(QueryOptions);
	}
	else if (Cli.got_subcommand(SubLogin))
	{
		if (bDecode || bPrintHttpHeader)
		{
			bPrint = true;
		}

		FCmdLoginOptions LoginOptions;
		LoginOptions.Remote			  = RemoteDesc;
		LoginOptions.bInteractive	  = bInteractive;
		LoginOptions.bDecode		  = bDecode;
		LoginOptions.bPrint			  = bPrint;
		LoginOptions.bPrintHttpHeader = bPrintHttpHeader;
		LoginOptions.bForceRefresh	  = bForceRefreshAuth;
		LoginOptions.bQuick			  = bQuickLogin;
		return CmdLogin(LoginOptions);
	}
	else if (Cli.got_subcommand(SubMount))
	{
		FCmdMountOptions MountOptions;
		MountOptions.Path = SourceFilename;
		return CmdMount(MountOptions);
	}

	return 0;
}

#if UNSYNC_PLATFORM_WINDOWS	 // TODO: Ctrl-C signal handler for Linux
static BOOL WINAPI
ConsoleCtrlHandler(int Signal)
{
	FLogFlushScope FlushScope;

	const char* TerminateReason = nullptr;
	switch (Signal)
	{
		case CTRL_C_EVENT:
			TerminateReason = "Ctrl-C";
			break;
		case CTRL_BREAK_EVENT:
			TerminateReason = "Ctrl-Break";
			break;
		case CTRL_CLOSE_EVENT:
			TerminateReason = "console closed";
			break;
		default:
			TerminateReason = nullptr;
	}

	if (TerminateReason)
	{
		UNSYNC_LOG(L"\nTerminating process on request: %hs\n", TerminateReason);
		TerminateProcess(GetCurrentProcess(), 1);
	}

	return true;
}

static LONG
ExceptionFilter(_EXCEPTION_POINTERS* ExceptionPointers)
{
	FLogFlushScope FlushScope;

	PEXCEPTION_RECORD Record = ExceptionPointers->ExceptionRecord;

	if (Record->ExceptionCode == EXCEPTION_BREAKPOINT)
	{
		LogPrintf(ELogLevel::Error, L"Break point at address 0x%016X\n", Record->ExceptionAddress);
	}
	else
	{
		LogPrintf(ELogLevel::Error, L"Unhandled exception 0x%08X at address 0x%016X\n", Record->ExceptionCode, Record->ExceptionAddress);
	}

	LogWriteCrashDump(ExceptionPointers);

	return EXCEPTION_EXECUTE_HANDLER;
}
#endif	// UNSYNC_PLATFORM_WINDOWS

}  // namespace unsync

int
main(int argc, char** argv)
{
	using namespace unsync;

	FLogFlushScope FlushScope;

#if UNSYNC_PLATFORM_WINDOWS
	SetConsoleCtrlHandler((PHANDLER_ROUTINE)ConsoleCtrlHandler, TRUE);
	SetUnhandledExceptionFilter(ExceptionFilter);

	_setmode(_fileno(stdout), _O_U8TEXT);

	LPCWSTR WideCmdLine = GetCommandLineW();
	int		NumWideArgs = 0;
	LPWSTR* ArgvWide	= CommandLineToArgvW(WideCmdLine, &NumWideArgs);
	UNSYNC_ASSERT(argc == NumWideArgs);

	std::vector<std::string> ArgvStringsUtf8;
	ArgvStringsUtf8.reserve(NumWideArgs);
	for (int32 I = 0; I < NumWideArgs; ++I)
	{
		ArgvStringsUtf8.push_back(ConvertWideToUtf8(ArgvWide[I]));
	}
	GExePath = FPath(ArgvWide[0]);
	LocalFree(ArgvWide);

	std::vector<char*> ArgvUtf8;
	ArgvUtf8.reserve(NumWideArgs);
	for (int32 I = 0; I < argc; ++I)
	{
		ArgvUtf8.push_back(ArgvStringsUtf8[I].data());
	}
#else	// UNSYNC_PLATFORM_WINDOWS
	GExePath = FPath(argv[0]);
#endif	// UNSYNC_PLATFORM_WINDOWS

	GExePath = std::filesystem::weakly_canonical(GExePath);
	GExePath = GetAbsoluteNormalPath(GExePath);

#if UNSYNC_PLATFORM_UNIX
	std::vector<char*> ArgvUtf8;
	ArgvUtf8.reserve(argc);
	for (int32 i = 0; i < argc; ++i)
	{
		ArgvUtf8.push_back(argv[i]);
	}
#endif	// UNSYNC_PLATFORM_UNIX

	if (GBreakOnError)
	{
		return InnerMain((int)ArgvUtf8.size(), ArgvUtf8.data());
	}
	else
	{
		try
		{
			return InnerMain((int)ArgvUtf8.size(), ArgvUtf8.data());
		}
		catch (const std::system_error& E)
		{
			UNSYNC_ERROR(L"System error %d: %hs", E.code().value(), E.what());
		}
		catch (const std::exception& E)
		{
			UNSYNC_ERROR(L"Unhandled exception: %hs", E.what());
		}
	}

	return 1;
 }
