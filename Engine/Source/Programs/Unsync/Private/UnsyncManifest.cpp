// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncManifest.h"
#include "UnsyncChunking.h"
#include "UnsyncCore.h"
#include "UnsyncFile.h"
#include "UnsyncFilter.h"
#include "UnsyncHash.h"
#include "UnsyncHashTable.h"
#include "UnsyncScheduler.h"
#include "UnsyncSerialization.h"
#include "UnsyncThread.h"

namespace unsync {

FDirectoryManifestInfo
GetManifestInfo(const FDirectoryManifest& Manifest, bool bGenerateSignature)
{
	FDirectoryManifestInfo Result = {};

	Result.NumBlocks	  = 0;
	Result.NumMacroBlocks = 0;
	Result.TotalSize	  = 0;
	Result.UniqueSize	  = 0;

	THashSet<FGenericHash> UniqueBlockSet;
	THashSet<FGenericHash> UniqueMacroBlockSet;

	for (const auto& It : Manifest.Files)
	{
		const FFileManifest& File = It.second;

		for (const FGenericBlock& Block : File.Blocks)
		{
			if (UniqueBlockSet.insert(Block.HashStrong).second)
			{
				Result.UniqueSize += Block.Size;
			}
		}

		for (const FGenericBlock& Block : File.MacroBlocks)
		{
			UniqueMacroBlockSet.insert(Block.HashStrong);
		}

		Result.TotalSize += File.Size;
	}

	Result.NumBlocks	  = UniqueBlockSet.size();
	Result.NumMacroBlocks = UniqueMacroBlockSet.size();
	Result.NumFiles		  = Manifest.Files.size();
	Result.Algorithm	  = Manifest.Algorithm;

	if (bGenerateSignature)
	{
		Result.StableSignature = ComputeManifestStableSignature(Manifest);
	}

	return Result;
}

void
LogManifestInfo(ELogLevel LogLevel, const FDirectoryManifestInfo& Info)
{
	FHash160	   ManifestSignature = ToHash160(Info.StableSignature);
	const FHash160 EmptySignature	 = {};

	if (ManifestSignature != EmptySignature)
	{
		std::string SignatureHexStr = HashToHexString(ManifestSignature);
		LogPrintf(LogLevel, L"Manifest signature: %hs\n", SignatureHexStr.c_str());
	}

	LogPrintf(LogLevel, L"Chunking mode: %hs\n", ToString(Info.Algorithm.ChunkingAlgorithmId));
	LogPrintf(LogLevel, L"Weak hash: %hs\n", ToString(Info.Algorithm.WeakHashAlgorithmId));
	LogPrintf(LogLevel, L"Strong hash: %hs\n", ToString(Info.Algorithm.StrongHashAlgorithmId));
	LogPrintf(LogLevel, L"Files: %llu\n", llu(Info.NumFiles));
	LogPrintf(LogLevel, L"Blocks: %llu\n", llu(Info.NumBlocks));
	LogPrintf(LogLevel, L"Macro blocks: %llu\n", llu(Info.NumMacroBlocks));
	LogPrintf(LogLevel, L"Unique data size: %.0f MB (%llu bytes)\n", SizeMb(Info.UniqueSize), llu(Info.UniqueSize));
	LogPrintf(LogLevel, L"Total data size: %.0f MB (%llu bytes)\n", SizeMb(Info.TotalSize), llu(Info.TotalSize));

	// TODO: block size distribution histogram
	// TODO: size distribution per file extension
}

void
LogManifestInfo(ELogLevel LogLevel, const FDirectoryManifest& Manifest)
{
	FDirectoryManifestInfo Info = GetManifestInfo(Manifest);
	LogManifestInfo(LogLevel, Info);
}

void
LogManifestFiles(ELogLevel LogLevel, const FDirectoryManifest& Manifest)
{
	std::vector<std::wstring> Files;
	for (const auto& ManifestIt : Manifest.Files)
	{
		Files.push_back(ManifestIt.first);
	}

	std::sort(Files.begin(), Files.end());

	for (const std::wstring& Filename : Files)
	{
		const FFileManifest& Info = Manifest.Files.at(Filename);
		LogPrintf(LogLevel, L"%ls : %llu\n", Filename.c_str(), Info.Size);
	}
}

template<typename T>
inline auto
UpdateHashT(FBlake3Hasher& Hasher, const T& V)
{
	Hasher.Update(&V, sizeof(T));
}

void
UpdateHashBlocks(FBlake3Hasher& Hasher, const FGenericBlockArray& Blocks)
{
	for (const FGenericBlock& Block : Blocks)
	{
		UpdateHashT(Hasher, Block.Offset);
		UpdateHashT(Hasher, Block.Size);
		UpdateHashT(Hasher, Block.HashWeak);
		Hasher.Update(Block.HashStrong.Data, Block.HashStrong.Size());
	}
}

FHash256
ComputeManifestStableSignature(const FDirectoryManifest& Manifest)
{
	FBlake3Hasher Hasher;

	UpdateHashT(Hasher, Manifest.Algorithm.ChunkingAlgorithmId);
	UpdateHashT(Hasher, Manifest.Algorithm.WeakHashAlgorithmId);
	UpdateHashT(Hasher, Manifest.Algorithm.StrongHashAlgorithmId);

	std::string	 FileNameUtf8;
	for (const auto& FileIt : Manifest.Files)
	{
		const std::wstring& FileName = FileIt.first;
		const FFileManifest& FileManifest = FileIt.second;

		// Canonical unsync file paths are utf8 with unix-style separator `/`
		ConvertWideToUtf8(FileName, FileNameUtf8);
		std::replace(FileNameUtf8.begin(), FileNameUtf8.end(), '\\', '/');

		Hasher.Update(FileNameUtf8.c_str(), FileNameUtf8.length());

		UpdateHashT(Hasher, FileManifest.Mtime);
		UpdateHashT(Hasher, FileManifest.Size);
		UpdateHashT(Hasher, FileManifest.BlockSize);

		UpdateHashBlocks(Hasher, FileManifest.Blocks);
		UpdateHashBlocks(Hasher, FileManifest.MacroBlocks);
	}

	FHash256 Result = Hasher.Finalize();

	return Result;
}

void
UpdateDirectoryManifestBlocks(FDirectoryManifest& Result, const FPath& Root, const FComputeBlocksParams& Params)
{
	UNSYNC_LOG_INDENT;

	UNSYNC_ASSERT(Params.bNeedBlocks);
	UNSYNC_ASSERT(Params.BlockSize != 0);

	FTimePoint TimeBegin = TimePointNow();

	uint32 NumProcessedFiles = 0;

	const uint32		MaxConcurrentFiles = Params.bAllowStreaming ? 16 : 8;
	FSchedulerSemaphore Semaphore(*GScheduler, MaxConcurrentFiles);
	FTaskGroup			TaskGroup = GScheduler->CreateTaskGroup(&Semaphore);

	uint64 NumSkippedBlocks = 0;
	uint64 NumSkippedBytes	= 0;

	for (auto& It : Result.Files)
	{
		FFileManifest& FileManifest	  = It.second;
		const bool	   bBlockSizeOk	  = FileManifest.BlockSize == Params.BlockSize;
		const bool	   bMacroBlocksOk = Params.bNeedMacroBlocks ? !FileManifest.MacroBlocks.empty() : true;
		if (bBlockSizeOk && bMacroBlocksOk)
		{
			NumSkippedBlocks += FileManifest.Blocks.size();
			NumSkippedBytes += FileManifest.Size;

			continue;
		}

		++NumProcessedFiles;

		FPath FilePath = Root / It.first;

		UNSYNC_VERBOSE(L"Computing blocks for '%ls' (%.2f MB)", FilePath.wstring().c_str(), SizeMb(It.second.Size));
		auto BlockTask = [&FileManifest, &Params, FilePath = std::move(FilePath)]()
		{
			FNativeFile File(FilePath, EFileMode::ReadOnlyUnbuffered);
			if (File.IsValid())
			{
				FComputeBlocksResult ComputedBlocks = ComputeBlocks(File, Params);
				std::swap(FileManifest.Blocks, ComputedBlocks.Blocks);
				std::swap(FileManifest.MacroBlocks, ComputedBlocks.MacroBlocks);

				FileManifest.BlockSize = Params.BlockSize;
			}
			else
			{
				UNSYNC_FATAL(L"Failed to open file '%ls' while computing manifest blocks. %hs",
							 FilePath.wstring().c_str(),
							 FormatSystemErrorMessage(File.GetError()).c_str());
			}
		};

		if (Params.bAllowThreading)
		{
			TaskGroup.run(BlockTask);
		}
		else
		{
			BlockTask();
		}
	}

	TaskGroup.wait();

	// TODO: move manifest stat printing into a helper

	uint64 NumTotalMacroBlocks = 0;
	uint64 NumTotalBlocks	   = 0;
	uint64 NumTotalBytes	   = 0;

	for (auto& It : Result.Files)
	{
		FFileManifest& FileManifest = It.second;
		NumTotalMacroBlocks += FileManifest.MacroBlocks.size();
		NumTotalBlocks += FileManifest.Blocks.size();
		NumTotalBytes += FileManifest.Size;
	}

	NumTotalBlocks -= NumSkippedBlocks;
	NumTotalBytes -= NumSkippedBytes;

	double Duration = DurationSec(TimeBegin, TimePointNow());

	if (NumTotalBlocks == 0)
	{
		UNSYNC_VERBOSE(L"No blocks needed to be computed")
	}
	else
	{
		UNSYNC_VERBOSE(L"Total computed %lld block(s) in %.3f sec (%.3f MB, %.3f MB / sec)",
					   (long long)NumTotalBlocks,
					   Duration,
					   SizeMb(NumTotalBytes),
					   SizeMb((double(NumTotalBytes) / Duration)));

		if (NumTotalMacroBlocks)
		{
			UNSYNC_VERBOSE(L"Macro blocks: %lld", NumTotalMacroBlocks);
		}
	}
}

FDirectoryManifest
CreateDirectoryManifest(const FPath& Root, const FComputeBlocksParams& Params)
{
	UNSYNC_LOG_INDENT;

	FDirectoryManifest Result;

	Result.Algorithm = Params.Algorithm;
	Result.Version	 = FDirectoryManifest::VERSION;

	FTimePoint TimeBegin = TimePointNow();

	const uint32		MaxConcurrentFiles = Params.bAllowStreaming ? 16 : 8;
	FSchedulerSemaphore Semaphore(*GScheduler, MaxConcurrentFiles);
	FTaskGroup			TaskGroup = GScheduler->CreateTaskGroup(&Semaphore);

	std::mutex ResultMutex;
	FPath	   UnsyncDirName = ".unsync";

	FThreadLogConfig MainLogConfig;

	for (const std::filesystem::directory_entry& Dir : RecursiveDirectoryScan(Root))
	{
		if (Dir.is_directory())
		{
			continue;
		}

		FPath RelativePath = GetRelativePath(Dir.path(), Root);

		if (RelativePath.native().starts_with(UnsyncDirName.native()))
		{
			continue;
		}

		const std::wstring PathKey = RelativePath.wstring();

		UNSYNC_VERBOSE2(L"Found '%ls'", PathKey.c_str());

		FFileManifest FileManifest;

		std::filesystem::perms Perms = Dir.status().permissions();

		FileManifest.Mtime		   = ToWindowsFileTime(Dir.last_write_time());
		FileManifest.Size		   = Dir.file_size();
		FileManifest.CurrentPath   = Dir.path();
		FileManifest.BlockSize	   = Params.BlockSize;
		FileManifest.bReadOnly	   = IsReadOnly(Perms);
		FileManifest.bIsExecutable = IsExecutable(Perms);

		{
			std::lock_guard<std::mutex> LockGuard(ResultMutex);
			Result.Files[PathKey] = std::move(FileManifest);
		}

		if (Params.bNeedBlocks && Params.BlockSize)
		{
			FPath FilePath = Root / RelativePath;

			TaskGroup.run(
				[&ResultMutex, &Result, &MainLogConfig, FilePath = std::move(FilePath), Key = std::move(PathKey), &Params]()
				{
					auto File = std::make_shared<FNativeFile>(FilePath, EFileMode::ReadOnlyUnbuffered);

					if (File->IsValid())
					{
						{
							// Log from worker using parent thread log config
							FThreadLogConfig::FScope LogConfigScope(MainLogConfig);
							UNSYNC_VERBOSE(L"Computing blocks for '%ls' (%.2f MB)",
										   FilePath.wstring().c_str(),
										   double(File->GetSize()) / (1 << 20));
						}

						FComputeBlocksResult ComputedBlocks = ComputeBlocks(*File, Params);

						std::lock_guard<std::mutex> LockGuard(ResultMutex);

						std::swap(Result.Files[Key].Blocks, ComputedBlocks.Blocks);
						std::swap(Result.Files[Key].MacroBlocks, ComputedBlocks.MacroBlocks);
					}
					else
					{
						UNSYNC_FATAL(L"Failed to open file '%ls' while computing manifest blocks. %hs",
										FilePath.wstring().c_str(),
										FormatSystemErrorMessage(File->GetError()).c_str());
					}
				});
		}
	}

	TaskGroup.wait();

	if (Params.bNeedBlocks && Params.BlockSize)
	{
		// TODO: move manifest stat printing into a helper

		uint64 NumTotalMacroBlocks = 0;
		uint64 NumTotalBlocks	   = 0;
		uint64 NumTotalBytes	   = 0;

		for (auto& It : Result.Files)
		{
			FFileManifest& FileManifest = It.second;
			NumTotalMacroBlocks += FileManifest.MacroBlocks.size();
			NumTotalBlocks += FileManifest.Blocks.size();
			NumTotalBytes += FileManifest.Size;
		}

		double Duration = DurationSec(TimeBegin, TimePointNow());
		UNSYNC_VERBOSE(L"Total computed %lld block(s) in %.3f sec (%.3f MB, %.3f MB / sec)",
					   (long long)NumTotalBlocks,
					   Duration,
					   SizeMb(NumTotalBytes),
					   SizeMb((double(NumTotalBytes) / Duration)));
		if (NumTotalMacroBlocks)
		{
			UNSYNC_VERBOSE(L"Macro blocks: %lld", NumTotalMacroBlocks);
		}
	}

	return Result;
}

void
MoveCompatibleManifestBlocks(FDirectoryManifest& Manifest, FDirectoryManifest&& DonorManifest)
{
	if (!AlgorithmOptionsCompatible(Manifest.Algorithm, Manifest.Algorithm))
	{
		UNSYNC_ERROR(L"MoveCompatibleManifestBlocks() requires that manifests use compatible block generation settings");
		return;
	}

	for (auto& ManifestFileEntry : Manifest.Files)
	{
		auto DonorManifestFileEntry = DonorManifest.Files.find(ManifestFileEntry.first);
		if (DonorManifestFileEntry == DonorManifest.Files.end())
		{
			continue;
		}

		FFileManifest& ResultEntry = ManifestFileEntry.second;
		FFileManifest& DonorEntry  = DonorManifestFileEntry->second;

		if (ResultEntry.Mtime == DonorEntry.Mtime && ResultEntry.Size == DonorEntry.Size)
		{
			ResultEntry.Blocks		= std::move(DonorEntry.Blocks);
			ResultEntry.MacroBlocks = std::move(DonorEntry.MacroBlocks);
			ResultEntry.BlockSize	= DonorEntry.BlockSize;
		}
	}
}

FDirectoryManifest
CreateDirectoryManifestIncremental(const FPath& Root, const FComputeBlocksParams& InParams)
{
	FComputeBlocksParams Params = InParams;

	FPath ManifestRoot			= Root / ".unsync";
	FPath DirectoryManifestPath = ManifestRoot / "manifest.bin";

	FDirectoryManifest OldManifest;
	const bool		   bExistingManifestLoaded = LoadDirectoryManifest(OldManifest, Root, DirectoryManifestPath);

	// Inherit algorithm options from the existing manifest
	if (bExistingManifestLoaded)
	{
		Params.Algorithm = OldManifest.Algorithm;
	}

	// Scan the input directory and gather file metadata, without generating blocks
	FComputeBlocksParams LightweightManifestParams = Params;
	LightweightManifestParams.bNeedBlocks		   = false;
	LightweightManifestParams.BlockSize			   = 0;

	FDirectoryManifest NewManifest = CreateDirectoryManifest(Root, LightweightManifestParams);

	// Copy file blocks from old manifest, if possible
	MoveCompatibleManifestBlocks(NewManifest, std::move(OldManifest));

	// Generate blocks for changed or new files
	UpdateDirectoryManifestBlocks(NewManifest, Root, Params);

	return NewManifest;
}

bool
AlgorithmOptionsCompatible(const FAlgorithmOptions& A, const FAlgorithmOptions& B)
{
	return A.StrongHashAlgorithmId == B.StrongHashAlgorithmId && B.WeakHashAlgorithmId == B.WeakHashAlgorithmId &&
		   B.ChunkingAlgorithmId == B.ChunkingAlgorithmId;
}

bool
LoadOrCreateDirectoryManifest(FDirectoryManifest& Result, const FPath& Root, const FComputeBlocksParams& Params)
{
	UNSYNC_LOG_INDENT;

	FPath ManifestRoot			= Root / ".unsync";
	FPath DirectoryManifestPath = ManifestRoot / "manifest.bin";

	FDirectoryManifest OldDirectoryManifest;
	FDirectoryManifest NewDirectoryManifest;

	const bool bManifestFileExists = PathExists(DirectoryManifestPath);
	if (!bManifestFileExists)
	{
		UNSYNC_VERBOSE(L"Manifest file '%ls' does not exist", DirectoryManifestPath.wstring().c_str());
	}

	const bool bExistingManifestLoaded = bManifestFileExists && LoadDirectoryManifest(OldDirectoryManifest, Root, DirectoryManifestPath);
	const bool bExistingManifestCompatible = AlgorithmOptionsCompatible(OldDirectoryManifest.Algorithm, Params.Algorithm);

	if (bExistingManifestLoaded && bExistingManifestCompatible)
	{
		UNSYNC_VERBOSE(L"Loaded existing manifest from '%ls'", DirectoryManifestPath.wstring().c_str());

		UNSYNC_VERBOSE(L"Creating lightweight manifest for '%ls'", Root.wstring().c_str());
		{
			// Verify that manifests match in dry run mode.
			// Otherwise just do a quick manifest generation, without file blocks.
			FComputeBlocksParams NewParams = Params;
			if (GDryRun)
			{
				NewParams.bNeedBlocks = true;
			}
			else
			{
				NewParams.bNeedBlocks = false;
				NewParams.BlockSize	  = 0;
			}

			NewDirectoryManifest = CreateDirectoryManifest(Root, NewParams);
		}

		UNSYNC_VERBOSE(L"Comparing manifests");
		for (const auto& OldManifestIt : OldDirectoryManifest.Files)
		{
			auto NewManifestIt = NewDirectoryManifest.Files.find(OldManifestIt.first);
			if (NewManifestIt != NewDirectoryManifest.Files.end())
			{
				const FFileManifest& OldFileManifest = OldManifestIt.second;
				FFileManifest&		 NewFileManifest = NewManifestIt->second;
				if (NewFileManifest.Size == OldFileManifest.Size && NewFileManifest.Mtime == OldFileManifest.Mtime)
				{
					if (NewFileManifest.BlockSize)
					{
						UNSYNC_ASSERT(NewFileManifest.BlockSize == OldFileManifest.BlockSize);
						UNSYNC_ASSERT(NewFileManifest.Blocks.size() == OldFileManifest.Blocks.size());
						for (uint64 I = 0; I < NewFileManifest.Blocks.size(); ++I)
						{
							const FGenericBlock& NewBlock = NewFileManifest.Blocks[I];
							const FGenericBlock& OldBlock = OldFileManifest.Blocks[I];
							UNSYNC_ASSERT(NewBlock.Offset == OldBlock.Offset);
							UNSYNC_ASSERT(NewBlock.Size == OldBlock.Size);
							UNSYNC_ASSERT(NewBlock.HashWeak == OldBlock.HashWeak);
							UNSYNC_ASSERT(NewBlock.HashStrong == OldBlock.HashStrong);
						}
					}
					else
					{
						NewFileManifest.BlockSize	= OldFileManifest.BlockSize;
						NewFileManifest.Blocks		= std::move(OldFileManifest.Blocks);
						NewFileManifest.MacroBlocks = std::move(OldFileManifest.MacroBlocks);
					}
				}
			}
		}

		if (Params.bNeedBlocks && Params.BlockSize)
		{
			UNSYNC_VERBOSE(L"Updating manifest blocks");
			UpdateDirectoryManifestBlocks(NewDirectoryManifest, Root, Params);
		}
	}
	else
	{
		UNSYNC_VERBOSE(L"Creating manifest for '%ls'", Root.wstring().c_str());
		NewDirectoryManifest = CreateDirectoryManifest(Root, Params);
	}

	std::swap(Result, NewDirectoryManifest);

	return true;
}

FHash256
ComputeSerializedManifestHash(const FDirectoryManifest& Manifest)
{
	FBuffer			 ManifestBuffer;
	FVectorStreamOut ManifestStream(ManifestBuffer);
	bool			 bSerializedOk = SaveDirectoryManifest(Manifest, ManifestStream);
	UNSYNC_ASSERT(bSerializedOk);
	return HashBlake3Bytes<FHash256>(ManifestBuffer.Data(), ManifestBuffer.Size());
}

FHash160
ComputeSerializedManifestHash160(const FDirectoryManifest& Manifest)
{
	return ToHash160(ComputeSerializedManifestHash(Manifest));
}

bool
MergeManifests(FDirectoryManifest& Existing, const FDirectoryManifest& Other, bool bCaseSensitive)
{
	if (!Existing.IsValid())
	{
		Existing = Other;
		return true;
	}

	if (!AlgorithmOptionsCompatible(Existing.Algorithm, Other.Algorithm))
	{
		UNSYNC_ERROR("Trying to merge incompatible manifests (diff algorithm options do not match)");
		return false;
	}

	if (bCaseSensitive)
	{
		// Trivial case: just replace existing entries
		for (const auto& OtherFile : Other.Files)
		{
			Existing.Files[OtherFile.first] = OtherFile.second;
		}
	}
	else
	{
		// Lookup table of lowercase -> original file name used to replace conflicting entries on non-case-sensitive filesystems
		// TODO: Could potentially add case-sensitive/insensitive entry lookup helper functions to FDirectoryManifest itself in the future
		std::unordered_map<std::wstring, std::wstring> ExistingFileNamesLowerCase;

		for (auto& ExistingEntry : Existing.Files)
		{
			std::wstring FileNameLowerCase = StringToLower(ExistingEntry.first);
			ExistingFileNamesLowerCase.insert(std::pair<std::wstring, std::wstring>(FileNameLowerCase, ExistingEntry.first));
		}

		for (const auto& OtherFile : Other.Files)
		{
			std::wstring OtherNameLowerCase = StringToLower(OtherFile.first);
			auto		 LowerCaseEntry		= ExistingFileNamesLowerCase.find(OtherNameLowerCase);
			if (LowerCaseEntry != ExistingFileNamesLowerCase.end())
			{
				// Remove file with conflicting case and add entry from the other manifest instead
				const std::wstring& ExistingNameOriginalCase = LowerCaseEntry->second;
				Existing.Files.erase(ExistingNameOriginalCase);

				// Update the lookup table entry to refer to the name we're about to insert
				ExistingFileNamesLowerCase[LowerCaseEntry->first] = OtherFile.first;
			}

			Existing.Files[OtherFile.first] = OtherFile.second;
		}
	}

	return true;
}

}  // namespace unsync
