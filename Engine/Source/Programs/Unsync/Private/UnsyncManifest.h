// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncHash.h"
#include "UnsyncProtocol.h"

#include <map>

namespace unsync {

struct FComputeBlocksParams;
struct FSyncFilter;

struct FFileManifest
{
	uint64 Mtime	 = 0;
	uint64 Size		 = 0;
	uint32 BlockSize = 0;

	// runtime-only data:
	FGenericBlockArray Blocks;
	FGenericBlockArray MacroBlocks;
	FPath			   CurrentPath;
	std::string		   RevisionControlIdentity;
	bool			   bReadOnly = false;
	bool               bIsExecutable = false;

	bool IsValid() const { return Mtime != 0 && Size != 0; }
};

struct FAlgorithmOptionsV5
{
	static constexpr uint64 MAGIC	= 0x96FF3B56283EC7DBull;
	static constexpr uint64 VERSION = 1;

	EChunkingAlgorithmID   ChunkingAlgorithmId	 = EChunkingAlgorithmID::VariableBlocks;
	EWeakHashAlgorithmID   WeakHashAlgorithmId	 = EWeakHashAlgorithmID::BuzHash;
	EStrongHashAlgorithmID StrongHashAlgorithmId = EStrongHashAlgorithmID::Blake3_128;
};

struct FAlgorithmOptions : FAlgorithmOptionsV5
{
};

struct FDirectoryManifest
{
	enum EVersions : uint64 {
		Invalid = 0,
		V1_Unsupported,
		V2_Unsupported,
		V3_Unsupported,
		V4,					  // unsync v1.0.7
		V5,					  // unsync v1.0.8: added options
		V6_VariableHash,	  // unsync v1.0.32-iohash: added variable size hashes, moved options section before files
		V7_OptionalSections,  // unsync v1.0.32-iohash

		Latest = V7_OptionalSections
	};

	static constexpr uint64 MAGIC	= 0x80F4CC2A414F4E41ull;
	static constexpr uint64 VERSION = EVersions::Latest;

	// wide string at runtime, utf8 serialized
	// TODO: keep paths in canonical form (utf-8, unix-style separators)
	using FFileMap = std::map<std::wstring, FFileManifest>; // regular map to keep files in a deterministic order
	FFileMap Files;

	// runtime data
	FAlgorithmOptions Algorithm = {};
	uint64			  Version	= 0;

	bool bHasFileRevisionControl = false;

	std::vector<FPackReference> PackReferences;

	bool IsValid() const { return Version != EVersions::Invalid; }
};

struct FDirectoryManifestInfo
{
	uint64	 TotalSize		 = 0;
	uint64	 UniqueSize		 = 0;
	uint64	 NumBlocks		 = 0;
	uint64	 NumMacroBlocks	 = 0;
	uint64	 NumFiles		 = 0;
	FHash256 StableSignature = {};

	FAlgorithmOptions Algorithm = {};
};

FDirectoryManifestInfo GetManifestInfo(const FDirectoryManifest& Manifest, bool bGenerateSignature = true);
void				   LogManifestInfo(ELogLevel LogLevel, const FDirectoryManifest& Manifest);
void				   LogManifestInfo(ELogLevel LogLevel, const FDirectoryManifestInfo& Info);
void				   LogManifestFiles(ELogLevel LogLevel, const FDirectoryManifest& Manifest);
void				   LogManifestFiles(ELogLevel LogLevel, const FDirectoryManifestInfo& Info);

// Computes a Blake3 hash of the manifest blocks and file metadata, ignoring any other metadata.
// Files are processed in sorted order, with file names treated as utf-8.
// This produces a relatively stable manifest key that does not depend on the serialization differences between versions.
FHash256 ComputeManifestStableSignature(const FDirectoryManifest& Manifest);

// Computes a Blake3 hash of the serialized manifest data
FHash256 ComputeSerializedManifestHash(const FDirectoryManifest& Manifest);

void			   UpdateDirectoryManifestBlocks(FDirectoryManifest& Result, const FPath& Root, const FComputeBlocksParams& Params);
FDirectoryManifest CreateDirectoryManifest(const FPath& Root, const FComputeBlocksParams& Params);
FDirectoryManifest CreateDirectoryManifestIncremental(const FPath& Root, const FComputeBlocksParams& Params);
bool			   LoadOrCreateDirectoryManifest(FDirectoryManifest& Result, const FPath& Root, const FComputeBlocksParams& Params);
bool			   MergeManifests(FDirectoryManifest& Existing, const FDirectoryManifest& Other, bool bCaseSensitive);
void			   MoveCompatibleManifestBlocks(FDirectoryManifest& Manifest, FDirectoryManifest&& DonorManifest);
bool			   AlgorithmOptionsCompatible(const FAlgorithmOptions& A, const FAlgorithmOptions& B);

}  // namespace unsync
