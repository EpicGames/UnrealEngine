// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncMemory.h"
#include "UnsyncLog.h"
#include "UnsyncUtil.h"

#if UNSYNC_PLATFORM_WINDOWS
#	include <Windows.h>
#endif	// UNSYNC_PLATFORM_WINDOWS

#if UNSYNC_USE_DEBUG_HEAP
#	include "ig-debugheap/DebugHeap.h"
#endif	// UNSYNC_USE_DEBUG_HEAP

#if __has_include(<malloc.h>)
#	include <malloc.h>
#endif

#include <atomic>
#include <mutex>

#ifdef __GNUC__
inline void*
_mm_malloc(size_t s, size_t a)
{
	void* res		 = nullptr;
	int	  error_code = posix_memalign(&res, a, s);
	if (error_code != 0)
	{
		using namespace unsync;
		UNSYNC_FATAL(L"Memory allocation failed. Error code: %d.", error_code);
	}
	return res;
}
#	define _mm_free free
#endif	//__GNUC__

struct DebugHeap;

#if defined(__has_feature)
#	if __has_feature(thread_sanitizer)
#		define UNSYNC_THREAD_SANITIZER 1
#	endif
#endif

#ifndef UNSYNC_THREAD_SANITIZER
#	define UNSYNC_THREAD_SANITIZER 0
#endif

namespace unsync {

#define UNSYNC_OVERRIDE_GLOBAL_NEW_DELETE	 (!UNSYNC_THREAD_SANITIZER)
#define UNSYNC_DEBUG_MALLOC_CANARY			 1
#define UNSYNC_DEBUG_HEAP_MINIMUM_ALLOC_SIZE 4096

static EMallocType GMallocType = EMallocType::Invalid;

void* UnsyncDebugMalloc(size_t Size);
void  UnsyncDebugFree(void* Ptr);

static DebugHeap* GDebugHeap = nullptr;
static std::mutex GDebugHeapMutex;

void
UnsyncMallocInit(EMallocType MallocType)
{
	UNSYNC_ASSERT(GMallocType == EMallocType::Invalid);

#if UNSYNC_USE_DEBUG_HEAP
	GMallocType = MallocType;
	if (MallocType == EMallocType::Debug)
	{
		// Use a huge virtual memory chunk for debug heap.
		// This is way more than necessary to run unsync.
		// More space reduces address reuse and increases accuracy.
		GDebugHeap = DebugHeapInit(512_GB);
	}
#else
	UNSYNC_UNUSED(MallocType);
	GMallocType = EMallocType::Default;
#endif	// UNSYNC_USE_DEBUG_HEAP
}

void*
UnsyncMalloc(size_t Size)
{
	UNSYNC_ASSERT(GMallocType != EMallocType::Invalid);

	if (GMallocType == EMallocType::Debug && Size >= UNSYNC_DEBUG_HEAP_MINIMUM_ALLOC_SIZE)
	{
		return UnsyncDebugMalloc(Size);
	}
	else
	{
		return _mm_malloc(Size, UNSYNC_MALLOC_ALIGNMENT);
	}
}

bool
DebugHeapOwns(void* Ptr)
{
#if UNSYNC_USE_DEBUG_HEAP
	std::lock_guard<std::mutex> LockGuard(GDebugHeapMutex);
	return DebugHeapOwns(GDebugHeap, Ptr);
#else
	UNSYNC_UNUSED(Ptr);
	return false;
#endif
}

void
UnsyncFree(void* Ptr)
{
	if (Ptr == nullptr)
	{
		return;
	}

	UNSYNC_ASSERT(GMallocType != EMallocType::Invalid);

	if (GMallocType == EMallocType::Debug && DebugHeapOwns(Ptr))
	{
		UnsyncDebugFree(Ptr);
	}
	else
	{
		_mm_free(Ptr);
	}
}

static constexpr uint32 CANARY_VALUE_BEFORE = 0x9b46750e;
static constexpr uint32 CANARY_VALUE_AFTER	= 0x326b54a6;
struct FDebugMemoryHeader
{
	uint64 Size	  = 0;
	uint32 Canary = 0;
	uint32 Serial = 0;
};
static_assert(sizeof(FDebugMemoryHeader) == UNSYNC_MALLOC_ALIGNMENT);

static std::atomic_uint32_t GMallocCounter;

#if UNSYNC_USE_DEBUG_HEAP
void*
UnsyncDebugMalloc(size_t Size)
{
#	if UNSYNC_DEBUG_MALLOC_CANARY

	size_t TotalSize = Size + sizeof(FDebugMemoryHeader) * 2;  // requested size + header + footer

	uint8* AllocatedBlock = nullptr;

	{
		std::lock_guard<std::mutex> LockGuard(GDebugHeapMutex);
		AllocatedBlock = (uint8*)DebugHeapAllocate(GDebugHeap, TotalSize, UNSYNC_MALLOC_ALIGNMENT);
	}

	UNSYNC_ASSERT(AllocatedBlock);

	uint8* Ptr = AllocatedBlock + sizeof(FDebugMemoryHeader);

	FDebugMemoryHeader& Header = *(FDebugMemoryHeader*)(Ptr - sizeof(FDebugMemoryHeader));

	Header.Size	  = Size;
	Header.Canary = CANARY_VALUE_BEFORE;
	Header.Serial = ++GMallocCounter;

	FDebugMemoryHeader& Footer = *(FDebugMemoryHeader*)(Ptr + Header.Size);

	Footer		  = Header;
	Footer.Canary = CANARY_VALUE_AFTER;

#		if 0
	memset(ptr, 0xCD, size);
#		endif

	return Ptr;

#	else  // UNSYNC_DEBUG_MALLOC_CANARY

	std::lock_guard<std::mutex> lock_guard(g_debug_heap_mutex);
	return DebugHeapAllocate(g_debug_heap, size, UNSYNC_MALLOC_ALIGNMENT);

#	endif	// UNSYNC_DEBUG_MALLOC_CANARY
}

void
UnsyncDebugFree(void* InPtr)
{
#	if UNSYNC_DEBUG_MALLOC_CANARY

	uint8* Ptr			  = (uint8*)InPtr;
	uint8* AllocatedBlock = Ptr - sizeof(FDebugMemoryHeader);

	FDebugMemoryHeader& Header = *(FDebugMemoryHeader*)(Ptr - sizeof(FDebugMemoryHeader));
	UNSYNC_ASSERT(Header.Canary == CANARY_VALUE_BEFORE);

	FDebugMemoryHeader& Footer = *(FDebugMemoryHeader*)(Ptr + Header.Size);
	UNSYNC_ASSERT(Footer.Size == Header.Size);
	UNSYNC_ASSERT(Footer.Canary == CANARY_VALUE_AFTER);
	UNSYNC_ASSERT(Footer.Serial == Header.Serial);

#		if 0
	size_t TotalSize = header.size + sizeof(FDebugMemoryHeader) * 2;
	memset(allocated_block, 0xDD, TotalSize);
#		else
	memset(&Header, 0xDD, sizeof(Header));
	memset(&Footer, 0xDD, sizeof(Footer));
#		endif

	std::lock_guard<std::mutex> LockGuard(GDebugHeapMutex);

	DebugHeapFree(GDebugHeap, AllocatedBlock);

#	else  // UNSYNC_DEBUG_MALLOC_CANARY

	std::lock_guard<std::mutex> lock_guard(g_debug_heap_mutex);
	DebugHeapFree(g_debug_heap, in_ptr);

#	endif	// UNSYNC_DEBUG_MALLOC_CANARY
}
#else  // UNSYNC_USE_DEBUG_HEAP

void* UnsyncDebugMalloc(size_t)
{
	return nullptr;
}

void
UnsyncDebugFree(void*)
{
}

#endif	// UNSYNC_USE_DEBUG_HEAP


#if UNSYNC_PLATFORM_WINDOWS
bool
QueryMemoryInfo(FSystemMemoryInfo& OutMemoryInfo)
{
	OutMemoryInfo = {};

	ULONGLONG TotalMemoryInKB = 0;

	if (!GetPhysicallyInstalledSystemMemory(&TotalMemoryInKB))
	{
		return false;
	}

	OutMemoryInfo.InstalledPhysicalMemory = uint64(TotalMemoryInKB) << 10ull;

	return true;
}
#else
bool
QueryMemoryInfo(FSystemMemoryInfo& OutMemoryInfo)
{
	OutMemoryInfo = {};
	return false;
}
#endif

}  // namespace unsync

#if UNSYNC_OVERRIDE_GLOBAL_NEW_DELETE

namespace unsync {
static constexpr size_t MAX_BOOTSTRAP_MEMORY_SIZE = 16_MB;
static uint8			GBootstrapMemory[MAX_BOOTSTRAP_MEMORY_SIZE];
static size_t			GBootstrapMemoryCursor = 0;

static bool
IsBootstrapPtr(void* InPtr)
{
	using namespace unsync;

	uint8* Ptr = (uint8*)InPtr;
	return Ptr >= GBootstrapMemory && Ptr < (GBootstrapMemory + MAX_BOOTSTRAP_MEMORY_SIZE);
}
}  // namespace unsync

#	if defined(_MSC_VER)
_NODISCARD _Ret_notnull_
_Post_writable_byte_size_(Size)
_VCRT_ALLOCATOR
void* __CRTDECL
operator new(size_t Size)
#	else
void*
operator new(size_t Size)
#	endif
{
	using namespace unsync;

	if (GMallocType == EMallocType::Invalid)
	{
		size_t						AlignedSize = AlignUpToMultiplePow2(std::max(Size, size_t(1)), UNSYNC_MALLOC_ALIGNMENT);
		static std::mutex			GBootstrapMemoryMutex;
		std::lock_guard<std::mutex> LockGuard(GBootstrapMemoryMutex);
		if (GBootstrapMemoryCursor + AlignedSize > MAX_BOOTSTRAP_MEMORY_SIZE)
		{
			UNSYNC_FATAL(L"Out of memory in bootstrap allocator!");
			return UnsyncMalloc(Size);
		}
		uint8* Result = GBootstrapMemory + GBootstrapMemoryCursor;
		GBootstrapMemoryCursor += AlignedSize;
		return Result;
	}
	else
	{
		return UnsyncMalloc(Size);
	}
}

void
operator delete(void* Ptr)
#ifdef __GNUC__
noexcept
#endif
{
	using namespace unsync;

	if (!IsBootstrapPtr(Ptr))
	{
		return UnsyncFree(Ptr);
	}
}

#endif	// UNSYNC_OVERRIDE_GLOBAL_NEW_DELETE
