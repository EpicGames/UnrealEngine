// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"

namespace unsync {

#ifndef UNSYNC_USE_DEBUG_HEAP
#	define UNSYNC_USE_DEBUG_HEAP 0
#endif

static constexpr size_t UNSYNC_MALLOC_ALIGNMENT = 16;

enum class EMallocType
{
	Invalid,
	Default,
	Debug
};

void  UnsyncMallocInit(EMallocType MallocType);
void* UnsyncMalloc(size_t Size);
void  UnsyncFree(void* Ptr);

struct FSystemMemoryInfo
{
	uint64 InstalledPhysicalMemory = 0;
};

// Queries various memory properties from the OS, returns false if information is not available
bool QueryMemoryInfo(FSystemMemoryInfo& OutMemoryInfo);

}  // namespace unsync
