// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncMiniCb.h"
#include "UnsyncHash.h"

namespace unsync {

void
TestMiniCb()
{
	{
		std::vector<FHash160> ManifestPartHashes;
		for (uint32 I = 0; I < 1000; ++I)
		{
			ManifestPartHashes.push_back(HashBlake3String<FHash160>("hello"));
		}

		FMiniCbWriter Writer;
		Writer.AddBinaryAttachmentArray(ManifestPartHashes, "needs");
		Writer.Finalize();
		FBufferView ResultBuffer = Writer.GetBufferView();
		UNSYNC_ASSERT(ResultBuffer.Size == 20017);

		FMiniCbReader	 RootReader(ResultBuffer);
		FMiniCbFieldView Object = RootReader.Child();
		UNSYNC_ASSERT(Object.Type == EMiniCbFieldType::Object);

		FMiniCbReader FieldReader(Object);

		FMiniCbFieldView Field = FieldReader.Child();
		UNSYNC_ASSERT(Field.Type == EMiniCbFieldType::UniformArray);
		UNSYNC_ASSERT(Field.Name == "needs");
		UNSYNC_ASSERT(Field.UniformArrayItemCount == ManifestPartHashes.size());
		UNSYNC_ASSERT(Field.UniformArrayByteCount == 20 * ManifestPartHashes.size());
		UNSYNC_ASSERT(Field.UniformArrayItemType == EMiniCbFieldType::BinaryAttachment);
		auto FieldValues = Field.GetUniformArray<FHash160>();
		for (uint64 I = 0; I < ManifestPartHashes.size(); ++I)
		{
			UNSYNC_ASSERT(FieldValues[I] == ManifestPartHashes[I]);
		}
	}

	{
		std::vector<FHash160> ManifestPartHashes;
		ManifestPartHashes.push_back(HashBlake3String<FHash160>("hello"));

		FMiniCbWriter Writer;
		Writer.AddBinaryAttachment(ManifestPartHashes[0], "unsync.manifest");
		Writer.AddHashArray(ManifestPartHashes, "unsync.manifest.parts");
		Writer.Finalize();

		FBufferView ResultBuffer = Writer.GetBufferView();
		UNSYNC_ASSERT(ResultBuffer.Size == 85);

		FMiniCbReader	 RootReader(ResultBuffer);
		FMiniCbFieldView Object = RootReader.Child();
		UNSYNC_ASSERT(Object.Type == EMiniCbFieldType::Object);

		FMiniCbReader FieldReader(Object);

		FMiniCbFieldView Field1 = FieldReader.Child();
		UNSYNC_ASSERT(Field1.Type == EMiniCbFieldType::BinaryAttachment);
		UNSYNC_ASSERT(Field1.Name == "unsync.manifest");

		FMiniCbFieldView Field2 = FieldReader.Child();
		UNSYNC_ASSERT(Field2.Type == EMiniCbFieldType::UniformArray);
		UNSYNC_ASSERT(Field2.Name == "unsync.manifest.parts");
		UNSYNC_ASSERT(Field2.UniformArrayItemCount == 1);
		UNSYNC_ASSERT(Field2.UniformArrayByteCount == 20);
		UNSYNC_ASSERT(Field2.UniformArrayItemType == EMiniCbFieldType::Hash);
	}
}

}  // namespace unsync
