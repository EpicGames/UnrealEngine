// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncBuffer.h"
#include "UnsyncUtil.h"
#include "UnsyncVarInt.h"

#include <span>
#include <string>
#include <string_view>

namespace unsync {

// Minimal partial implementation of Unreal Engine Compact Binary.
// (just barely enough to talk to Jupiter)

enum class EMiniCbFieldType : uint8
{
	None			 = 0x00,
	Null			 = 0x01,
	Object			 = 0x02,
	UniformObject	 = 0x03,
	Array			 = 0x04,
	UniformArray	 = 0x05,
	Binary			 = 0x06,
	String			 = 0x07,
	IntegerPositive	 = 0x08,
	IntegerNegative	 = 0x09,
	Float32			 = 0x0a,
	Float64			 = 0x0b,
	BoolFalse		 = 0x0c,
	BoolTrue		 = 0x0d,
	ObjectAttachment = 0x0e,
	BinaryAttachment = 0x0f,
	Hash			 = 0x10,
	Uuid			 = 0x11,
	DateTime		 = 0x12,
	TimeSpan		 = 0x13,
	ObjectId		 = 0x14,
	CustomById		 = 0x1e,
	CustomByName	 = 0x1f,

	// Flags
	Reserved	 = 0x20,
	HasFieldType = 0x40,
	HasFieldName = 0x80,
};

inline bool
HasName(EMiniCbFieldType Type)
{
	return (uint8(Type) & uint8(EMiniCbFieldType::HasFieldName)) != 0;
}

struct FMiniCbPayloadSize
{
	uint64 DataSize	  = 0;
	uint64 HeaderSize = 0;
};

inline FMiniCbPayloadSize
GetPayloadSize(EMiniCbFieldType Type, const uint8* Payload = nullptr)
{
	switch (Type)
	{
		default:
			UNSYNC_FATAL(L"Field type %d is not implemented", uint8(Type));
			return {0, 0};
		case EMiniCbFieldType::None:
			return {0, 0};
		case EMiniCbFieldType::Null:
			return {1, 0};
		case EMiniCbFieldType::Hash:
		case EMiniCbFieldType::BinaryAttachment:
			return {20, 0};
		case EMiniCbFieldType::Object:
		case EMiniCbFieldType::UniformObject:
		case EMiniCbFieldType::Array:
		case EMiniCbFieldType::UniformArray:
		case EMiniCbFieldType::Binary:
		case EMiniCbFieldType::String:
		case EMiniCbFieldType::CustomByName:
		case EMiniCbFieldType::CustomById:
			{
				uint32		 SizeFieldByteCount = 0;
				const uint64 PayloadSize		= ReadVarUint(Payload, SizeFieldByteCount);
				return {PayloadSize, SizeFieldByteCount};
			}
	}
}

struct FMiniCbWriter
{
	static const uint64 MAX_OBJECT_HEADER_SIZE = 9 + 1;	 // maximum var-uint size + object type identifier

	FMiniCbWriter()
	{
		// allocate some space for the header
		Output.Resize(MAX_OBJECT_HEADER_SIZE);
		memset(Output.Data(), 0, MAX_OBJECT_HEADER_SIZE);
	}

	void AddNull() { Output.PushBack(uint8(EMiniCbFieldType::Null)); }

	void AddBinaryAttachment(FHash160 Hash, std::string_view Name = {})	 // TODO: add field name
	{
		WriteFieldHeader(EMiniCbFieldType::BinaryAttachment, Name);
		Output.Append(Hash.Data, sizeof(Hash.Data));

		NumAttachments++;
	}

	void AddHash(FHash160 Hash, std::string_view Name = {})	 // TODO: add field name
	{
		WriteFieldHeader(EMiniCbFieldType::Hash, Name);
		Output.Append(Hash.Data, sizeof(Hash.Data));
	}

	void AddHashArray(std::span<FHash160> Hashes, std::string_view Name = {})
	{
		WriteUniformArrayHeader(EMiniCbFieldType::Hash, Hashes.size_bytes(), Hashes.size(), Name);
		Output.Append((const uint8*)Hashes.data(), Hashes.size_bytes());
	}

	void AddBinaryAttachmentArray(std::span<FHash160> Hashes, std::string_view Name = {})
	{
		WriteUniformArrayHeader(EMiniCbFieldType::BinaryAttachment, Hashes.size_bytes(), Hashes.size(), Name);
		Output.Append((const uint8*)Hashes.data(), Hashes.size_bytes());
	}

	void Finalize()
	{
		uint64 PayloadSize		  = Output.Size() - MAX_OBJECT_HEADER_SIZE;
		uint32 SizeFieldByteCount = MeasureVarUint(PayloadSize);
		HeaderOffset			  = MAX_OBJECT_HEADER_SIZE - SizeFieldByteCount - 1;
		Output[HeaderOffset]	  = uint8(EMiniCbFieldType::Object);
		uint32 WroteBytes		  = WriteVarUint(PayloadSize, &Output[HeaderOffset + 1]);

		UNSYNC_ASSERT(WroteBytes == SizeFieldByteCount);
	}

	const uint8* Data() const
	{
		UNSYNC_ASSERT(HeaderOffset != ~0ull);
		return Output.Data() + HeaderOffset;
	}
	const uint64 Size() const
	{
		UNSYNC_ASSERT(HeaderOffset != ~0ull);
		return Output.Size() - HeaderOffset;
	}

	FBufferView GetBufferView() const { return FBufferView{Data(), Size()}; }

	uint64 GetNumAttachments() const { return NumAttachments; }

private:
	void WriteUint(uint64 V)
	{
		uint8  Buffer[16]  = {};
		uint32 EncodedSize = MeasureVarUint(V);
		UNSYNC_ASSERT(EncodedSize <= sizeof(Buffer));
		WriteVarUint(V, Buffer);
		Output.Append(Buffer, EncodedSize);
	}

	void WriteString(std::string_view Str)
	{
		WriteUint(Str.length());
		Output.Append((const uint8*)Str.data(), Str.length());
	}

	void WriteFieldHeader(EMiniCbFieldType Type, std::string_view Name = {})
	{
		uint8 FieldType = uint8(Type);

		if (!Name.empty())
		{
			FieldType |= uint8(EMiniCbFieldType::HasFieldName);
		}

		Output.PushBack(FieldType);

		if (!Name.empty())
		{
			WriteString(Name);
		}
	}

	void WriteUniformArrayHeader(EMiniCbFieldType ElemType, uint64 SizeInBytes, uint64 ItemCount, std::string_view Name = {})
	{
		uint32 ItemCountSize = MeasureVarUint(ItemCount);
		uint64 PayloadSize	 = SizeInBytes + ItemCountSize + 1;	 // full size of the field, including 1 byte for element type

		WriteFieldHeader(EMiniCbFieldType::UniformArray, Name);

		WriteUint(PayloadSize);
		WriteUint(ItemCount);

		Output.PushBack(uint8(ElemType));
	}

	FBuffer Output;
	uint64	HeaderOffset   = ~0ull;
	uint64	NumAttachments = 0;
};

struct FMiniCbReader;

struct FMiniCbFieldView : FBufferView
{
	EMiniCbFieldType Type = EMiniCbFieldType::None;
	std::string_view Name;

	uint64			 UniformArrayByteCount = 0;
	uint64			 UniformArrayItemCount = 0;
	EMiniCbFieldType UniformArrayItemType  = EMiniCbFieldType::None;

	FMiniCbFieldView Child();

	bool IsValid() const { return Data != nullptr; }

	template<typename T>
	const T& GetValue() const
	{
		UNSYNC_ASSERT(IsValid());
		// TODO: validate requested type against type field
		return *reinterpret_cast<const T*>(Data);
	}

	template<typename T>
	std::span<const T> GetUniformArray() const
	{
		UNSYNC_ASSERT(IsValid());
		UNSYNC_ASSERT(Type == EMiniCbFieldType::UniformArray);
		return std::span<const T>(reinterpret_cast<const T*>(Data), UniformArrayItemCount);
	}
};

struct FMiniCbReader
{
	FMiniCbReader(const uint8* InData, uint64 InSize) : Data(InData), Size(InSize), Cursor(InData) {}
	FMiniCbReader(FBufferView View) : FMiniCbReader(View.Data, View.Size) {}
	FMiniCbReader(const FMiniCbFieldView& InView) : FMiniCbReader(InView.Data, InView.Size) {}

	FMiniCbFieldView Child()
	{
		FMiniCbFieldView Result = {};

		if (Cursor >= (Data + Size))
		{
			return Result;
		}

		uint8			 RawId	  = *Cursor;
		bool			 bHasName = (RawId & uint8(EMiniCbFieldType::HasFieldName)) != 0;
		EMiniCbFieldType Id		  = EMiniCbFieldType(RawId & (uint8(EMiniCbFieldType::HasFieldName) - 1));

		Cursor += 1;

		if (bHasName)
		{
			uint32		 NameHeaderSize = 0;
			const uint64 NameSize		= ReadVarUint(Cursor, NameHeaderSize);

			Result.Name = std::string_view((const char*)Cursor + NameHeaderSize, NameSize);
			Cursor += NameHeaderSize + NameSize;
		}

		Result.Type = Id;

		switch (Id)
		{
			default:
				UNSYNC_FATAL(L"Field type %d is not implemented", uint8(Id));
				break;
			case EMiniCbFieldType::None:
			case EMiniCbFieldType::Null:
				break;
			case EMiniCbFieldType::Object:
			case EMiniCbFieldType::BinaryAttachment:
			case EMiniCbFieldType::Hash:
				{
					FMiniCbPayloadSize FieldSize = GetPayloadSize(Id, Cursor);

					Result.Data = Cursor + FieldSize.HeaderSize;
					Result.Size = FieldSize.DataSize;

					Cursor += FieldSize.DataSize + FieldSize.HeaderSize;

					break;
				}
			case EMiniCbFieldType::UniformArray:
				{
					uint32 VarUintSize = 0;

					uint64 PayloadSize = ReadVarUint(Cursor, VarUintSize);
					Cursor += VarUintSize;

					Result.UniformArrayItemCount = ReadVarUint(Cursor, VarUintSize);
					Cursor += VarUintSize;

					Result.UniformArrayByteCount = PayloadSize - VarUintSize - 1;

					Result.UniformArrayItemType = EMiniCbFieldType(*Cursor);
					Cursor += 1;

					Result.Data = Cursor;
					Result.Size = Result.UniformArrayByteCount;

					Cursor += Result.UniformArrayByteCount;

					break;
				}
		}

		return Result;
	}

	const uint8* Data;
	uint64		 Size;

	const uint8* Cursor = nullptr;
};

inline FMiniCbFieldView
FMiniCbFieldView::Child()
{
	FMiniCbReader Reader(*this);
	return Reader.Child();
}

}  // namespace unsync
