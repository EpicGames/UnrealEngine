// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncMount.h"
#include "UnsyncCore.h"
#include "UnsyncSerialization.h"

#include <algorithm>

namespace unsync {

static int32
GetPathDepth(std::wstring_view S)
{
	int32 Result = 0;
	for (wchar_t c : S)
	{
		if (c == L'/' || c == L'\\')
		{
			Result++;
		}
	}
	return Result;
}

std::wstring_view
GetParentDirectory(std::wstring_view Name)
{
	size_t Pos = Name.find_last_of(L"\\/");
	if (Pos == std::string::npos)
	{
		return {};
	}
	else
	{
		return Name.substr(0, Pos);
	}
}

bool
FMountedDirectory::Mount(const FPath& RootPath)
{
	FPath DirectoryManifestPath = RootPath / ".unsync" / "manifest.bin";

	bValid = false;
	DirectoryEntries.clear();

	if (!LoadDirectoryManifest(Manifest, RootPath, DirectoryManifestPath))
	{
		UNSYNC_ERROR("Failed to load directory manifest");
		return false;
	}

	std::unordered_set<std::wstring_view> SubdirectorySet;

	for (const auto& it : Manifest.Files)
	{
		FDirectoryEntry FileEntry;
		FileEntry.Type	   = EDirectoryEntryType::File;
		FileEntry.Path	   = it.first;
		FileEntry.Manifest = &it.second;
		FileEntry.Depth	   = GetPathDepth(FileEntry.Path);

		DirectoryEntries.push_back(FileEntry);

		std::wstring_view ParentName = GetParentDirectory(FileEntry.Path);

		while (!ParentName.empty())
		{
			auto InsertResult = SubdirectorySet.insert(ParentName);

			if (InsertResult.second)
			{
				DirectoryEntries.back().Parent = *InsertResult.first;

				FDirectoryEntry SubdirEntry;
				SubdirEntry.Type	 = EDirectoryEntryType::Directory;
				SubdirEntry.Path	 = ParentName;
				SubdirEntry.Manifest = nullptr;
				SubdirEntry.Depth	 = GetPathDepth(SubdirEntry.Path);

				DirectoryEntries.push_back(SubdirEntry);
			}
			else
			{
				if (DirectoryEntries.back().Parent.empty())
				{
					DirectoryEntries.back().Parent = *InsertResult.first;
				}
			}

			ParentName = GetParentDirectory(ParentName);
		}
	}

	std::sort(DirectoryEntries.begin(), DirectoryEntries.end());
	std::wstring_view LastParent;
	for (uint64 i = 0; i < DirectoryEntries.size(); ++i)
	{
		FDirectoryEntry& Entry = DirectoryEntries[i];

		if (Entry.Parent.empty())
		{
			Entry.Name = Entry.Path;
		}
		else
		{
			Entry.Name = Entry.Path.substr(Entry.Parent.length() + 1);
		}

		if (Entry.Parent != LastParent)
		{
			DirectoryMap[Entry.Parent] = i;
			LastParent = Entry.Parent;
		}
	}

	bValid = true;

	return true;
}
}

