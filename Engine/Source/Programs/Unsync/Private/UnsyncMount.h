// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"

namespace unsync {

// Represents a VFS-like view of a directory manifest, providing API to list files,
struct FMountedDirectory
{
	UNSYNC_DISALLOW_COPY_ASSIGN(FMountedDirectory)

	FMountedDirectory() = default;

	bool			   bValid = false;
	FDirectoryManifest Manifest;

	enum class EDirectoryEntryType {
		// Filesystem entries are sorted by this and we want to list directories first during iteration
		Directory = 0,
		File	  = 1,
	};

	struct FDirectoryEntry
	{
		int32				Depth = 0;
		EDirectoryEntryType Type  = EDirectoryEntryType::File;
		// Full path relative to mounted directory root
		std::wstring_view Path;
		// Full path of parent directory (guaranteed to be deduplicated, i.e. identical strings point to the same memory)
		std::wstring_view Parent;
		// File/directory name (without parent path)
		std::wstring_view	 Name;
		const FFileManifest* Manifest = nullptr;

		bool operator<(const FDirectoryEntry& Other) const
		{
			if (Depth != Other.Depth)
				return Depth < Other.Depth;
			else if (Type != Other.Type)
				return Type < Other.Type;
			else
				return Path < Other.Path;
		}
	};

	std::vector<FDirectoryEntry> DirectoryEntries;

	// Offset of the first directory entry corresponding to a speficic parent name
	std::unordered_map<std::wstring_view, uint64> DirectoryMap;

	bool Mount(const FPath& RootPath);
};

}  // namespace unsync
