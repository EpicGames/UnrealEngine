// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncPack.h"
#include "UnsyncFile.h"
#include "UnsyncCompression.h"
#include "UnsyncHashTable.h"
#include "UnsyncCore.h"
#include "UnsyncManifest.h"

namespace unsync {

inline void
AddHash(uint64* Accumulator, const FHash128& Hash)
{
	uint64 BlockHashParts[2];
	memcpy(BlockHashParts, &Hash, sizeof(FHash128));
	Accumulator[0] += BlockHashParts[0];
	Accumulator[1] += BlockHashParts[1];
}

inline FHash128
MakeHashFromParts(uint64* Parts)
{
	FHash128 Result;
	memcpy(&Result, Parts, sizeof(Result));
	return Result;
}

FPackWriteContext::FPackWriteContext(const FPath& InOutputRoot) : OutputRoot(InOutputRoot)
{
	InternalReset();
}

FPackWriteContext::~FPackWriteContext()
{
	InternalFinishPack();
}

void
FPackWriteContext::AddRawBlock(const FGenericBlock& Block, FBufferView RawData)
{
	std::lock_guard<std::mutex> LockGuard(Mutex);
	InternalAddBlock(Block, Block.HashStrong.ToHash128(), RawData);
	++NumRawBlocks;
}

void
FPackWriteContext::AddCompressedBlock(const FGenericBlock& Block, FHash128 CompressedHash, FBufferView CompressedData)
{
	std::lock_guard<std::mutex> LockGuard(Mutex);
	InternalAddBlock(Block, CompressedHash, CompressedData);
	++NumCompressedBlocks;
}

void
FPackWriteContext::InternalAddBlock(const FGenericBlock& Block, FHash128 CompressedHash, FBufferView Data)
{
	UNSYNC_ASSERT(Data.Size <= GMaxPackFileSize);

	if (PackBuffer.Size() + Data.Size > GMaxPackFileSize)
	{
		InternalFinishPack();
	}

	FPackIndexEntry IndexEntry;
	IndexEntry.BlockHash	  = Block.HashStrong.ToHash128();
	IndexEntry.CompressedHash = CompressedHash;
	IndexEntry.PackBlockOffset = CheckedNarrow(PackBuffer.Size());
	IndexEntry.PackBlockSize   = CheckedNarrow(Data.Size);

	IndexEntries.push_back(IndexEntry);
	PackBuffer.Append(Data);

	UNSYNC_ASSERT(PackBuffer.Size() == IndexEntry.PackBlockOffset + IndexEntry.PackBlockSize);

	AddHash(IndexFileHashSum, IndexEntry.BlockHash);

	ProcessedRawBytes += Block.Size;
	ProcessedCompressedBytes = Data.Size;
}

void
FPackWriteContext::FinishPack()
{
	std::lock_guard<std::mutex> LockGuard(Mutex);
	InternalFinishPack();
}

void
FPackWriteContext::InternalFinishPack()
{
	// Assumes the function is called from a thread-safe context

	if (IndexEntries.empty())
	{
		return;
	}

	FHash128	BlockHash128 = MakeHashFromParts(IndexFileHashSum);
	std::string OutputId	 = HashToHexString(BlockHash128);

	FPath FinalPackFilename	 = OutputRoot / (OutputId + ".unsync_pack");
	FPath FinalIndexFilename = OutputRoot / (OutputId + ".unsync_index");

	// Force non-indented log
	FLogIndentScope IndentScope(0, true /*override*/);

	UNSYNC_LOG(L"* Saving new pack: %hs", OutputId.c_str());

	if (!GDryRun)
	{
		// TODO: offload final disk write to a background thread

		if (!WriteBufferToFile(FinalPackFilename, PackBuffer, EFileMode::CreateWriteOnly))
		{
			UNSYNC_FATAL(L"Failed to write pack file '%ls'", FinalPackFilename.wstring().c_str());
		}

		const uint8* IndexData	   = reinterpret_cast<const uint8*>(IndexEntries.data());
		uint64		 IndexDataSize = sizeof(IndexEntries[0]) * IndexEntries.size();

		FPackIndexHeader IndexHeader;
		IndexHeader.NumEntries = IndexEntries.size();

		const uint64	 IndexTotalSize = sizeof(IndexHeader) + IndexDataSize;

		FNativeFile IndexFile(FinalIndexFilename, EFileMode::CreateWriteOnly, IndexTotalSize);
		if (!IndexFile.IsValid())
		{
			UNSYNC_FATAL(L"Failed to write pack index file '%ls'", FinalIndexFilename.wstring().c_str());
		}

		uint64 WroteBytes = 0;
		
		WroteBytes += IndexFile.Write(&IndexHeader, WroteBytes, sizeof(IndexHeader));
		WroteBytes += IndexFile.Write(IndexData, WroteBytes, IndexDataSize);

		if (WroteBytes != IndexTotalSize)
		{
			UNSYNC_FATAL(L"Failed to write pack index file '%ls'", FinalIndexFilename.wstring().c_str());
		}
	}

	FPackReference PackReference;
	PackReference.Id = BlockHash128;

	if (NumRawBlocks != 0)
	{
		PackReference.Flags = PackReference.Flags | EPackReferenceFlags::HasRawBlocks;
	}

	if (NumCompressedBlocks != 0)
	{
		PackReference.Flags = PackReference.Flags | EPackReferenceFlags::HasCompressedBlocks;
	}

	PackReference.NumTotalBlocks = NumCompressedBlocks + NumRawBlocks;
	PackReference.NumUsedBlocks	 = PackReference.NumTotalBlocks; // By default assume all blocks in the pack are referenced

	GeneratedPackIds.push_back(PackReference);

	InternalReset();
}

void
FPackWriteContext::InternalReset()
{
	PackBuffer.Reserve(GMaxPackFileSize);
	PackBuffer.Clear();
	IndexEntries.clear();

	IndexFileHashSum[0] = 0;
	IndexFileHashSum[1] = 0;

	NumCompressedBlocks = 0;
	NumRawBlocks		= 0;
}

FPackWriteContext::FCompressedBlock
FPackWriteContext::CompressBlock(FBufferView RawData)
{
	FPackWriteContext::FCompressedBlock Result;

	const uint64 MaxCompressedSize = GetMaxCompressedSize(RawData.Size);
	Result.Data					   = FIOBuffer::Alloc(MaxCompressedSize, L"PackBlock");
	uint64 ActualCompressedSize	   = CompressInto(RawData, Result.Data.GetMutBufferView(), 9);

	if (!ActualCompressedSize)
	{
		UNSYNC_FATAL(L"Failed to compress file block");
	}
	Result.Data.SetDataRange(0, ActualCompressedSize);

	Result.Hash = HashBlake3Bytes<FHash128>(Result.Data.GetData(), ActualCompressedSize);

	return Result;
}

void FPackWriteContext::CompressAndAddBlock(const FGenericBlock& Block, FBufferView RawData)
{
	FCompressedBlock Compressed = CompressBlock(RawData);
	AddCompressedBlock(Block, Compressed.Hash, Compressed.Data.GetBufferView());
}

void
FPackWriteContext::GetUniqueGeneratedPackIds(std::vector<FPackReference>& Output) const
{
	std::lock_guard<std::mutex> LockGuard(Mutex);

	MergePackReferences(Output, GeneratedPackIds);
}

void
MergePackReferences(std::vector<FPackReference>& Destination, const std::vector<FPackReference>& Source)
{
	THashSet<FPackReference, FPackReference::Hasher> KnownHashes;

	for (const FPackReference& Hash : Destination)
	{
		KnownHashes.insert(Hash);
	}

	for (const FPackReference& Hash : Source)
	{
		if (KnownHashes.insert(Hash).second)
		{
			Destination.push_back(Hash);
		}
	}
}

void DeletePackAndIdexData(const FPath& PackRootDirectory)
{
	UNSYNC_VERBOSE(L"Deleting packs in '%ls'", PackRootDirectory.wstring().c_str());
	for (const std::filesystem::directory_entry& Dir : DirectoryScan(PackRootDirectory))
	{
		if (!Dir.is_regular_file())
		{
			continue;
		}

		const FPath& FilePath	   = Dir.path();
		const FPath	 FileExtension = FilePath.extension();
		if (FileExtension == ".unsync_pack" || FileExtension == ".unsync_index")
		{
			std::error_code ErrorCode;
			bool bDeleted = FileRemove(FilePath, ErrorCode);
			if (!bDeleted)
			{
				UNSYNC_ERROR(L"Could not delete file '%ls'. Error code: %d.", FilePath.wstring().c_str(), ErrorCode.value());
			}
		}
	}
}

bool LoadPackIndexDatabase(FPackIndexDatabase& Output, FIOReaderStream& Stream)
{
	FPackIndexHeader IndexHeader = {};
	memset(&IndexHeader, 0, sizeof(IndexHeader));

	Stream.ReadInto(IndexHeader);

	if (IndexHeader.Version != FPackIndexHeader::VERSION || IndexHeader.Magic != FPackIndexHeader::MAGIC)
	{
		return false;
	}

	Output.Entries.resize(IndexHeader.NumEntries);

	if (IndexHeader.NumEntries != 0)
	{
		const uint64 ExpectedReadSize = IndexHeader.NumEntries * sizeof(FPackIndexEntry);
		const uint64 ReadSize = Stream.Read(Output.Entries.data(), ExpectedReadSize);
		return ReadSize == ExpectedReadSize;
	}
	else
	{
		return true;
	}
}

}  // namespace unsync
