// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncHash.h"
#include "UnsyncProtocol.h"
#include "UnsyncFile.h"

#include <mutex>
#include <vector>

namespace unsync {

static constexpr uint64 GMaxPackFileSize = 1_GB;

struct FDirectoryManifest;

struct FPackIndexDatabase
{
	std::vector<FPackIndexEntry> Entries;

	// Runtime-only data
	FPath IndexPath;
	FPath DataPath;
};

bool LoadPackIndexDatabase(FPackIndexDatabase& Output, FIOReaderStream& Stream);

struct FPackWriteContext
{
	FPackWriteContext(const FPath& InOutputRoot);
	~FPackWriteContext();

	void AddRawBlock(const FGenericBlock& Block, FBufferView RawData);  // thread-safe
	void AddCompressedBlock(const FGenericBlock& Block, FHash128 CompressedHash, FBufferView CompressedData); // thread-safe
	void CompressAndAddBlock(const FGenericBlock& Block, FBufferView RawData); // thread-safe

	void FinishPack(); // thread-safe

	// Common utility function to compress a block and generate compressed hash
	struct FCompressedBlock
	{
		FIOBuffer Data; // zstd-compressed block
		FHash128  Hash; // block hash after compression
	};
	static FCompressedBlock CompressBlock(FBufferView RawData);

	void GetUniqueGeneratedPackIds(std::vector<FPackReference>& Output) const; // thread-safe

private:

	void InternalAddBlock(const FGenericBlock& Block, FHash128 CompressedHash, FBufferView Data);
	void InternalReset();
	void InternalFinishPack();

	mutable std::mutex Mutex;

	// Independent sums of low and high 32 bits of all seen block hashes.
	// Used to generate a stable hash while allowing out-of-order block processing.
	uint64 IndexFileHashSum[2] = {};

	FBuffer						 PackBuffer;
	std::vector<FPackIndexEntry> IndexEntries;

	uint64 ProcessedRawBytes = 0;
	uint64 ProcessedCompressedBytes = 0;

	uint32 NumRawBlocks = 0;
	uint32 NumCompressedBlocks = 0;

	FPath OutputRoot;

	std::vector<FPackReference> GeneratedPackIds;
};

void MergePackReferences(std::vector<FPackReference>& Destination, const std::vector<FPackReference>& Source);

void DeletePackAndIdexData(const FPath& PackRootDirectory);

}  // namespace unsync
