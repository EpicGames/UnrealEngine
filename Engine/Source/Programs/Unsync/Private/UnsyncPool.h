// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncThread.h"

#include <mutex>

namespace unsync {

template<typename T>
struct TObjectPool
{
	TObjectPool(std::function<T*()>&& InCreateObject) : CreateObject(InCreateObject) {}

	std::unique_ptr<T> Acquire()
	{
		std::lock_guard<std::mutex> LockGuard(Mutex);

		std::unique_ptr<T> Result;
		if (AvailableObjects.empty())
		{
			Result = std::unique_ptr<T>(CreateObject());
		}
		else
		{
			std::swap(Result, AvailableObjects.back());
			AvailableObjects.pop_back();
		}

		return Result;
	}

	void Release(std::unique_ptr<T>&& Object)
	{
		std::lock_guard<std::mutex> LockGuard(Mutex);
		AvailableObjects.push_back(std::move(Object));
	}

	std::function<T*()>				CreateObject;
	std::vector<std::unique_ptr<T>> AvailableObjects;
	std::mutex						Mutex;
};

}  // namespace unsync
