// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncProgress.h"

namespace unsync {

std::atomic<uint64> GGlobalProgressCurrent;
std::atomic<uint64> GGlobalProgressTotal;

void
AddGlobalProgress(uint64 Size, EBlockListType ListType)
{
	if (ListType == EBlockListType::Source)
	{
		Size *= GLOBAL_PROGRESS_SOURCE_SCALE;
	}
	else
	{
		Size *= GLOBAL_PROGRESS_BASE_SCALE;
	}

	GGlobalProgressCurrent += Size;
}

FLogProgressScope::FLogProgressScope(uint64 InTotal, ELogProgressUnits InUnits, uint64 InPeriodMilliseconds, bool bInVerboseOnly)
: bParentThreadVerbose(GLogVerbose)
, ParentThreadIndent(GLogIndent)
, Current(0)
, Total(InTotal)
, PeriodMilliseconds(InPeriodMilliseconds)
, Units(InUnits)
, NextProgressLogTime(TimePointNow())
, bEnabled(true)
, bVerboseOnly(bInVerboseOnly)
{
	Add(0);
}

void
FLogProgressScope::Complete()
{
	Add(0, true);
}

void
FLogProgressScope::Add(uint64 X, bool bForceComplete)
{
	if (!bEnabled)
	{
		return;
	}

	Current += X;

	std::lock_guard<std::mutex> LockGuard(Mutex);
	const uint64				CurrentClamped = std::min<uint64>(Current, Total);
	const bool					bComplete	   = (CurrentClamped == Total) || bForceComplete;

	const ELogLevel LogLevel = bVerboseOnly ? ELogLevel::Debug : ELogLevel::Info;

	if (bEnabled && (!bVerboseOnly || GLogVerbose) && (TimePointNow() > NextProgressLogTime || bComplete))
	{
		const wchar_t* Ending = bComplete ? L"\n" : L"\r";
		switch (Units)
		{
			case ELogProgressUnits::GB:
				LogPrintf(LogLevel,
						  L"%.2f / %.2f GB%ls",
						  double(CurrentClamped) / double(1_GB),
						  double(Total) / double(1_GB),
						  Ending);
				break;
			case ELogProgressUnits::MB:
				LogPrintf(LogLevel,
						  L"%.2f / %.2f MB%ls",
						  double(CurrentClamped) / double(1_MB),
						  double(Total) / double(1_MB),
						  Ending);
				break;
			case ELogProgressUnits::Bytes:
				LogPrintf(LogLevel, L"%llu / %llu bytes%ls", (uint64)CurrentClamped, Total, Ending);
				break;
			case ELogProgressUnits::Raw:
			default:
				LogPrintf(LogLevel, L"%llu / %llu%ls", (uint64)CurrentClamped, Total, Ending);
				break;
		}

		NextProgressLogTime = TimePointNow() + std::chrono::milliseconds(PeriodMilliseconds);
		LogGlobalProgress();

		if (bComplete)
		{
			bEnabled = false;
		}
	}
}

}  // namespace unsync
