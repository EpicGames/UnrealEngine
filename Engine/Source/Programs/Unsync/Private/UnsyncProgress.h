// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncLog.h"
#include "UnsyncUtil.h"

#include <atomic>
#include <mutex>

namespace unsync {

// pretend that reading remote files is ~25x slower than local
static constexpr uint64 GLOBAL_PROGRESS_SOURCE_SCALE = 25;
static constexpr uint64 GLOBAL_PROGRESS_BASE_SCALE	 = 1;

extern std::atomic<uint64> GGlobalProgressCurrent;
extern std::atomic<uint64> GGlobalProgressTotal;

inline void
LogGlobalProgress()
{
	LogProgress(nullptr, GGlobalProgressCurrent, GGlobalProgressTotal);
}

enum class EBlockListType {
	Source,
	Base
};

void AddGlobalProgress(uint64 Size, EBlockListType ListType);

enum class ELogProgressUnits {
	Raw,
	Bytes,
	MB,
	GB,
};

struct FLogProgressScope
{
	UNSYNC_DISALLOW_COPY_ASSIGN(FLogProgressScope)

	FLogProgressScope(uint64 InTotal, ELogProgressUnits InUnits = ELogProgressUnits::Raw, uint64 InPeriodMilliseconds = 500, bool bInVerboseOnly = true);

	void Complete();

	void Add(uint64 X, bool bForceComplete = false);

	const bool	 bParentThreadVerbose;
	const uint32 ParentThreadIndent;

	std::mutex				Mutex;
	std::atomic<uint64>		Current;
	const uint64			Total;
	const uint64			PeriodMilliseconds;
	const ELogProgressUnits Units;
	FTimePoint				NextProgressLogTime;
	std::atomic<bool>		bEnabled;
	const bool				bVerboseOnly;
};

}  // namespace unsync
