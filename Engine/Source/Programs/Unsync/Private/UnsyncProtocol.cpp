// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncProtocol.h"
#include "UnsyncLog.h"

namespace unsync {

const char*
ToString(EChunkingAlgorithmID Algorithm)
{
	switch (Algorithm)
	{
		case EChunkingAlgorithmID::FixedBlocks:
			return "Fixed";
		case EChunkingAlgorithmID::VariableBlocks:
			return "Variable";
		default:
			UNSYNC_ERROR(L"Unexpected hash algorithm id");
			return "UNKNOWN";
	}
}

const char*
ToString(EStrongHashAlgorithmID Algorithm)
{
	switch (Algorithm)
	{
		case EStrongHashAlgorithmID::MD5:
			return "MD5";
		case EStrongHashAlgorithmID::Meow:
			return "Meow";
		case EStrongHashAlgorithmID::Blake3_128:
			return "Blake3.128";
		case EStrongHashAlgorithmID::Blake3_160:
			return "Blake3.160";
		case EStrongHashAlgorithmID::Blake3_256:
			return "Blake3.256";
		default:
			UNSYNC_ERROR(L"Unexpected strong hash algorithm id");
			return "UNKNOWN";
	}
}

const char*
ToString(EWeakHashAlgorithmID Algorithm)
{
	switch (Algorithm)
	{
		case EWeakHashAlgorithmID::Naive:
			return "Naive";
		case EWeakHashAlgorithmID::BuzHash:
			return "BuzHash";
		default:
			UNSYNC_ERROR(L"Unexpected weak hash algorithm id");
			return "UNKNOWN";
	}
}

}  // namespace unsync
