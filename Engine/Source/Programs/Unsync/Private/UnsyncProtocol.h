// Copyright Epic Games, Inc. All Rights Reserved.

//  _____ __  __ _____   ____  _____ _______       _   _ _______
// |_   _|  \/  |  __ \ / __ \|  __ \__   __|/\   | \ | |__   __|
//   | | | \  / | |__) | |  | | |__) | | |  /  \  |  \| |  | |
//   | | | |\/| |  ___/| |  | |  _  /  | | / /\ \ | . ` |  | |
//  _| |_| |  | | |    | |__| | | \ \  | |/ ____ \| |\  |  | |
// |_____|_|  |_|_|     \____/|_|  \_\ |_/_/    \_\_| \_|  |_|
//
// Most structures in this file are part of binary protocol.
// Modifications of the data types have back-compat implications!

#pragma once

#include "UnsyncBuffer.h"
#include "UnsyncCommon.h"
#include "UnsyncHash.h"

namespace unsync {

static constexpr uint32	   MAX_BLOCK_SIZE		 = uint32(1_MB);
static constexpr EHashType MACRO_BLOCK_HASH_TYPE = EHashType::Blake3_160;  // Blake3 160bit (IoHash) is required for macro blocks due to back-end storage implementation

enum class EChunkingAlgorithmID : uint64
{
	Invalid		   = 0,
	FixedBlocks	   = 0xE9457636EEF48607ull,	 // "FixedBlocks" as fnv1a64
	VariableBlocks = 0xE62448A75E8B1CC3ull,	 // "VariableBlocks" as fnv1a64
};

const char* ToString(EChunkingAlgorithmID Algorithm);

enum class EWeakHashAlgorithmID : uint64
{
	Invalid = 0,
	Naive	= 0x4BD87A66500A16D6ull,  // "Naive" as fnv1a64
	BuzHash = 0x9A8AB46A97A95962ull,  // "Buzhash" as fnv1a64
};

const char* ToString(EWeakHashAlgorithmID Algorithm);

enum class EStrongHashAlgorithmID : uint64
{
	Invalid	   = 0,
	MD5		   = 0x1E8F2819B58AD6B3ull,	 // "MD5" as fnv1a64
	Meow	   = 0x6D8900AEE47E763Dull,	 // "Meow" as fnv1a64 (deprecated)
	Blake3_128 = 0x7FD87D89C7C1D597ull,	 // "Blake3" as fnv1a64 ("_128" is omitted for backwards compatibility)
	Blake3_160 = 0xB68497EF4370C4F5ull,	 // "Blake3_160" as fnv1a64
	Blake3_256 = 0xBF89BAEF48A68CC5ull,	 // "Blake3_256" as fnv1a64
};

const char* ToString(EStrongHashAlgorithmID Algorithm);

template<typename StrongHashT>
struct TBlock
{
	using StrongHashType = StrongHashT;

	uint64		Offset	   = 0;
	uint32		Size	   = 0;
	uint32		HashWeak   = 0;
	StrongHashT HashStrong = {};

	struct FCompareByOffset
	{
		bool operator()(const TBlock<StrongHashT>& A, const TBlock<StrongHashT>& B) const { return A.Offset < B.Offset; }
	};
};

using FBlock128 = TBlock<FHash128>;
using FBlock160 = TBlock<FHash160>;
using FBlock256 = TBlock<FHash256>;

using FGenericBlock = TBlock<FGenericHash>;
using FGenericBlockArray = std::vector<FGenericBlock>;

// Json formatting helpers
void FormatJsonBlock(std::wstring& Output, const FGenericBlock& Block);
void FormatJsonBlock(std::string& Output, const FGenericBlock& Block);
void FormatJsonBlockArray(std::wstring& Output, const FGenericBlockArray& Blocks);
void FormatJsonBlockArray(std::string& Output, const FGenericBlockArray& Blocks);

// These are just random 64 bit numbers, not based on anything in particular
static constexpr uint64 SERIALIZED_SECTION_ID_TERMINATOR			= 0;
static constexpr uint64 SERIALIZED_SECTION_ID_METADATA_STRING		= 0xC6BD6CDCEEF79533ull;
static constexpr uint64 SERIALIZED_SECTION_ID_MACRO_BLOCK			= 0x8390AEBB745E08BCull;
static constexpr uint64 SERIALIZED_SECTION_ID_FILE_READ_ONLY_MASK	= 0x851F32ED3615F0ADull;
static constexpr uint64 SERIALIZED_SECTION_ID_FILE_REVISION_CONTROL = 0x2C1C72E6B78B1B50ull;
static constexpr uint64 SERIALIZED_SECTION_ID_PACK_REFERENCE		= 0x634EA57F1E48DFBDull;
static constexpr uint64 SERIALIZED_SECTION_ID_FILE_EXECUTABLE_BIT	= 0x2F4212FDAEF5C1ADull;

struct FSerializedSectionHeader
{
	static constexpr uint64 MAGIC	= 0xEE5037CFF5BC71B2ull;
	uint64					Magic	= MAGIC;
	uint64					Id		= SERIALIZED_SECTION_ID_TERMINATOR;
	uint64					Version = 0;
	uint64					Size	= 0;
};

struct FMetadataStringSection
{
	static constexpr uint64 MAGIC	= SERIALIZED_SECTION_ID_METADATA_STRING;
	static constexpr uint64 VERSION = 1;
	std::string				NameUtf8;
	std::string				ValueUtf8;
};

struct FMacroBlockSection
{
	static constexpr uint64 MAGIC	= SERIALIZED_SECTION_ID_MACRO_BLOCK;
	static constexpr uint64 VERSION = 2;
};

struct FFileReadOnlyMaskSection
{
	static constexpr uint64 MAGIC	= SERIALIZED_SECTION_ID_FILE_READ_ONLY_MASK;
	static constexpr uint64 VERSION = 1;
};

struct FFileExecutableBitSection
{
	static constexpr uint64 MAGIC	= SERIALIZED_SECTION_ID_FILE_EXECUTABLE_BIT;
	static constexpr uint64 VERSION = 1;
};

struct FFileRevisionControlSection
{
	static constexpr uint64 MAGIC	= SERIALIZED_SECTION_ID_FILE_REVISION_CONTROL;
	static constexpr uint64 VERSION = 1;
};

struct FPackReferenceSection
{
	static constexpr uint64 MAGIC	= SERIALIZED_SECTION_ID_PACK_REFERENCE;
	static constexpr uint64 VERSION = 3;
};

struct FBlockFileHeader
{
	static constexpr uint64 MAGIC	= 0x1DCB86A5BDBA27CFull;
	static constexpr uint64 VERSION = 2;

	uint64 Magic	 = MAGIC;
	uint64 Version	 = VERSION;
	uint64 BlockSize = 0;
	uint64 NumBlocks = 0;
};

struct FBlockRequest
{
	FHash128 FilenameMd5 = {};
	FHash128 BlockHash	 = {};
	uint64	 Offset		 = 0;
	uint64	 Size		 = 0;
};

struct FHandshakePacket
{
	static constexpr uint64 MAGIC	 = 0xAE2C046180D0914Eull;
	uint64					Magic	 = MAGIC;
	uint32					Protocol = 1;
	uint32					Size	 = 0;
};

static constexpr uint64 COMMAND_ID_DISCONNECT = 0;
static constexpr uint64 COMMAND_ID_AUTHENTICATE = 0xAA77CB56ABD7153Aull;
static constexpr uint64 COMMAND_ID_GET_BLOCKS = 0xBBE2A1CECC8C949Cull;

struct FCommandPacket
{
	static constexpr uint64 MAGIC	  = 0x251B6A201A26EC82ull;
	uint64					Magic	  = MAGIC;
	uint64					CommandId = 0;
};

struct FBufferPacket
{
	static constexpr uint64 MAGIC		  = 0x6539A89058A3400Aull;
	uint64					Magic		  = MAGIC;
	uint32					DataSizeBytes = 0;
};

struct FFileListPacket
{
	static constexpr uint64 MAGIC		  = 0x28B96050A327172Aull;
	uint64					Magic		  = MAGIC;
	uint32					DataSizeBytes = 0;
	uint32					NumFiles	  = 0;
};

struct FRequestBlocksPacket
{
	static constexpr uint64 MAGIC				  = 0x6A885827EA6659F7ull;
	uint64					Magic				  = MAGIC;
	uint64					StrongHashAlgorithmId = 0;
	uint32					CompressedSizeBytes	  = 0;
	uint32					DecompressedSizeBytes = 0;
	uint32					NumRequests			  = 0;
};

struct FBlockPacket
{
	FHash128 Hash			  = {};
	uint64	 DecompressedSize = 0; // 0 if data is not compressed
	FBuffer	 Data;
};

struct FPatchHeader
{
	static constexpr uint64 VALIDATION_BLOCK_SIZE = 16_MB;

	static constexpr uint64 MAGIC	= 0x3E63942C4C9ECE16ull;
	static constexpr uint64 VERSION = 2;

	uint64				   Magic					 = MAGIC;
	uint64				   Version					 = VERSION;
	uint64				   SourceSize				 = 0;
	uint64				   BaseSize					 = 0;
	uint64				   NumSourceValidationBlocks = 0;
	uint64				   NumBaseValidationBlocks	 = 0;
	uint64				   NumSourceBlocks			 = 0;
	uint64				   NumBaseBlocks			 = 0;
	uint64				   BlockSize				 = 0;
	EWeakHashAlgorithmID   WeakHashAlgorithmId		 = EWeakHashAlgorithmID::Naive;
	EStrongHashAlgorithmID StrongHashAlgorithmId	 = EStrongHashAlgorithmID::Blake3_128;
};

struct FPackIndexEntry
{
	// Decompressed block hash
	FHash128 BlockHash		 = {};
	// Compressed block hash (may be equal to BlockHash to signal uncompressed block)
	FHash128 CompressedHash	 = {};

	// Offset and size within pack file
	uint32	 PackBlockOffset = 0;
	uint32	 PackBlockSize	 = 0;
};
static_assert(sizeof(FPackIndexEntry) == 40);

struct FPackIndexHeader
{
	static constexpr uint64 MAGIC	= 0xEEC735E03053CC3Full;
	static constexpr uint64 VERSION = 2;

	uint64 Magic	  = MAGIC;
	uint64 Version	  = VERSION;
	uint64 NumEntries = 0;
};
static_assert(sizeof(FPackIndexHeader) == 24);

enum class EPackReferenceFlags : uint32 {
	Default				= 0,
	HasRawBlocks		= 1 << 0,
	HasCompressedBlocks = 1 << 1,
};
UNSYNC_ENUM_CLASS_FLAGS(EPackReferenceFlags, uint32);

// Basic information about a pack file referenced by a manifest
struct FPackReference
{
	using EFlags = EPackReferenceFlags;

	bool operator==(const FPackReference& Other) const { return 
		Id == Other.Id && Flags == Other.Flags
		&& NumUsedBlocks == Other.NumUsedBlocks
		&& NumTotalBlocks == Other.NumTotalBlocks; }
	struct Hasher
	{
		size_t operator()(const FPackReference& X) const
		{
			FHash128::Hasher H;
			return H(X.Id);
		}
	};

	FHash128 Id				= {};
	EFlags	 Flags			= EFlags::Default;
	uint32	 NumUsedBlocks	= 0;
	uint32	 NumTotalBlocks = 0;
};
static_assert(sizeof(FPackReference) == 28);

// Protocol V2: support for up to 256bit hashes

struct FBlockRequest256
{
	FHash128 FilenameMd5 = {};
	FHash256 BlockHash	 = {};
	uint64	 Offset		 = 0;
	uint64	 Size		 = 0;
};

struct FBlockPacket256
{
	FHash256 Hash;
	uint64	 DecompressedSize = 0;
	FBuffer	 CompressedData;
};

struct FHordeUnsyncBlobHeaderV1
{
	static constexpr uint64 MAGIC = 0x4C5C2AABA992610Cull;

	uint64 Magic = 0;
	uint64 PayloadSize = 0;
	uint64 DecompressedSize = 0;
	FHash160 DecompressedHash = {};
};

struct FHordeUnsyncBlobErrorHeaderV1
{
	static constexpr uint64 MAGIC = 0x4C5C2AABA992DEADull;

	uint64	 Magic			  = 0;
	uint32	 PayloadSize	  = 0;
};

// Block stream response is always terminated using a packet with this hash.
// The packet payload may optionally contain a Json with diagnostics.
inline const FHash128 TERMINATOR_BLOCK_HASH = FHash128{};

}  // namespace unsync
