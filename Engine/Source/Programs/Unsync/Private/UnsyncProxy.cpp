// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncProxy.h"
#include "UnsyncAuth.h"
#include "UnsyncCompression.h"
#include "UnsyncFile.h"
#include "UnsyncHash.h"
#include "UnsyncJupiter.h"
#include "UnsyncPool.h"
#include "UnsyncProgress.h"
#include "UnsyncScheduler.h"
#include "UnsyncHorde.h"
#include "UnsyncUtil.h"

#include <atomic>
#include <json11.hpp>

#include <fmt/format.h>
#if __has_include(<fmt/xchar.h>)
#	include <fmt/xchar.h>
#endif

namespace unsync {

struct FUnsyncBaseProtocolImpl : FRemoteProtocolBase
{
	FUnsyncBaseProtocolImpl(const FRemoteDesc&			   InRemoteDesc,
							const FBlockRequestMap*		   InRequestMap,
							const FRemoteProtocolFeatures& InFeatures)
	: FRemoteProtocolBase(InRemoteDesc, InRequestMap)
	, Features(InFeatures)
	{
	}

	virtual TResult<FDirectoryManifest> DownloadManifest(std::string_view ManifestName) override
	{
		return AppError(L"Manifests can't be downloaded from UNSYNC proxy.");
	};

	const FRemoteProtocolFeatures Features;
};

struct FUnsyncProtocolImpl : FUnsyncBaseProtocolImpl
{
	FUnsyncProtocolImpl(const FRemoteDesc&			   InRemoteDesc,
						const FRemoteProtocolFeatures& InFeatures,
						const FAuthDesc*			   InAuthDesc,
						const FBlockRequestMap*		   InRequestMap);
	virtual ~FUnsyncProtocolImpl() override;
	virtual bool			IsValid() const override;
	virtual FDownloadResult Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback) override;

	virtual void Invalidate() override;

	ESocketSecurity GetSocketSecurity() const;

	bool						 bIsConnetedToHost = false;
	std::unique_ptr<FSocketBase> SocketHandle;

	static void SendTelemetryEvent(const FRemoteDesc& RemoteDesc, const FTelemetryEventSyncComplete& Event);

	static TResult<ProxyQuery::FHelloResponse>	  QueryHello(FHttpConnection& HttpConnection, const FAuthDesc* OptAuthDesc = nullptr);
	static TResult<ProxyQuery::FDirectoryListing> QueryListDirectory(FHttpConnection&	Connection,
																	 const FAuthDesc*	AuthDesc,
																	 const std::string& Path);
};

struct FUnsyncHttpProtocolImpl : FUnsyncBaseProtocolImpl
{
	FUnsyncHttpProtocolImpl(const FRemoteDesc&			   InRemoteDesc,
							const FRemoteProtocolFeatures& InFeatures,
							const FBlockRequestMap*		   InRequestMap,
							FProxyPool&					   InProxyPool)
	: FUnsyncBaseProtocolImpl(InRemoteDesc, InRequestMap, InFeatures)
	, ProxyPool(InProxyPool)
	{
	}

	virtual bool IsValid() const override { return bValid; }
	virtual void Invalidate() override { bValid = false; }

	virtual FDownloadResult Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback) override;

	const FRemoteProtocolFeatures Features;
	bool						  bValid = true;
	FProxyPool&					  ProxyPool;
};

FProxy::FProxy(FProxyPool&					  ProxyPool,
			   const FRemoteDesc&			  RemoteDesc,
			   const FRemoteProtocolFeatures& InFeatures,
			   const FAuthDesc*				  InAuthDesc,
			   const FBlockRequestMap*		  InRequestMap)
{
	UNSYNC_ASSERT(InRequestMap);

	if (RemoteDesc.Protocol == EProtocolFlavor::Jupiter)
	{
		auto Inner	 = new FJupiterProtocolImpl(RemoteDesc, InRequestMap, RemoteDesc.HttpHeaders);
		ProtocolImpl = std::unique_ptr<FRemoteProtocolBase>(Inner);
	}
	else if (RemoteDesc.Protocol == EProtocolFlavor::Horde)
	{
		auto Inner	 = new FHordeProtocolImpl(RemoteDesc, InRequestMap, ProxyPool);
		ProtocolImpl = std::unique_ptr<FRemoteProtocolBase>(Inner);
	}
	else if (RemoteDesc.Protocol == EProtocolFlavor::Unsync)
	{
		if (GExperimental && ProxyPool.SupportsHttp() && InFeatures.bBlockDownload)
		{
			auto* Inner	 = new FUnsyncHttpProtocolImpl(RemoteDesc, InFeatures, InRequestMap, ProxyPool);
			ProtocolImpl = std::unique_ptr<FRemoteProtocolBase>(Inner);
		}
		else
		{
			auto* Inner	 = new FUnsyncProtocolImpl(RemoteDesc, InFeatures, InAuthDesc, InRequestMap);
			ProtocolImpl = std::unique_ptr<FRemoteProtocolBase>(Inner);
		}
	}
	else
	{
		UNSYNC_FATAL(L"Unknown remote protocol %d", (int)RemoteDesc.Protocol);
	}
}

FUnsyncProtocolImpl::FUnsyncProtocolImpl(const FRemoteDesc&				RemoteDesc,
										 const FRemoteProtocolFeatures& InFeatures,
										 const FAuthDesc*				InAuthDesc,
										 const FBlockRequestMap*		InRequestMap)
: FUnsyncBaseProtocolImpl(RemoteDesc, InRequestMap, InFeatures)
{
	if (RemoteDesc.TlsRequirement != ETlsRequirement::None)
	{
		FTlsClientSettings TlsSettings	   = RemoteDesc.GetTlsClientSettings();
		FSocketHandle	   RawSocketHandle = SocketConnectTcp(RemoteDesc.Host.Address.c_str(), RemoteDesc.Host.Port);
		SocketSetRecvTimeout(RawSocketHandle, RemoteDesc.RecvTimeoutSeconds);

		if (RawSocketHandle)
		{
			FSocketTls* TlsSocket = new FSocketTls(RawSocketHandle, TlsSettings);
			if (TlsSocket->IsTlsValid())
			{
				SocketHandle = std::unique_ptr<FSocketTls>(TlsSocket);
			}
			else
			{
				delete TlsSocket;
			}
		}
	}

	if (!SocketHandle && RemoteDesc.TlsRequirement != ETlsRequirement::Required)
	{
		FSocketHandle RawSocketHandle = SocketConnectTcp(RemoteDesc.Host.Address.c_str(), RemoteDesc.Host.Port);
		SocketSetRecvTimeout(RawSocketHandle, RemoteDesc.RecvTimeoutSeconds);

		SocketHandle = std::unique_ptr<FSocketRaw>(new FSocketRaw(RawSocketHandle));
	}

	if (SocketHandle)
	{
		bIsConnetedToHost = [this]()
		{
			FHandshakePacket HandshakePacketTx;
			if (!SocketSendT(*SocketHandle, HandshakePacketTx))
			{
				UNSYNC_LOG(L"Failed to send the handshake packet");
				return false;
			}

			FHandshakePacket HandshakePacketRx;
			memset(&HandshakePacketRx, 0, sizeof(HandshakePacketRx));
			if (!SocketRecvT(*SocketHandle, HandshakePacketRx))
			{
				UNSYNC_LOG(L"Failed to receive the handshake packet");
				return false;
			}

			if (HandshakePacketRx.Magic != HandshakePacketTx.Magic || HandshakePacketRx.Protocol != HandshakePacketTx.Protocol ||
				HandshakePacketRx.Size != HandshakePacketTx.Size)
			{
				UNSYNC_LOG(L"Failed to receive the handshake packet");
				return false;
			}

			return true;
		}();
	}

	if (IsValid() && Features.bAuthentication && RemoteDesc.bAuthenticationRequired)
	{
		if (InAuthDesc)
		{
			bool bOk = IsValid();

			TResult<FAuthToken> AuthTokenResult = Authenticate(*InAuthDesc, RemoteDesc.RecvTimeoutSeconds);

			if (AuthTokenResult.IsOk())
			{
				FBufferView AccessToken = {(const uint8*)AuthTokenResult->Access.data(), AuthTokenResult->Access.length()};

				FCommandPacket Packet;
				Packet.CommandId = COMMAND_ID_AUTHENTICATE;
				bOk &= SendStruct(*SocketHandle, Packet);
				bOk &= SendBuffer(*SocketHandle, AccessToken);

				int32 ResultSize = 0;
				bOk &= SocketRecvT(*SocketHandle, ResultSize);

				FBuffer ResultBuffer;
				if (ResultSize)
				{
					ResultBuffer.Resize(ResultSize);
					bOk &= (SocketRecvAll(*SocketHandle, ResultBuffer.Data(), ResultSize) == ResultSize);
				}

				// TODO: parse authentication result packet and report errors
			}
			else
			{
				LogError(AuthTokenResult.GetError(), L"Server requires authentication, but access token could not be acquired");
				Invalidate();
			}
		}
		else
		{
			UNSYNC_ERROR(L"Server requires authentication, but required parameters were not provided");
			Invalidate();
		}
	}
}

FProxy::~FProxy()
{
}

bool
FProxy::Contains(const FDirectoryManifest& Manifest)
{
	return ProtocolImpl.get() && ProtocolImpl->Contains(Manifest);
}

bool
FProxy::IsValid() const
{
	return ProtocolImpl.get() && ProtocolImpl->IsValid();
}

TResult<FDirectoryManifest>
FProxy::DownloadManifest(std::string_view ManifestName)
{
	if (ProtocolImpl.get())
	{
		return ProtocolImpl->DownloadManifest(ManifestName);
	}
	else
	{
		return AppError(L"Server connection is invalid");
	}
}

FDownloadResult
FProxy::Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback)
{
	if (ProtocolImpl.get() && ProtocolImpl->IsValid())
	{
		return ProtocolImpl->Download(NeedBlocks, CompletionCallback);
	}
	else
	{
		return FDownloadResult(EDownloadRetryMode::Disconnect);
	}
}

bool
FUnsyncProtocolImpl::IsValid() const
{
	return bIsConnetedToHost && SocketHandle && SocketValid(*SocketHandle);
}

template<typename RequestType>
void
SortBlockRequestsByFileName(std::vector<RequestType>& Requests)
{
	std::sort(Requests.begin(),
			  Requests.end(),
			  [](const RequestType& A, const RequestType& B) -> bool
			  {
				  int32 FileCmp = std::memcmp(A.FilenameMd5.Data, B.FilenameMd5.Data, A.FilenameMd5.Size());
				  if (FileCmp != 0)
				  {
					  return FileCmp < 0;
				  }
				  return A.Offset < B.Offset;
			  });
}

// Log any diagnostic messages. Return true if block download request can be retried.
bool
ProcessBlockPacketErrorJson(std::string_view DiagnosticViewJson)
{
	std::string DiagnosticStringJson(DiagnosticViewJson);

	using namespace json11;

	std::string JsonErrorString;
	Json		JsonObject = Json::parse(DiagnosticStringJson, JsonErrorString);

	if (!JsonErrorString.empty())
	{
		UNSYNC_ERROR(L"Failed to parse diagnostic message from UNSYNC server: %hs", JsonErrorString.c_str());
		return false;
	}

	if (auto& Field = JsonObject["message"]; Field.is_string())
	{
		const std::string& FieldVal = Field.string_value();
		UNSYNC_ERROR(L"Server error: %hs", FieldVal.c_str());
	}

	if (auto& Field = JsonObject["can_retry"]; Field.is_bool())
	{
		bool FieldVal = Field.bool_value();
		return FieldVal;
	}

	// Allow retrying the request by default
	return true;
}

FDownloadResult
FUnsyncProtocolImpl::Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback)
{
	if (!IsValid())
	{
		return FDownloadResult(EDownloadRetryMode::Disconnect);
	}

	const EStrongHashAlgorithmID StrongHasher = RequestMap->GetStrongHasher();

	std::unordered_set<FHash128> UniqueFileNamesMd5;

	std::vector<FBlockRequest> Requests;
	Requests.reserve(NeedBlocks.Size());

	for (const FNeedBlock& Block : NeedBlocks)
	{
		if (const FBlockRequest* Request = RequestMap->FindRequest(Block.Hash))
		{
			Requests.push_back(*Request);
			UniqueFileNamesMd5.insert(Request->FilenameMd5);
		}
	}

	std::vector<const std::string*> FileListUtf8;
	for (const FHash128& It : UniqueFileNamesMd5)
	{
		const std::string* Name = RequestMap->FindSourceFile(It);
		if (Name)
		{
			FileListUtf8.push_back(Name);
		}
	}

	SortBlockRequestsByFileName(Requests);

	bool bOk = bIsConnetedToHost;

	// begin the command

	{
		FCommandPacket Packet;
		Packet.CommandId = COMMAND_ID_GET_BLOCKS;
		bOk &= SendStruct(*SocketHandle, Packet);
	}

	// send file list followed by requests
	if (bOk)
	{
		FBuffer FileListData;

		{
			FVectorStreamOut Writer(FileListData);

			for (const std::string* Str : FileListUtf8)
			{
				uint64 Len = Str->length();	 // 64 bit length, compatible with rust bincode
				Writer.WriteT(Len);
				Writer.Write(Str->c_str(), Len);
			}
		}

		FFileListPacket FileListHeader;
		FileListHeader.DataSizeBytes = CheckedNarrow(FileListData.Size());
		FileListHeader.NumFiles		 = CheckedNarrow(FileListUtf8.size());

		bOk &= SendStruct(*SocketHandle, FileListHeader);
		bOk &= SendBuffer(*SocketHandle, FileListData);
	}

	uint64 BytesToDownload = 0;

	if (bOk)
	{
		FBuffer RequestData;

		{
			FVectorStreamOut Writer(RequestData);

			for (const FBlockRequest& It : Requests)
			{
				BytesToDownload += It.Size;
				Writer.WriteT(It);
			}
		}

		FBuffer RequestDataCompressed = Compress(RequestData.Data(), RequestData.Size());

		FRequestBlocksPacket RequestHeader;
		RequestHeader.CompressedSizeBytes	= CheckedNarrow(RequestDataCompressed.Size());
		RequestHeader.DecompressedSizeBytes = CheckedNarrow(RequestData.Size());
		RequestHeader.NumRequests			= CheckedNarrow(Requests.size());
		RequestHeader.StrongHashAlgorithmId = uint64(StrongHasher);

		bOk &= SendStruct(*SocketHandle, RequestHeader);
		bOk &= SendBuffer(*SocketHandle, RequestDataCompressed);
	}

	if (!bOk)
	{
		bIsConnetedToHost = false;
		return FDownloadResult(EDownloadRetryMode::Disconnect);
	}

	uint64 BytesDownloaded = 0;

	FBlockPacket BlockPacket;

	for (uint64 I = 0; I < (Requests.size() + 1) && bOk; ++I)
	{
		BlockPacket.DecompressedSize = 0;
		BlockPacket.Hash			 = {};

		uint32 PacketSize = 0;
		bOk &= SocketRecvT(*SocketHandle, PacketSize);
		bOk &= SocketRecvT(*SocketHandle, BlockPacket.Hash);
		bOk &= SocketRecvT(*SocketHandle, BlockPacket.DecompressedSize);

		uint64 CompressedDataSize = 0;
		bOk &= SocketRecvT(*SocketHandle, CompressedDataSize);

		if (!bOk)
		{
			UNSYNC_WARNING(L"Failed to receive block header");
			break;
		}

		BlockPacket.Data.Resize(CompressedDataSize);

		bOk &= (SocketRecvAll(*SocketHandle, BlockPacket.Data.Data(), BlockPacket.Data.Size()) == CompressedDataSize);

		if (!bOk)
		{
			UNSYNC_WARNING(L"Failed to receive block buffer");
			break;
		}

		if (BlockPacket.Hash == TERMINATOR_BLOCK_HASH)
		{
			// If we received the terminator packet before the last expected block, it likely indicates an error of some kind.
			if (I < Requests.size())
			{
				std::string_view DiagnosticView = AsStringView(BlockPacket.Data);
				if (DiagnosticView.starts_with('{'))
				{
					bool bCanRetry = ProcessBlockPacketErrorJson(DiagnosticView);
					if (!bCanRetry)
					{
						return FDownloadError(EDownloadRetryMode::Abort);
					}
				}
				else
				{
					UNSYNC_WARNING(L"Received early stream termination packet without diagnostics");
				}
			}

			break;
		}

		if (bOk)
		{
			FDownloadedBlock DownloadedBlock;

			DownloadedBlock.CompressedSize = BlockPacket.Data.Size();
			DownloadedBlock.Data		   = BlockPacket.Data.Data();

			if (BlockPacket.DecompressedSize != 0)
			{
				DownloadedBlock.bCompressed		 = true;
				DownloadedBlock.DecompressedSize = BlockPacket.DecompressedSize;
			}
			else
			{
				DownloadedBlock.bCompressed		 = false;
				DownloadedBlock.DecompressedSize = DownloadedBlock.CompressedSize;
			}

			CompletionCallback(DownloadedBlock, BlockPacket.Hash);
		}

		BytesDownloaded += BlockPacket.DecompressedSize;
	}

	if (!bOk)
	{
		SocketHandle	  = {};
		bIsConnetedToHost = false;
	}

	return ResultOk<FDownloadError>();
}

std::string
FormatBlockRequestJson(const FBlockRequestMap& RequestMap, const TArrayView<FNeedBlock> NeedBlocks)
{
	const char* StrongHashAlgorithm = ToString(RequestMap.GetStrongHasher());

	struct FBlockRequestAndHash : FBlockRequest
	{
		FGenericHash FullHash;
	};

	std::vector<FBlockRequestAndHash> Requests;

	for (const FNeedBlock& Block : NeedBlocks)
	{
		if (const FBlockRequest* Request = RequestMap.FindRequest(Block.Hash))
		{
			FBlockRequestAndHash Item;
			static_cast<FBlockRequest&>(Item) = *Request;
			Item.FullHash					  = Block.Hash;

			Requests.push_back(Item);
		}
	}

	SortBlockRequestsByFileName(Requests);

	std::string Output;

	Output += "{ ";	 // main object

	FormatJsonKeyValueStr(Output, "hash_strong", StrongHashAlgorithm, ",\n");

	Output += "\"files\": [\n";

	static const FHash128 InvalidHash  = {};
	FHash128			  FilenameHash = InvalidHash;

	uint32 BlockIndex = 0;
	for (const FBlockRequestAndHash& Request : Requests)
	{
		if (FilenameHash != Request.FilenameMd5)
		{
			if (FilenameHash != InvalidHash)
			{
				Output += "]},\n";	// close blocks arary and file object
			}

			const std::string* FilenameUtf8 = RequestMap.FindSourceFile(Request.FilenameMd5);
			UNSYNC_ASSERTF(FilenameUtf8, L"Could not find file in the block request map");

			// Start file object and blocks array
			Output += "{";

			std::string EscapedFilenameUtf8 = StringEscape(*FilenameUtf8);
			FormatJsonKeyValueStr(Output, "name", EscapedFilenameUtf8, ", ");
			Output += "\"blocks\": [\n";

			BlockIndex	 = 0;
			FilenameHash = Request.FilenameMd5;
		}

		if (BlockIndex != 0)
		{
			Output += ",\n";
		}

		FGenericBlock Block;
		Block.HashStrong = Request.FullHash;
		Block.Offset	 = Request.Offset;
		Block.Size		 = CheckedNarrow(Request.Size);

		FormatJsonBlock(Output, Block);

		++BlockIndex;
	}

	if (FilenameHash != InvalidHash)
	{
		Output += "]}\n";  // close blocks arary and file object
	}

	Output += "]\n";  // files array
	Output += "}\n";  // main object

	return Output;
}

FDownloadResult
FUnsyncHttpProtocolImpl::Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback)
{
	if (!IsValid())
	{
		return FDownloadResult(EDownloadRetryMode::Disconnect);
	}

	std::string RequestJson = FormatBlockRequestJson(*RequestMap, NeedBlocks);

	// TODO: send compressed requests for ~3x upload bandwidth reduction
	// FBuffer CompressedRequest = Compress(reinterpret_cast<const uint8*>(RequestJson.data()), RequestJson.size(), 9);

	auto ChunkCallback = [&CompletionCallback](FHttpResponse& Response)
	{
		if (Response.Success() && Response.Buffer.Size())
		{
			FBuffer			 DecompressedBuffer = Decompress(Response.Buffer);
			FDownloadedBlock DownloadedBlock;
			DownloadedBlock.bCompressed		 = false;
			DownloadedBlock.Data			 = DecompressedBuffer.Data();
			DownloadedBlock.DecompressedSize = DecompressedBuffer.Size();
			FHash128 DecompressedHash		 = HashBlake3Bytes<FHash128>(DecompressedBuffer.Data(), DecompressedBuffer.Size());
			CompletionCallback(DownloadedBlock, DecompressedHash);
		}

		Response.Buffer.Clear();
	};

	FPooledHttpConnection HttpConnection(ProxyPool);

	FHttpRequest Request;
	Request.Method			   = EHttpMethod::POST;
	Request.PayloadContentType = EHttpContentType::Application_Json;
	Request.Payload.Data	   = reinterpret_cast<const uint8*>(RequestJson.data());
	Request.Payload.Size	   = RequestJson.length();
	Request.Url				   = "/api/v1/blocks";

	FHttpResponse Response = HttpRequest(HttpConnection, Request, ChunkCallback);

	if (Response.Success())
	{
		return ResultOk<FDownloadError>();
	}
	else
	{
		UNSYNC_ERROR(L"Failed to complete block request. HTTP error code: %d.", Response.Code);
		return FDownloadError(EDownloadRetryMode::Abort);
	}
}

void
FUnsyncProtocolImpl::Invalidate()
{
	bIsConnetedToHost = false;
	SocketHandle	  = {};
}

ESocketSecurity
FUnsyncProtocolImpl::GetSocketSecurity() const
{
	if (SocketHandle)
	{
		return SocketHandle->Security;
	}
	else
	{
		return ESocketSecurity::None;
	}
}

TResult<ProxyQuery::FHelloResponse>
ProxyQuery::Hello(const FRemoteDesc& RemoteDesc, const FAuthDesc* OptAuthDesc, bool bDisplayAnnouncements)
{
	FTlsClientSettings TlsSettings = RemoteDesc.GetTlsClientSettings();
	FHttpConnection	   Connection(RemoteDesc.Host.Address, RemoteDesc.Host.Port, RemoteDesc.TlsRequirement, TlsSettings, RemoteDesc.RecvTimeoutSeconds);

	return Hello(RemoteDesc.Protocol, Connection, OptAuthDesc, bDisplayAnnouncements);
}

TResult<ProxyQuery::FHelloResponse>
ProxyQuery::Hello(EProtocolFlavor Protocol, FHttpConnection& Connection, const FAuthDesc* OptAuthDesc, bool bDisplayAnnouncements)
{
	TResult<FHelloResponse> Result = [&]() -> TResult<FHelloResponse>
	{
		if (Protocol == EProtocolFlavor::Horde)
		{
			return FHordeProtocolImpl::QueryHello(Connection);
		}
		else if (Protocol == EProtocolFlavor::Unsync)
		{
			return FUnsyncProtocolImpl::QueryHello(Connection, OptAuthDesc);
		}
		else
		{
			return AppError("Protocol does not support server information query");
		}
	}();

	if (Result.IsOk() && bDisplayAnnouncements)
	{
		DisplayAnnouncements(Result->Announcements);
	}

	return Result;
}

TResult<ProxyQuery::FHelloResponse>
FUnsyncProtocolImpl::QueryHello(FHttpConnection& HttpConnection, const FAuthDesc* OptAuthDesc)
{
	using ProxyQuery::FHelloResponse;

	const char* Url = "/api/v1/hello";

	std::string BearerToken;
	if (OptAuthDesc)
	{
		TResult<FAuthToken> AuthTokenResult = Authenticate(*OptAuthDesc, HttpConnection.RecvTimeoutSeconds);
		if (AuthTokenResult.IsOk())
		{
			BearerToken = std::move(AuthTokenResult.GetData().Access);
		}
		else
		{
			return MoveError<ProxyQuery::FHelloResponse>(AuthTokenResult);
		}
	}

	FHttpResponse Response = HttpRequest(HttpConnection, EHttpMethod::GET, Url, {} /*CustomHeaders*/, BearerToken);

	if (!Response.Success())
	{
		UNSYNC_ERROR(L"Failed to establish connection to UNSYNC server. Error code: %d.", Response.Code);
		return HttpError(fmt::format("{}:{}{}", HttpConnection.HostAddress.c_str(), HttpConnection.HostPort, Url), Response.Code);
	}

	FHelloResponse Result;

	using namespace json11;
	std::string JsonString = std::string(Response.AsStringView());

	std::string JsonErrorString;
	Json		JsonObject = Json::parse(JsonString, JsonErrorString);

	if (!JsonErrorString.empty())
	{
		return AppError(std::string("JSON parse error while connecting to UNSYNC server: ") + JsonErrorString);
	}

	if (auto& Field = JsonObject["service"]; Field.is_string())
	{
		const std::string& FieldVal = Field.string_value();
		if (FieldVal != "unsync")
		{
			return AppError(fmt::format("Expected service name 'unsync', but found '{}'.", FieldVal.c_str()));
		}
	}

	if (auto& Field = JsonObject["name"]; Field.is_string())
	{
		Result.Name = Field.string_value();
	}

	if (auto& Field = JsonObject["version"]; Field.is_string())
	{
		Result.VersionNumber = Field.string_value();
	}

	if (auto& Field = JsonObject["git"]; Field.is_string())
	{
		Result.VersionGit = Field.string_value();
	}

	if (auto& Field = JsonObject["session"]; Field.is_string())
	{
		Result.SessionId = Field.string_value();
	}

	if (auto& Field = JsonObject["auth"]; Field.is_object())
	{
		Result.AuthServerUri = Field["server"].string_value();
		Result.AuthClientId	 = Field["client_id"].string_value();
		Result.AuthAudience	 = Field["audience"].string_value();
		Result.CallbackUri	 = Field["callback"].string_value();
	}

	if (auto& Field = JsonObject["features"]; Field.is_array())
	{
		Result.FeatureNames.reserve(Field.array_items().size());
		for (auto& Elem : Field.array_items())
		{
			if (Elem.is_string())
			{
				Result.FeatureNames.push_back(Elem.string_value());

				if (Elem.string_value() == "telemetry")
				{
					Result.Features.bTelemetry = true;
				}
				else if (Elem.string_value() == "mirrors")
				{
					Result.Features.bMirrors = true;
				}
				else if (Elem.string_value() == "authentication")
				{
					Result.Features.bAuthentication = true;
				}
				else if (Elem.string_value() == "list")
				{
					Result.Features.bDirectoryListing = true;
				}
				else if (Elem.string_value() == "file")
				{
					Result.Features.bFileDownload = true;
				}
				else if (Elem.string_value() == "blocks")
				{
					Result.Features.bBlockDownload = true;
				}
				else if (Elem.string_value() == "announcements")
				{
					Result.Features.bAnnouncements = true;
				}
			}
		}
	}

	if (auto& Field = JsonObject["announcements"]; Field.is_array())
	{
		for (const auto& Elem : Field.array_items())
		{
			if (!Elem.is_object())
			{
				continue;
			}

			ProxyQuery::FAnnouncement Announcement;

			const auto& IdField = Elem["id"];
			if (!IdField.is_string() || !ParseUint64FromHexString(IdField.string_value(), Announcement.Id))
			{
				continue;
			}

			Announcement.Level	 = Elem["level"].string_value();
			Announcement.Message = Elem["message"].string_value();

			Result.Announcements.push_back(std::move(Announcement));
		}
	}

	if (auto& Field = JsonObject["primary"]; Field.is_string())
	{
		const std::string&	 PrimaryHostStr	 = Field.string_value();
		TResult<FRemoteDesc> PrimaryHostDesc = FRemoteDesc::FromUrl(PrimaryHostStr);
		if (PrimaryHostDesc.IsOk())
		{
			Result.PrimaryHost = PrimaryHostDesc->Host;
		}
	}

	Result.bConnectionEncrypted = HttpConnection.IsEncrypted();

	return ResultOk(std::move(Result));
}

bool GNoServerAnnouncements = false;

void
ProxyQuery::DisplayAnnouncements(const std::vector<ProxyQuery::FAnnouncement>& Announcements)
{
	if (GNoServerAnnouncements || Announcements.empty())
	{
		return;
	}

	static std::unordered_set<uint64> DisplayedIds;

	for (const FAnnouncement& Announcement : Announcements)
	{
		if (!DisplayedIds.insert(Announcement.Id).second)
		{
			continue;
		}

		// Split on newlines so each line gets the appropriate log prefix
		const std::string& Msg = Announcement.Message;
		uint64 Pos = 0;
		while (Pos <= Msg.size())
		{
			uint64 End = Msg.find('\n', Pos);
			if (End == std::string::npos)
			{
				End = Msg.size();
			}

			std::string Line = Msg.substr(Pos, End - Pos);

			if (Announcement.Level == "error")
			{
				UNSYNC_ERROR(L"%hs", Line.c_str());
			}
			else if (Announcement.Level == "warning")
			{
				UNSYNC_WARNING(L"%hs", Line.c_str());
			}
			else
			{
				UNSYNC_LOG(L"%hs", Line.c_str());
			}

			Pos = End + 1;
		}
	}
}

TResult<ProxyQuery::FDirectoryListing>
ProxyQuery::FDirectoryListing::FromJson(const char* JsonString)
{
	FDirectoryListing Result;

	std::string	 JsonErrorString;
	json11::Json JsonObject = json11::Json::parse(JsonString, JsonErrorString);

	if (!JsonErrorString.empty())
	{
		return AppError(fmt::format("JSON error: {}", JsonErrorString.c_str()));
	}

	const json11::Json& EntriesObject = JsonObject["entries"];

	if (EntriesObject.is_array())
	{
		for (const auto& Elem : EntriesObject.array_items())
		{
			FDirectoryListingEntry Entry;
			for (const auto& Field : Elem.object_items())
			{
				if (Field.first == "name" && Field.second.is_string())
				{
					Entry.Name = Field.second.string_value();
				}
				else if (Field.first == "is_directory" && Field.second.is_bool())
				{
					Entry.bDirectory = Field.second.bool_value();
				}
				else if (Field.first == "mtime" && Field.second.is_number())
				{
					Entry.Mtime = uint64(Field.second.number_value());
				}
				else if (Field.first == "size" && Field.second.is_number())
				{
					Entry.Size = uint64(Field.second.number_value());
				}
			}
			Result.Entries.push_back(Entry);
		}
	}

	return ResultOk(std::move(Result));
}

std::string
ProxyQuery::FDirectoryListing::ToJson() const
{
	std::string Result;

	Result += "{\"entries\": [\n";

	uint64 EntryIndex = 0;
	for (const FDirectoryListingEntry& Entry : Entries)
	{
		if (EntryIndex != 0)
		{
			Result += ",\n";
		}

		Result += "{ ";
		FormatJsonKeyValueStr(Result, "name", StringEscape(Entry.Name), ", ");
		FormatJsonKeyValueBool(Result, "is_directory", Entry.bDirectory, ", ");
		FormatJsonKeyValueUInt(Result, "mtime", Entry.Mtime, ", ");
		FormatJsonKeyValueUInt(Result, "size", Entry.Size);
		Result += "}";

		++EntryIndex;
	}

	Result += "\n]}\n";

	return Result;
}

TResult<ProxyQuery::FDirectoryListing>
ProxyQuery::ListDirectory(EProtocolFlavor Protocol, FHttpConnection& Connection, const FAuthDesc* AuthDesc, const std::string& Path)
{
	if (Protocol == EProtocolFlavor::Horde)
	{
		return FHordeProtocolImpl::QueryListDirectory(Connection, AuthDesc, Path);
	}
	else if (Protocol == EProtocolFlavor::Unsync)
	{
		return FUnsyncProtocolImpl::QueryListDirectory(Connection, AuthDesc, Path);
	}
	else
	{
		return AppError("Protocol does not support server directory listing");
	}
}

TResult<ProxyQuery::FDirectoryListing>
FUnsyncProtocolImpl::QueryListDirectory(FHttpConnection& Connection, const FAuthDesc* AuthDesc, const std::string& Path)
{
	using ProxyQuery::FDirectoryListing;

	std::string Url = fmt::format("/api/v1/list?{}", Path);

	std::string BearerToken;
	if (AuthDesc)
	{
		TResult<FAuthToken> AuthToken = Authenticate(*AuthDesc, Connection.RecvTimeoutSeconds);
		if (!AuthToken.IsOk())
		{
			return MoveError<FDirectoryListing>(AuthToken);
		}

		BearerToken = std::move(AuthToken->Access);
	}

	FHttpRequest Request;
	Request.Url			= Url;
	Request.Method		= EHttpMethod::GET;
	Request.BearerToken = BearerToken;

	FHttpResponse Response = HttpRequest(Connection, Request);

	if (!Response.Success())
	{
		std::string ErrorContext = fmt::format("Listing server directory: {}", Path);
		return HttpError(std::move(ErrorContext), Response.Code);
	}

	Response.Buffer.PushBack(0);

	return FDirectoryListing::FromJson((const char*)Response.Buffer.Data());
}

TResult<>
ProxyQuery::DownloadFile(FHttpConnection&					 InConnection,
						 const FAuthDesc*					 AuthDesc,
						 const std::string&					 Path,
						 ProxyQuery::FDownloadOutputCallback OutputCallback)
{
	TObjectPool<FHttpConnection> ConnectionPool(
		[&InConnection]
		{
			return new FHttpConnection(InConnection);  // Clone the connection
		});

	std::string Url = fmt::format("/api/v1/file?{}", Path);

	uint64 FileSize = 0;

	{
		std::unique_ptr<FHttpConnection> Connection = ConnectionPool.Acquire();

		std::string BearerToken;
		if (AuthDesc)
		{
			TResult<FAuthToken> AuthToken = Authenticate(*AuthDesc, Connection->RecvTimeoutSeconds);
			if (!AuthToken.IsOk())
			{
				return std::move(AuthToken.GetError());
			}
			BearerToken = std::move(AuthToken->Access);
		}

		FHttpRequest HeadRequest;
		HeadRequest.Url			   = Url;
		HeadRequest.Method		   = EHttpMethod::HEAD;
		HeadRequest.BearerToken	   = BearerToken;
		FHttpResponse HeadResponse = HttpRequest(*Connection, HeadRequest);
		if (!HeadResponse.Success())
		{
			return HttpError(HeadResponse.Code);
		}

		FileSize = HeadResponse.ContentLength;
		UNSYNC_VERBOSE(L"File size: %llu bytes (%.3f MB)", llu(FileSize), SizeMb(FileSize));

		ConnectionPool.Release(std::move(Connection));
	}

	const uint64 MaxChunkSize = 8_MB;

	FIOWriter& Result = OutputCallback(FileSize);
	if (!Result.IsValid())
	{
		return AppError(L"Failed to create download output stream");
	}

	const uint64		NumChunks = DivUp(FileSize, MaxChunkSize);
	std::vector<FRange> Chunks;
	Chunks.reserve(NumChunks);

	for (uint64 i = 0; i < NumChunks; ++i)
	{
		FRange Range;
		Range.Offset = i * MaxChunkSize;
		Range.Size	 = CalcChunkSize(i, MaxChunkSize, FileSize);
		Chunks.push_back(Range);
	}

	FAtomicError Error;

	FLogProgressScope DownloadProgress(FileSize, ELogProgressUnits::MB);

	std::string BearerToken;
	if (AuthDesc)
	{
		TResult<FAuthToken> AuthToken = Authenticate(*AuthDesc, InConnection.RecvTimeoutSeconds);
		if (!AuthToken.IsOk())
		{
			Error.Set(std::move(AuthToken.GetError()));
			return AppError(L"Failed to acquire access token");
		}
		BearerToken = std::move(AuthToken->Access);
	}

	auto ProcessChunk =
		[&Error, &Result, &Url, &ConnectionPool, &DownloadProgress, &BearerToken](
			const FRange& Range)
	{
		FLogIndentScope	   IndentScope(DownloadProgress.ParentThreadIndent, true);
		FLogVerbosityScope VerbosityScope(DownloadProgress.bParentThreadVerbose);

		if (Error.Test())
		{
			return;
		}

		GScheduler->NetworkSemaphore.Acquire();

		std::unique_ptr<FHttpConnection> Connection = ConnectionPool.Acquire();

		std::string RequestHeaders = fmt::format("Range: bytes={}-{}", Range.Offset, Range.Offset + Range.Size - 1);

		FHttpRequest Request;
		Request.Url			   = Url;
		Request.Method		   = EHttpMethod::GET;
		Request.BearerToken	   = BearerToken;
		Request.CustomHeaders  = RequestHeaders;
		FHttpResponse Response = HttpRequest(*Connection, Request);

		if (!Response.Success())
		{
			Error.Set(HttpError(Response.Code));
			return;
		}

		if (Range.Size != Response.Buffer.Size())
		{
			std::string ErrorMessage = fmt::format("Downloaded file chunk size mismatch. Expected {} bytes, got {} byte.", Range.Size, Response.Buffer.Size());
			Error.Set(AppError(std::move(ErrorMessage)));
			return;
		}

		uint64 WrittenBytes = Result.Write(Response.Buffer.Data(), Range.Offset, Range.Size);

		DownloadProgress.Add(WrittenBytes);

		ConnectionPool.Release(std::move(Connection));

		GScheduler->NetworkSemaphore.Release();
	};

	ParallelForEach(Chunks, ProcessChunk);

	DownloadProgress.Complete();

	if (Error.Test())
	{
		return std::move(*Error.Data);
	}

	return ResultOk();
}

TResult<FBuffer>
ProxyQuery::DownloadFile(FHttpConnection& Connection, const FAuthDesc* AuthDesc, const std::string& Path)
{
	FBuffer Result;

	std::unique_ptr<FMemReaderWriter> ResultWriter;

	auto OutputCallback = [&Result, &ResultWriter](uint64 Size) -> FIOWriter&
	{
		Result.Resize(Size);
		ResultWriter = std::make_unique<FMemReaderWriter>(Result.Data(), Result.Size());
		return *ResultWriter;
	};

	TResult<> DownloadResult = DownloadFile(Connection, AuthDesc, Path, OutputCallback);
	if (DownloadResult.IsOk())
	{
		return ResultOk(std::move(Result));
	}
	else
	{
		return MoveError<FBuffer>(DownloadResult);
	}
}

void
FUnsyncProtocolImpl::SendTelemetryEvent(const FRemoteDesc& RemoteDesc, const FTelemetryEventSyncComplete& Event)
{
	using namespace json11;
	json11::Json::object Obj;

	if (!Event.Session.empty())
	{
		Obj["session"] = Event.Session;
	}

	Obj["type"]			  = "sync_complete";
	Obj["client_version"] = Event.ClientVersion;
	if (!Event.ClientHostNameHash.empty())
	{
		Obj["client_host_name_hash"] = Event.ClientHostNameHash;
	}
	Obj["source"]	 = Event.Source;
	Obj["total_mb"]	 = SizeMb(Event.TotalBytes);  // use size in megabytes due lack of JSON 64 bit int support and better human readability
	Obj["source_mb"] = SizeMb(Event.SourceBytes);
	Obj["base_mb"]	 = SizeMb(Event.BaseBytes);
	Obj["files_skipped"] = int(Event.SkippedFiles);
	Obj["files_full"]	 = int(Event.FullCopyFiles);
	Obj["files_partial"] = int(Event.PartialCopyFiles);
	Obj["elapsed"]		 = Event.Elapsed;
	Obj["success"]		 = Event.bSuccess;

	std::string EventJson = Json(Obj).dump();

	FBufferView EventJsonView;
	EventJsonView.Data = (const uint8*)EventJson.c_str();
	EventJsonView.Size = (uint64)EventJson.length();

	const char* Url = "/api/v1/telemetry";
	HttpRequest(RemoteDesc, EHttpMethod::POST, Url, EHttpContentType::Application_Json, EventJsonView);
}

FUnsyncProtocolImpl::~FUnsyncProtocolImpl()
{
	if (IsValid())
	{
		FCommandPacket Packet;
		Packet.CommandId = COMMAND_ID_DISCONNECT;
		SendStruct(*SocketHandle, Packet);
	}

	SocketHandle = {};

	bIsConnetedToHost = false;
}

FHash128
FBlockRequestMap::AddFile(const FPath& OriginalFilePath, const FPath& ResolvedFilePath)
{
	std::string OriginalFilePathUtf8 = ConvertWideToUtf8(OriginalFilePath.wstring());
	std::string ResolvedFilePathUtf8 = ConvertWideToUtf8(ResolvedFilePath.wstring());

	FHash128 OriginalNameHash = HashMd5Bytes((const uint8*)OriginalFilePathUtf8.c_str(), OriginalFilePathUtf8.length());
	FHash128 ResolvedNameHash = HashMd5Bytes((const uint8*)ResolvedFilePathUtf8.c_str(), ResolvedFilePathUtf8.length());

	auto FindResult = HashToFile.find(OriginalNameHash);
	if (FindResult == HashToFile.end())
	{
		HashToFile[OriginalNameHash] = uint32(SourceFileListUtf8.size());
		HashToFile[ResolvedNameHash] = uint32(SourceFileListUtf8.size());
		SourceFileListUtf8.push_back(OriginalFilePathUtf8);
	}

	return OriginalNameHash;
}

void
FBlockRequestMap::AddPackBlocks(const FPath&					  OriginalFilePath,
								const FPath&					  ResolvedFilePath,
								const TArrayView<FPackIndexEntry> PackManifest)
{
	UNSYNC_ASSERTF(StrongHasher != EStrongHashAlgorithmID::Invalid, L"Request map is not initialized");

	FHash128 FileId = AddFile(OriginalFilePath, ResolvedFilePath);

	for (const FPackIndexEntry& Block : PackManifest)
	{
		FBlockRequestEx Request;
		Request.FilenameMd5				 = FileId;
		Request.BlockHash				 = Block.BlockHash;
		Request.Offset					 = Block.PackBlockOffset;
		Request.Size					 = Block.PackBlockSize;
		BlockRequests[Request.BlockHash] = Request;
	}
}

void
FBlockRequestMap::AddFileBlocks(uint32 SourceId, const FPath& OriginalFilePath, const FPath& ResolvedFilePath, const FFileManifest& FileManifest)
{
	UNSYNC_ASSERTF(StrongHasher != EStrongHashAlgorithmID::Invalid, L"Request map is not initialized");

	FHash128 FileId = AddFile(OriginalFilePath, ResolvedFilePath);

	for (const FGenericBlock& Block : FileManifest.Blocks)
	{
		FBlockRequestEx Request;
		Request.FilenameMd5				 = FileId;
		Request.BlockHash				 = Block.HashStrong.ToHash128();  // #wip-widehash
		Request.Offset					 = Block.Offset;
		Request.Size					 = Block.Size;
		Request.SourceId				 = SourceId;
		BlockRequests[Request.BlockHash] = Request;

		if (!FileManifest.MacroBlocks.empty())
		{
			// TODO: could also just do the search in GetMacroBlockRequest() directly instead of pre-caching
			// TODO: since we know that blocks and macro-blocks are sorted, then we don't need a binary search here
			auto MacroBlockIt = std::lower_bound(FileManifest.MacroBlocks.begin(),
												 FileManifest.MacroBlocks.end(),
												 Block.Offset,
												 [](const FGenericBlock& A, uint64 B) { return (A.Offset + A.Size) < (B + 1); });

			if (MacroBlockIt != FileManifest.MacroBlocks.end())
			{
				const FGenericBlock& MacroBlock = *MacroBlockIt;
				UNSYNC_ASSERT(Block.Offset >= MacroBlock.Offset);
				UNSYNC_ASSERT(Block.Offset + Block.Size <= MacroBlock.Offset + MacroBlock.Size);

				FHash128 RequestKey = Block.HashStrong.ToHash128();

				if (MacroBlockRequests.find(RequestKey) == MacroBlockRequests.end())
				{
					FMacroBlockRequest MacroRequest;
					MacroRequest.Hash				  = MacroBlock.HashStrong;
					MacroRequest.Offset				  = Block.Offset - MacroBlock.Offset;
					MacroRequest.Size				  = Block.Size;
					MacroRequest.MacroBlockBaseOffset = MacroBlock.Offset;
					MacroRequest.MacroBlockTotalSize  = MacroBlock.Size;
					MacroBlockRequests[RequestKey]	  = MacroRequest;
				}
			}
			else
			{
				UNSYNC_FATAL(L"Found a block that does not belong to any macro block.");
			}
		}
	}
}

const FBlockRequestMap::FBlockRequestEx*
FBlockRequestMap::FindRequest(const FGenericHash& BlockHash) const
{
	FHash128 BlockHash128 = BlockHash.ToHash128();

	auto It = BlockRequests.find(BlockHash128);
	if (It == BlockRequests.end())
	{
		return nullptr;
	}
	else
	{
		return &It->second;
	}
}

const std::string*
FBlockRequestMap::FindSourceFile(const FHash128& NameHashMd5) const
{
	auto It = HashToFile.find(NameHashMd5);
	if (It == HashToFile.end())
	{
		return nullptr;
	}
	else
	{
		return &SourceFileListUtf8[It->second];
	}
}

FMacroBlockRequest
FBlockRequestMap::GetMacroBlockRequest(const FGenericHash& BlockHash) const
{
	FMacroBlockRequest Result = {};
	auto			   It	  = MacroBlockRequests.find(BlockHash.ToHash128());
	if (It != MacroBlockRequests.end())
	{
		Result = It->second;
	}
	return Result;
}

FProxyPool::FProxyPool() : FProxyPool(FRemoteDesc(), nullptr)
{
}

FProxyPool::FProxyPool(const FRemoteDesc& InRemoteDesc, const FAuthDesc* InAuthDesc)
: RemoteDesc(InRemoteDesc)
, AuthDesc(InAuthDesc)
, bValid(InRemoteDesc.IsValid())
{
	if (!bValid)
	{
		return;
	}

	auto CreateHttpConnection = [Remote = RemoteDesc]
	{
		FTlsClientSettings TlsSettings = Remote.GetTlsClientSettings();
		return new FHttpConnection(Remote.Host.Address, Remote.Host.Port, Remote.TlsRequirement, TlsSettings, Remote.RecvTimeoutSeconds);
	};

	HttpPool.emplace(CreateHttpConnection);

	if (RemoteDesc.Protocol == EProtocolFlavor::Unsync)
	{
		UNSYNC_VERBOSE(L"Connecting to %hs server '%hs:%d' ...",
					   ToString(RemoteDesc.Protocol),
					   RemoteDesc.Host.Address.c_str(),
					   RemoteDesc.Host.Port);

		std::unique_ptr<FHttpConnection> HttpConnection = HttpPool->Acquire();

		TResult<ProxyQuery::FHelloResponse> Response = ProxyQuery::Hello(RemoteDesc.Protocol, *HttpConnection, AuthDesc);

		HttpPool->Release(std::move(HttpConnection));

		if (Response.IsError())
		{
			LogError(Response.GetError(), L"Failed to query basic server information");
		}
		else
		{
			const ProxyQuery::FHelloResponse& Data = Response.GetData();
			UNSYNC_VERBOSE(L"Connection established. Server name: %hs, version: %hs, git: %hs, tls: %hs",
						   Data.Name.empty() ? "unknown" : Data.Name.c_str(),
						   Data.VersionNumber.empty() ? "unknown" : Data.VersionNumber.c_str(),
						   Data.VersionGit.empty() ? "unknown" : Data.VersionGit.c_str(),
						   (Data.bConnectionEncrypted) ? "yes" : "no"
				);

			Features  = Data.Features;
			SessionId = Data.SessionId;
		}

		bValid = Response.IsOk();
	}
	else if (RemoteDesc.Protocol == EProtocolFlavor::Jupiter)
	{
		Features.bAuthentication = true;
		Features.bManifestDownload = true;
	}
	else if (RemoteDesc.Protocol == EProtocolFlavor::Horde)
	{
		Features.bAuthentication   = true;
		Features.bBlockDownload	   = true;
		Features.bFileDownload	   = true;
		Features.bManifestDownload = true;
	}
}

std::unique_ptr<FProxy>
FProxyPool::Alloc()
{
	if (!bValid)
	{
		return nullptr;
	}

	std::lock_guard<std::mutex> LockGuard(Mutex);
	std::unique_ptr<FProxy>		Result;
	if (!Pool.empty())
	{
		std::swap(Pool.back(), Result);
		Pool.pop_back();
	}

	if (!Result || !Result->IsValid())
	{
 		Result = std::make_unique<FProxy>(*this, RemoteDesc, Features, AuthDesc, &RequestMap);
	}

	return Result;
}

void
FProxyPool::Dealloc(std::unique_ptr<FProxy>&& Proxy)
{
	if (Proxy.get() && Proxy->IsValid())
	{
		std::lock_guard<std::mutex> LockGuard(Mutex);
		Pool.push_back(std::move(Proxy));
	}
}

std::unique_ptr<FHttpConnection>
FProxyPool::AllocHttp()
{
	if (!bValid || !HttpPool)
	{
		return nullptr;
	}
	return HttpPool->Acquire();
}
void
FProxyPool::DeallocHttp(std::unique_ptr<FHttpConnection>&& Connection)
{
	if (Connection && HttpPool)
	{
		HttpPool->Release(std::move(Connection));
	}
}

std::string
FProxyPool::GetAccessToken()
{
	std::string Result;

	if (AuthDesc)
	{
		TResult<FAuthToken> AuthTokenResult = Authenticate(*AuthDesc, RemoteDesc.RecvTimeoutSeconds);
		if (FAuthToken* AuthToken = AuthTokenResult.TryData())
		{
			std::swap(Result, AuthToken->Access);
		}
		else
		{
			LogError(AuthTokenResult.GetError(), L"Failed to authenticate");
			UNSYNC_FATAL(L"Cannot proceed without a valid authentication token");
		}
	}

	return Result;
}

void
FProxyPool::Invalidate()
{
	bValid = false;
}

bool
FProxyPool::IsValid() const
{
	return bValid;
}

void
FProxyPool::SendTelemetryEvent(const FTelemetryEventSyncComplete& Event)
{
	if (RemoteDesc.Protocol == EProtocolFlavor::Unsync && Features.bTelemetry)
	{
		FUnsyncProtocolImpl::SendTelemetryEvent(RemoteDesc, Event);
	}
}

void
FProxyPool::SetRequestMap(FBlockRequestMap&& InRequestMap)
{
	std::lock_guard<std::mutex> LockGuard(Mutex);
	RequestMap = std::move(InRequestMap);
}

FPhysicalFileSystem::FPhysicalFileSystem(const FPath& InRoot) : Root(InRoot)
{
}

TResult<FProxyDirectoryListing>
FPhysicalFileSystem::ListDirectory(const std::string_view RelativePath)
{
	FProxyDirectoryListing Result;

	std::wstring RelativePathWide = ConvertUtf8ToWide(RelativePath);
	FPath		 FullPath		  = Root / FPath(RelativePathWide);

	for (const std::filesystem::directory_entry& Dir : DirectoryScan(FullPath))
	{
		FPath FileName = Dir.path().filename();

		FProxyDirectoryEntry Entry;
		Entry.bDirectory = Dir.is_directory();
		Entry.Size		 = Entry.bDirectory ? 0 : Dir.file_size();
		Entry.Mtime		 = ToWindowsFileTime(Dir.last_write_time());
		Entry.Name		 = ToString(FileName);

		Result.Entries.emplace_back(std::move(Entry));
	}

	return ResultOk(std::move(Result));
}

TResult<FBuffer>
FPhysicalFileSystem::ReadFile(const std::string_view RelativePath)
{
	std::wstring RelativePathWide = ConvertUtf8ToWide(RelativePath);
	FPath		 FullPath		  = Root / FPath(RelativePathWide);
	FBuffer		 Buffer			  = ReadFileToBuffer(FullPath);

	if (Buffer.Empty())
	{
		// TODO: ReadFileToBuffer should return an error code or TResult
		return AppError(fmt::format(L"Could not read file '{}'", FullPath.wstring()));
	}
	else
	{
		return ResultOk(std::move(Buffer));
	}
}

TResult<FProxyDirectoryListing>
FRemoteFileSystem::ListDirectory(const std::string_view RelativePath)
{
	FPooledHttpConnection HttpConnection(ProxyPool);
	std::string			  FullPath;
	FullPath.append(Root);
	if (!RelativePath.empty())
	{
		FullPath.append("/");
		FullPath.append(RelativePath);
	}
	ConvertDirectorySeparatorsToUnix(FullPath);
	return ProxyQuery::ListDirectory(ProxyPool.RemoteDesc.Protocol, HttpConnection, ProxyPool.AuthDesc, FullPath);
}

TResult<FBuffer>
FRemoteFileSystem::ReadFile(const std::string_view RelativePath)
{
	FPooledHttpConnection HttpConnection(ProxyPool);
	std::string			  FullPath;
	FullPath.append(Root);
	FullPath.append("/");
	FullPath.append(RelativePath);
	ConvertDirectorySeparatorsToUnix(FullPath);
	return ProxyQuery::DownloadFile(HttpConnection, ProxyPool.AuthDesc, FullPath);
}

}  // namespace unsync
