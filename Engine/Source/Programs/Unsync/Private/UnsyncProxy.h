// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCore.h"
#include "UnsyncError.h"
#include "UnsyncRemote.h"
#include "UnsyncSocket.h"
#include "UnsyncThread.h"
#include "UnsyncUtil.h"
#include "UnsyncPool.h"
#include "UnsyncHttp.h"

#include <functional>
#include <mutex>
#include <unordered_map>
#include <optional>

namespace unsync {

struct FDirectoryManifest;
struct FAuthDesc;
struct FHttpConnection;

enum class EDownloadRetryMode
{
	Retry,		 // potentially recoverable error (caller can retry the same request)
	Abort,		 // issuing the same request will likely fail, but other requests may succeed
	Disconnect,	 // further server API calls are likely to fail
};

struct FDownloadError : FError
{
	FDownloadError() = default;
	FDownloadError(EDownloadRetryMode InRetryMode) : RetryMode(InRetryMode) {}

	EDownloadRetryMode RetryMode = EDownloadRetryMode::Abort;

	bool CanRetry() const { return RetryMode == EDownloadRetryMode::Retry; }
};

using FDownloadResult = TResult<FEmpty, FDownloadError>;

struct FDownloadedBlock
{
	uint64		 DecompressedSize = 0;
	uint64		 CompressedSize	  = 0;
	const uint8* Data			  = nullptr;
	bool		 bCompressed	  = false;
};

using FBlockDownloadCallback = std::function<void(const FDownloadedBlock& Block, FHash128 BlockHash)>;

struct FMacroBlockRequest
{
	FGenericHash Hash	= {};
	uint64		 Offset = 0;
	uint64		 Size	= 0;

	uint64 MacroBlockBaseOffset = 0;
	uint64 MacroBlockTotalSize	= 0;

	bool IsValid() const { return Size != 0; }
};

class FBlockRequestMap
{
public:
	void Init(EStrongHashAlgorithmID InStrongHasher, const std::vector<FPath>& InSourceRoots)
	{
		UNSYNC_ASSERTF(StrongHasher == EStrongHashAlgorithmID::Invalid, L"Request map is already initialized");
		StrongHasher = InStrongHasher;
		SourceRoots	 = InSourceRoots;
	}

	void AddFileBlocks(uint32 SourceId, const FPath& OriginalFilePath, const FPath& ResolvedFilePath, const FFileManifest& Manifest);
	void AddPackBlocks(const FPath& OriginalFilePath, const FPath& ResolvedFilePath, const TArrayView<FPackIndexEntry> PackManifest);

	struct FBlockRequestEx : FBlockRequest
	{
		uint32 SourceId = ~0u;
	};

	const std::vector<std::string>& GetSourceFileList() const { return SourceFileListUtf8; }
	const FBlockRequestEx*			FindRequest(const FGenericHash& BlockHash) const;
	const std::string*				FindSourceFile(const FHash128& NameHashMd5) const;
	EStrongHashAlgorithmID			GetStrongHasher() const { return StrongHasher; }
	FMacroBlockRequest				GetMacroBlockRequest(const FGenericHash& BlockHash) const;
	const std::vector<FPath>&		GetSourceRoots() const { return SourceRoots; }

private:

	FHash128 AddFile(const FPath& OriginalFilePath, const FPath& ResolvedFilePath);

	EStrongHashAlgorithmID							 StrongHasher = EStrongHashAlgorithmID::Invalid;
	std::vector<std::string>						 SourceFileListUtf8;
	std::unordered_map<FHash128, uint32>			 HashToFile;
	std::unordered_map<FHash128, FBlockRequestEx>	 BlockRequests;
	std::unordered_map<FHash128, FMacroBlockRequest> MacroBlockRequests;
	std::vector<FPath>								 SourceRoots;
};

struct FRemoteProtocolFeatures
{
	bool bTelemetry		   = false;
	bool bMirrors		   = false;
	bool bAuthentication   = false;
	bool bDirectoryListing = false;
	bool bFileDownload	   = false;
	bool bManifestDownload = false;
	bool bBlockDownload	   = false;
	bool bAnnouncements	   = false;
};

struct FTelemetryEventSyncComplete
{
	std::string ClientVersion;
	std::string Session;
	std::string Source;
	std::string ClientHostNameHash;
	uint64 TotalBytes = 0;
	uint64 SourceBytes = 0;
	uint64 BaseBytes = 0;
	uint32 SkippedFiles = 0;
	uint32 FullCopyFiles = 0;
	uint32 PartialCopyFiles = 0;
	double Elapsed = 0;
	bool bSuccess = false;
};

struct FRemoteProtocolBase
{
	FRemoteProtocolBase(const FRemoteDesc& InRemoteDesc, const FBlockRequestMap* InRequestMap)
	: RequestMap(InRequestMap)
	, RemoteDesc(InRemoteDesc)
	{
	}

	virtual ~FRemoteProtocolBase(){};

	virtual bool Contains(const FDirectoryManifest& Manifest) { return true; }
	virtual bool IsValid() const							  = 0;
	virtual void Invalidate()								  = 0;

	virtual FDownloadResult Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback) = 0;
	virtual TResult<FDirectoryManifest> DownloadManifest(std::string_view ManifestName)											= 0;

	const FBlockRequestMap* RequestMap;
	FRemoteDesc				RemoteDesc;
};

// TODO:
// - transparently go through proxy when reading blocks during patching
//   i.e. read blocks from disk / network share or from proxy automatically
class FProxy
{
public:
	FProxy(FProxyPool&					  ProxyPool,
		   const FRemoteDesc&			  InRemoteDesc,
		   const FRemoteProtocolFeatures& InFeatures,
		   const FAuthDesc*				  InAuthDesc,
		   const FBlockRequestMap*		  InRequestMap);
	~FProxy();

	bool Contains(const FDirectoryManifest& Manifest);
	bool IsValid() const;

	FDownloadResult				Download(const TArrayView<FNeedBlock> NeedBlocks, const FBlockDownloadCallback& CompletionCallback);
	TResult<FDirectoryManifest> DownloadManifest(std::string_view ManifestName);

private:
	std::unique_ptr<FRemoteProtocolBase> ProtocolImpl;
};

class FProxyPool
{
public:
	FProxyPool();
	FProxyPool(const FRemoteDesc& InRemoteDesc, const FAuthDesc* InAuthDesc);

	std::unique_ptr<FProxy> Alloc();
	void					Dealloc(std::unique_ptr<FProxy>&& Proxy);

	std::unique_ptr<FHttpConnection> AllocHttp();
	void							 DeallocHttp(std::unique_ptr<FHttpConnection>&& Connection);

	std::string GetAccessToken();

	bool SupportsHttp() const { return HttpPool.has_value(); }

	void Invalidate();
	bool IsValid() const;

	const FRemoteDesc RemoteDesc;
	const FAuthDesc* AuthDesc = nullptr; // optional reference to externally-owned auth parameters

	void SetRequestMap(FBlockRequestMap&& InRequestMap);

	const FRemoteProtocolFeatures& GetFeatures() const { return Features; }
	const std::string& GetSessionId() const { return SessionId; }

	void SendTelemetryEvent(const FTelemetryEventSyncComplete& Event);

private:
	std::vector<std::unique_ptr<FProxy>> Pool;
	bool								 bValid = true;

	std::optional<TObjectPool<FHttpConnection>> HttpPool;

	FRemoteProtocolFeatures Features;
	std::string SessionId;

	FBlockRequestMap RequestMap;

	std::mutex Mutex;
};

struct FPooledHttpConnection
{
	FPooledHttpConnection(FProxyPool& InProxyPool) : ProxyPool(InProxyPool) { Inner = ProxyPool.AllocHttp(); }
	~FPooledHttpConnection() { ProxyPool.DeallocHttp(std::move(Inner)); }
	FHttpConnection* Get() { return Inner.get(); }
	FHttpConnection& operator*() { return *Get(); }
	FHttpConnection* operator->() { return Get(); }
	operator FHttpConnection&() { return *Get(); }
	bool							 IsValid() const { return ProxyPool.IsValid() && Inner.get(); }
	FProxyPool&						 ProxyPool;
	std::unique_ptr<FHttpConnection> Inner;
};

extern bool GNoServerAnnouncements;

namespace ProxyQuery {

struct FAnnouncement
{
	uint64		Id = 0;	 // 64-bit hash, parsed from hex string in JSON
	std::string Level;	 // "info", "warning", "error"
	std::string Message;
};

struct FHelloResponse
{
	std::string Name;
	std::string VersionNumber;
	std::string VersionGit;
	std::string SessionId;

	std::string AuthServerUri;
	std::string AuthClientId;
	std::string AuthAudience;
	std::string CallbackUri;

	std::vector<std::string> FeatureNames;
	FRemoteProtocolFeatures	 Features;

	std::optional<FHostAddressAndPort> PrimaryHost;

	std::vector<FAnnouncement> Announcements;

	// Derived data

	bool bConnectionEncrypted = false;

	bool SupportsAuthentication() const { return Features.bAuthentication && !AuthServerUri.empty() && !AuthClientId.empty(); }
};
TResult<FHelloResponse> Hello(const FRemoteDesc& RemoteDesc, const FAuthDesc* OptAuthDesc = nullptr, bool bDisplayAnnouncements = true);
TResult<FHelloResponse> Hello(EProtocolFlavor Protocol, FHttpConnection& Connection, const FAuthDesc* OptAuthDesc = nullptr, bool bDisplayAnnouncements = true);

void DisplayAnnouncements(const std::vector<FAnnouncement>& Announcements);

struct FDirectoryListingEntry
{
	std::string Name;  // utf-8
	uint64		Mtime	   = 0;
	uint64		Size	   = 0;
	bool		bDirectory = false;
};

struct FDirectoryListing
{
	std::vector<FDirectoryListingEntry> Entries;

	static TResult<FDirectoryListing> FromJson(const char* JsonString);
	std::string						  ToJson() const;
};

TResult<FDirectoryListing> ListDirectory(EProtocolFlavor	Protocol,
										 FHttpConnection&	Connection,
										 const FAuthDesc*	AuthDesc,
										 const std::string& Path);

TResult<FBuffer>		   DownloadFile(FHttpConnection& Connection, const FAuthDesc* AuthDesc, const std::string& Path);

using FDownloadOutputCallback = std::function<FIOWriter&(uint64 Size)>;
TResult<> DownloadFile(FHttpConnection&		   Connection,
					   const FAuthDesc*		   AuthDesc,
					   const std::string&	   Path,
					   FDownloadOutputCallback OutputCallback);

}

using FProxyDirectoryListing = ProxyQuery::FDirectoryListing;
using FProxyDirectoryEntry	 = ProxyQuery::FDirectoryListingEntry;

// Abstracts basic filesystem operations, such as directory listing and file download.
// Can be used to transparently handle basic local and remote file operations.
struct FProxyFileSystem
{
	virtual TResult<FProxyDirectoryListing> ListDirectory(const std::string_view RelativePath) = 0;
	virtual TResult<FBuffer>				ReadFile(const std::string_view RelativePath)	   = 0;

	virtual ~FProxyFileSystem() = default;
};

struct FPhysicalFileSystem : public FProxyFileSystem
{
	FPhysicalFileSystem(const FPath& InRoot);

	virtual TResult<FProxyDirectoryListing> ListDirectory(const std::string_view RelativePath) final override;
	virtual TResult<FBuffer>				ReadFile(const std::string_view RelativePath) final override;

	FPath Root;
};

struct FRemoteFileSystem : public FProxyFileSystem
{
	FRemoteFileSystem(const std::string& InRoot, FProxyPool& InProxyPool) : Root(InRoot), ProxyPool(InProxyPool) {}

	virtual TResult<FProxyDirectoryListing> ListDirectory(const std::string_view RelativePath) final override;
	virtual TResult<FBuffer>				ReadFile(const std::string_view RelativePath) final override;

	std::string	Root;
	FProxyPool& ProxyPool;
};

// Build request block batch using Horde/Unsync JSON request format
std::string FormatBlockRequestJson(const FBlockRequestMap& RequestMap, const TArrayView<FNeedBlock> NeedBlocks);

}  // namespace unsync
