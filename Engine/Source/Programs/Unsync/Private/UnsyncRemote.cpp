// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncRemote.h"
#include "UnsyncCore.h"
#include "UnsyncHorde.h"

namespace unsync {

const char*
ToString(EProtocolFlavor Protocol)
{
	switch (Protocol)
	{
		default:
		case EProtocolFlavor::Unknown:
			return "Unknown";
		case EProtocolFlavor::Unsync:
			return "Unsync";
		case EProtocolFlavor::Horde:
			return "Horde";
		case EProtocolFlavor::Jupiter:
			return "Jupiter";
	}
}

EProtocolFlavor ProtocolFlavorFromString(std::string_view Str)
{
	if (UncasedStringEquals(Str, "unsync"))
	{
		return EProtocolFlavor::Unsync;
	}
	else if (UncasedStringEquals(Str, "horde"))
	{
		return EProtocolFlavor::Horde;
	}
	else if (UncasedStringEquals(Str, "jupiter"))
	{
		return EProtocolFlavor::Jupiter;
	}
	else
	{
		return EProtocolFlavor::Unknown;
	}
}

static bool
IsValidUrlCharacter(char C)
{
	const char AllowedSpecials[] = ":/?#[]@!$&'()*+,;=-_.~";
	if ((C >= 'a' && C <= 'z') || (C >= 'A' && C <= 'Z') || (C >= '0' && C <= '9'))
	{
		return true;
	}
	for (char AllowedC : AllowedSpecials)
	{
		if (C == AllowedC)
		{
			return true;
		}
	}
	return false;
}

static bool
IsValidUrl(std::string_view Url)
{
	for (char C : Url)
	{
		if (!IsValidUrlCharacter(C))
		{
			return false;
		}
	}
	return true;
}

TResult<FRemoteDesc>
FRemoteDesc::FromUrl(std::string_view Url, EProtocolFlavor ProtocolFlavorHint, uint32 RecvTimeoutSeconds)
{
	if (!IsValidUrl(Url))
	{
		return AppError("Invalid URL");
	}

	FRemoteDesc Result;

	const size_t NamespacePos = Url.find_last_of('#');
	if (NamespacePos != std::string::npos)
	{
		Result.StorageNamespace = Url.substr(NamespacePos + 1);
		Url						= Url.substr(0, NamespacePos);
	}

	const size_t SchemePos	 = Url.find("://");
	const bool	 bHaveScheme = SchemePos != std::string::npos;

	std::string_view Scheme = bHaveScheme ? Url.substr(0, SchemePos) : std::string_view();

	ETlsRequirement	   TlsRequirement = ETlsRequirement::None;
	std::string_view   HostAddress	  = bHaveScheme ? Url.substr(SchemePos + 3) : Url;
	ETransportProtocol Transport	  = ETransportProtocol::Unsync;

	if (Scheme.ends_with("https"))
	{
		TlsRequirement = ETlsRequirement::Required;
		Transport	   = ETransportProtocol::Http;
	}
	else if (Scheme.ends_with("http"))
	{
		TlsRequirement = ETlsRequirement::None;
		Transport	   = ETransportProtocol::Http;
	}
	else if (Scheme.ends_with("tls"))
	{
		TlsRequirement = ETlsRequirement::Required;
		if (Scheme.starts_with("unsync"))
		{
			Transport = ETransportProtocol::Unsync;
		}
		else
		{
			return AppError("Invalid transport scheme");
		}
	}

	const size_t RequestPos = HostAddress.find_first_of('/');
	if (RequestPos != std::string::npos)
	{
		Result.RequestPath = HostAddress.substr(RequestPos + 1);
		HostAddress		   = HostAddress.substr(0, RequestPos);
	}

	const bool bRequestLooksLikeHordeArtifact = RequestPathLooksLikeHordeArtifact(Result.RequestPath);

	if (ProtocolFlavorHint == EProtocolFlavor::Unknown)
	{
		// Try to guess protocol flavor

		switch (Transport)
		{
			default:
			case ETransportProtocol::Unsync:
				Result.Protocol = EProtocolFlavor::Unsync;
				break;
			case ETransportProtocol::Http:
				if (NamespacePos != std::string::npos || Scheme.starts_with("jupiter"))
				{
					Result.Protocol = EProtocolFlavor::Jupiter;
				}
				else if (bRequestLooksLikeHordeArtifact || Scheme.starts_with("horde"))
				{
					Result.Protocol = EProtocolFlavor::Horde;
				}
				else
				{
					Result.Protocol = EProtocolFlavor::Unsync;
				}
				break;
		}
	}
	else
	{
		Result.Protocol = ProtocolFlavorHint;
	}

	uint16		 HostPort = 0;
	const size_t PortPos  = HostAddress.find_first_of(':');

	if (PortPos == std::string::npos)
	{
		switch (Transport)
		{
			default:
			case ETransportProtocol::Unsync:
				HostPort = UNSYNC_DEFAULT_PORT;
				break;
			case ETransportProtocol::Http:
				HostPort = (TlsRequirement == ETlsRequirement::Required) ? 443 : 80;
				break;
		}
	}
	else
	{
		int ParsedHostPort = std::atoi(&HostAddress[PortPos + 1]);
		if (ParsedHostPort > 0 && ParsedHostPort < 65536)
		{
			HostPort = uint16(ParsedHostPort);
		}
		HostAddress = HostAddress.substr(0, PortPos);
	}

	if (HostPort == 0)
	{
		return AppError("Invalid host port");  // TODO: extract the port substring
	}

	if (HostPort == 443)
	{
		TlsRequirement = ETlsRequirement::Required;
	}
	else if (Result.Protocol == EProtocolFlavor::Unsync
		&& TlsRequirement < ETlsRequirement::Required
		&& Transport == ETransportProtocol::Unsync)
	{
		TlsRequirement = ETlsRequirement::Preferred;
	}

	Result.Host.Address	  = HostAddress;
	Result.Host.Port	  = HostPort;
	Result.TlsRequirement = TlsRequirement;
	Result.RecvTimeoutSeconds = RecvTimeoutSeconds;

	return ResultOk(Result);
}

FTlsClientSettings
FRemoteDesc::GetTlsClientSettings() const
{
	FTlsClientSettings Result = {};
	if (TlsRequirement != ETlsRequirement::None)
	{
		Result.Subject			  = GetTlsSubject();
		Result.bVerifyCertificate = bTlsVerifyCertificate;
		Result.bVerifySubject	  = bTlsVerifySubject;
		if (TlsCacert)
		{
			Result.CACert = TlsCacert->View();
		}
	}
	else
	{
		Result.bVerifyCertificate = false;
		Result.bVerifySubject	  = false;
	}
	return Result;
}

void
TestParseRemote()
{
	{
		UNSYNC_ASSERT(FRemoteDesc::FromUrl("bad url").IsError());
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("example.com");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Preferred);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == UNSYNC_DEFAULT_PORT);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Unsync);
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("http://example.com#foo");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::None);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 80);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Jupiter);
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("https://example.com#foo");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Required);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 443);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Jupiter);
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("http://example.com:1234#foo");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::None);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 1234);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Jupiter);
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("https://example.com:1234#foo");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Required);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 1234);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Jupiter);
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("jupiter+http://example.com:1234");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::None);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 1234);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Jupiter);
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("jupiter+https://example.com");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Required);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 443);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Jupiter);
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("jupiter+https://example.com#test.namespace");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Required);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 443);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Jupiter);
		UNSYNC_ASSERT(ParseResult->StorageNamespace == "test.namespace");
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("jupiter+https://example.com:1234#test.namespace");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Required);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 1234);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Jupiter);
		UNSYNC_ASSERT(ParseResult->StorageNamespace == "test.namespace");
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("example.com:1234#test.namespace");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Preferred);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 1234);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Unsync);
		UNSYNC_ASSERT(ParseResult->StorageNamespace == "test.namespace");
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("unsync://example.com:1234");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Preferred);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 1234);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Unsync);
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("unsync+tls://example.com:1234");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Required);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 1234);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Unsync);
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("unsync+tls://example.com:invalid_port");
		UNSYNC_ASSERT(ParseResult.IsError());
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("unsync+tls://example.com:1234/request/path#namespace");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Required);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 1234);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Unsync);
		UNSYNC_ASSERT(ParseResult->StorageNamespace == "namespace");
		UNSYNC_ASSERT(ParseResult->RequestPath == "request/path");
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("http://example.com/request/path#namespace");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::None);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 80);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Jupiter);
		UNSYNC_ASSERT(ParseResult->StorageNamespace == "namespace");
		UNSYNC_ASSERT(ParseResult->RequestPath == "request/path");
	}

	{
		TResult<FRemoteDesc> ParseResult = FRemoteDesc::FromUrl("example.com:443");
		UNSYNC_ASSERT(ParseResult.IsOk());
		UNSYNC_ASSERT(ParseResult->TlsRequirement == ETlsRequirement::Required);
		UNSYNC_ASSERT(ParseResult->Host.Address == "example.com");
		UNSYNC_ASSERT(ParseResult->Host.Port == 443);
		UNSYNC_ASSERT(ParseResult->Protocol == EProtocolFlavor::Unsync);
	}
}

}  // namespace unsync
