// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncError.h"
#include "UnsyncSocket.h"

#include <optional>
#include <memory>
#include <string>

namespace unsync {

class FBuffer;

static constexpr uint16 UNSYNC_DEFAULT_PORT = 53841;

enum class EProtocolFlavor : uint8 {
	Unknown,
	Unsync,
	Jupiter,
	Horde,
};

enum class ETransportProtocol : uint8 {
	Http,
	Unsync,
};

EProtocolFlavor ProtocolFlavorFromString(std::string_view Str);
const char*		ToString(EProtocolFlavor Protocol);

struct FHostAddressAndPort
{
	std::string Address;
	uint16		Port = 0;

	bool IsValid() const { return Port != 0 && !Address.empty(); }
};

struct FRemoteDesc
{
	EProtocolFlavor Protocol = EProtocolFlavor::Unknown;

	FHostAddressAndPort Host;

	std::string RequestPath;
	std::string StorageNamespace;
	std::string StorageBucket = "unsync";  // TODO: override via command line
	std::string HttpHeaders;

	bool					 bTlsVerifyCertificate = true;	// Disabling this allows self-signed certificates
	bool					 bTlsVerifySubject	   = true;	// Disabling this is insecure, but may be useful during development
	ETlsRequirement			 TlsRequirement		   = ETlsRequirement::None;
	std::string				 TlsSubjectOverride;			// Use host address if empty (default)
	std::shared_ptr<FBuffer> TlsCacert;	 // Custom CA to use for server certificate validation (system root CA is used by default)

	const std::string& GetTlsSubject() const { return TlsSubjectOverride.length() ? TlsSubjectOverride : Host.Address; }

	bool bAuthenticationRequired = false;
	std::optional<FHostAddressAndPort> PrimaryHost;	 // Optional address of the server used for login requests and other queries. If
													 // empty, then HostAddress is used.

	bool bPreferCompression = true;	 // Whether to prefer compressed encoding during bulk data transfer

	const FHostAddressAndPort& GetPrimaryHostAddress() const { return PrimaryHost ? *PrimaryHost : Host; }

	uint32 RecvTimeoutSeconds = 0;

	uint32 MaxConnections = 8;	// Limit on concurrent connections to this server

	bool IsValid() const { return Protocol != EProtocolFlavor::Unknown && Host.IsValid(); }

	static TResult<FRemoteDesc> FromUrl(std::string_view Url, EProtocolFlavor ProtocolFlavorHint = EProtocolFlavor::Unknown, uint32 RecvTimeoutSeconds = 0);

	FTlsClientSettings GetTlsClientSettings() const;
};

}  // namespace unsync
