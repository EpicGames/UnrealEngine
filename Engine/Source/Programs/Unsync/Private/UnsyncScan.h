// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <stdint.h>
#include <string.h>
#include <functional>

namespace unsync {

template<typename H, typename F>
void
HashScan(H& Hash, const uint8* Data, uint64 DataSize, uint64 WindowSize, F& Callback)
{
	const uint8* DataEnd = Data + DataSize;

	const uint8* WindowBegin = Data;
	const uint8* WindowEnd	 = WindowBegin;

	for (;;)
	{
		uint64 RemainingDataSize = DataEnd - WindowBegin;
		if (RemainingDataSize == 0)
		{
			break;
		}

		uint64 ThisWindowSize = std::min(RemainingDataSize, WindowSize);
		while (Hash.Count < ThisWindowSize)
		{
			Hash.Add(*WindowEnd);
			++WindowEnd;
		}

		bool bAccepted = Callback(WindowBegin, WindowEnd, Hash.Get());

		if (bAccepted)
		{
			WindowBegin = WindowEnd;
			WindowEnd	= WindowBegin;
			Hash.Reset();
			continue;
		}

		Hash.Sub(*(WindowBegin++));
	}
}

template<typename H, typename F>
void
HashScan(const uint8* Data, uint64 DataSize, uint64 WindowSize, F& Callback)
{
	H Hash;
	HashScan(Hash, Data, DataSize, WindowSize, Callback);
}

}  // namespace unsync
