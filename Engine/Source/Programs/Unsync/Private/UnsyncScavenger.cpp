// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncScavenger.h"
#include "UnsyncFile.h"
#include "UnsyncLog.h"
#include "UnsyncProgress.h"
#include "UnsyncSerialization.h"
#include "UnsyncThread.h"
#include "UnsyncScheduler.h"

namespace unsync {

FScavengeDatabase*
FScavengeDatabase::BuildFromFileSyncTasks(const FSyncDirectoryOptions& SyncOptions, TArrayView<FFileSyncTask> AllFileTasks)
{
	FScavengeDatabase* Result = new FScavengeDatabase;

	THashSet<FHash128> NeededBlocks;
	for (const FFileSyncTask& FileTask : AllFileTasks)
	{
		for (const FNeedBlock& SourceNeedBlock : FileTask.NeedList.Source)
		{
			NeededBlocks.insert(SourceNeedBlock.Hash.ToHash128());
		}
	}

	if (NeededBlocks.empty())
	{
		return Result;
	}

	const FPath ExtendedTargetPath = RemoveExtendedPathPrefix(SyncOptions.Target);

	UNSYNC_VERBOSE(L"Scanning '%ls' for usable manifests", SyncOptions.ScavengeRoot.wstring().c_str());

	for (std::filesystem::recursive_directory_iterator DirIt = RecursiveDirectoryScan(SyncOptions.ScavengeRoot);
		 DirIt != std::filesystem::recursive_directory_iterator();
		 ++DirIt)
	{
		const std::filesystem::directory_entry& Dir = *DirIt;

		if (!Dir.is_directory())
		{
			continue;
		}

		const int32 DirDepth = DirIt.depth();
		if (DirDepth + 1 >= int32(SyncOptions.ScavengeDepth))
		{
			DirIt.disable_recursion_pending();
		}

		const FPath& DirPath = Dir.path();
		if (DirPath == ExtendedTargetPath)
		{
			// Exclude the current sync target path from scavanging database
			DirIt.disable_recursion_pending();
			continue;
		}

		FPath DirStem = DirPath.stem();
		if (!DirStem.compare(L".unsync"))
		{
			DirIt.disable_recursion_pending();

			FPath ManifestPath = DirPath / "manifest.bin";
			if (PathExists(ManifestPath))
			{
				FScavengedManifest Entry;
				Entry.ManifestPath = RemoveExtendedPathPrefix(ManifestPath);
				Entry.Root		   = RemoveExtendedPathPrefix(DirPath.parent_path());
				Result->Manifests.push_back(std::move(Entry));
			}
		}
	}

	UNSYNC_VERBOSE(L"Loading scavenged manifests: %llu", llu(Result->Manifests.size()));

	ParallelForEach(Result->Manifests, [](FScavengedManifest& Entry) {
		FLogVerbosityScope VerbosityScope(false);  // turn off logging from threads
		Entry.bValid = LoadDirectoryManifest(Entry.Manifest, Entry.Root, Entry.ManifestPath);
		if (Entry.bValid)
		{
			Entry.FileList.reserve(Entry.Manifest.Files.size());
			for (const auto& It : Entry.Manifest.Files)
			{
				const FPath& RelativeFileName = It.first;
				Entry.FileList.push_back(RelativeFileName);
			}
		}
	});

	UNSYNC_VERBOSE(L"Building block database");

	THashMap<FScavengeBlockSource, uint64, FScavengeBlockSource::FHash> BlockSourceUseCounts;

	uint32 ManifestIndex = 0;
	for (const FScavengedManifest& Entry : Result->Manifests)
	{
		uint32 FileIndex = 0;
		for (const auto& FileManifestIt : Entry.Manifest.Files)
		{
			const FFileManifest& FileManifest = FileManifestIt.second;

			THashSet<FHash128> UniqueBlocksPerFile;

			for (const FGenericBlock& BlockInfo : FileManifest.Blocks)
			{
				FHash128 BlockHash = BlockInfo.HashStrong.ToHash128();
				Result->UniqueBlockHashes.insert(BlockHash);
				if (NeededBlocks.find(BlockHash) != NeededBlocks.end())
				{
					FScavengeBlockSource BlockSource;
					BlockSource.Data.ManifestIndex = ManifestIndex;
					BlockSource.Data.FileIndex	   = FileIndex;

					if (UniqueBlocksPerFile.insert(BlockHash).second)
					{
						Result->BlockMap.insert(std::make_pair(BlockHash, BlockSource));
						Result->UniqueUsableBlockHashes.insert(BlockHash);
						BlockSourceUseCounts[BlockSource] += 1;
					}
				}
			}
			++FileIndex;
		}
		++ManifestIndex;
	}

	struct FBlockSourceAndUseCount
	{
		FScavengeBlockSource Source;
		uint64				 Count;
	};

	std::vector<FBlockSourceAndUseCount> SortedBlockSources;
	for (const auto& It : BlockSourceUseCounts)
	{
		FBlockSourceAndUseCount Entry;
		Entry.Source = It.first;
		Entry.Count	 = It.second;
		SortedBlockSources.push_back(Entry);
	}

	std::sort(SortedBlockSources.begin(), SortedBlockSources.end(), [](const FBlockSourceAndUseCount& A, const FBlockSourceAndUseCount& B) {
		return A.Count > B.Count;
	});

	UNSYNC_VERBOSE(L"Found potentially useful files: %llu", llu(BlockSourceUseCounts.size()));

#if 0  // Extra verbose scavenging status report
	if (SortedBlockSources.size())
	{
		UNSYNC_LOG_INDENT;
		UNSYNC_VERBOSE(L"Top useful files:");
		for (size_t I = 0; I < SortedBlockSources.size() && I < 100; ++I)
		{
			const FBlockSourceAndUseCount& Item				= SortedBlockSources[I];
			const FScavengedManifest&	   Manifest			= Result->Manifests[Item.Source.Data.ManifestIndex];
			const FPath&				   RelativeFilePath = Manifest.FileList[Item.Source.Data.FileIndex];

			FPath FilePath = Manifest.Root / RelativeFilePath;

			UNSYNC_VERBOSE(L"- %ls: %llu", FilePath.wstring().c_str(), Item.Count);
		}
	}
#endif

	UNSYNC_VERBOSE(L"Found potential usable blocks: %llu out of %llu needed (%.2f %%)",
				   llu(Result->UniqueUsableBlockHashes.size()),
				   llu(NeededBlocks.size()),
				   100.0 * double(Result->UniqueUsableBlockHashes.size()) / double(NeededBlocks.size()));

	return Result;
}

const FPath&
FScavengeDatabase::GetPartialSourceFilePath(FScavengeBlockSource SourceId) const
{
	const FScavengedManifest& Manifest	   = Manifests[SourceId.Data.ManifestIndex];
	const FPath&			  RelativePath = Manifest.FileList[SourceId.Data.FileIndex];
	return RelativePath;
}

FPath
FScavengeDatabase::GetFullSourceFilePath(FScavengeBlockSource SourceId) const
{
	const FScavengedManifest& Manifest	   = Manifests[SourceId.Data.ManifestIndex];
	const FPath&			  RelativePath = Manifest.FileList[SourceId.Data.FileIndex];
	return Manifest.Root / RelativePath;
}

const FFileManifest&
FScavengeDatabase::GetFileManifest(FScavengeBlockSource SourceId) const
{
	const FScavengedManifest& ScavengedManifest = GetScavengedManifest(SourceId);
	auto					  FileIt = ScavengedManifest.Manifest.Files.find(ScavengedManifest.FileList[SourceId.Data.FileIndex].wstring());
	return FileIt->second;
}

bool
FScavengeDatabase::IsSourceValid(FScavengeBlockSource SourceId) const
{
	if (SourceId.Data.ManifestIndex >= Manifests.size())
	{
		return false;
	}

	const FScavengedManifest& ScavangedManifest = Manifests[SourceId.Data.ManifestIndex];
	if (SourceId.Data.FileIndex >= ScavangedManifest.FileList.size())
	{
		return false;
	}

	const FPath& RelativeFilePath = ScavangedManifest.FileList[SourceId.Data.FileIndex];
	FPath		 FullFilePath	  = ScavangedManifest.Root / RelativeFilePath;

	auto FileIt = ScavangedManifest.Manifest.Files.find(RelativeFilePath.wstring());
	if (FileIt == ScavangedManifest.Manifest.Files.end())
	{
		return false;
	}

	const FFileManifest& FileManifest = FileIt->second;

	FFileAttributes Attrib = GetFileAttrib(FullFilePath);

	return Attrib.bValid && Attrib.Mtime == FileManifest.Mtime && Attrib.Size == FileManifest.Size;
}

struct FCopyCommandWithBlockRange : FCopyCommand
{
	TArrayView<FNeedBlock> BlockRange;
};

// Similar to OptimizeNeedList, but assumes maintains commands in the same order as input blocks.
// Input blocks are preferred to be sorted by source offset, but it is not a hard requirement.
static std::vector<FCopyCommandWithBlockRange>
OptimizeNeedListWithBlockRange(const std::vector<FNeedBlock>& Input, uint64 MaxMergedBlockSize)
{
	std::vector<FCopyCommandWithBlockRange> Result;
	Result.reserve(Input.size());
	for (const FNeedBlock& Block : Input)
	{
		FCopyCommandWithBlockRange Cmd;
		Cmd.SourceOffset = Block.SourceOffset;
		Cmd.TargetOffset = Block.TargetOffset;
		Cmd.Size		 = Block.Size;
		Cmd.BlockRange	 = MakeView(&Block, 1);
		Result.push_back(Cmd);
	}

	for (uint64 I = 1; I < Result.size(); ++I)
	{
		FCopyCommandWithBlockRange& PrevBlock = Result[I - 1];
		FCopyCommandWithBlockRange& ThisBlock = Result[I];
		if (PrevBlock.SourceOffset + PrevBlock.Size == ThisBlock.SourceOffset &&
			PrevBlock.TargetOffset + PrevBlock.Size == ThisBlock.TargetOffset && PrevBlock.Size + ThisBlock.Size <= MaxMergedBlockSize)
		{
			ThisBlock.SourceOffset = PrevBlock.SourceOffset;
			ThisBlock.TargetOffset = PrevBlock.TargetOffset;
			ThisBlock.Size += PrevBlock.Size;

			UNSYNC_ASSERT(PrevBlock.BlockRange.EndPtr == ThisBlock.BlockRange.BeginPtr);
			UNSYNC_ASSERT(ThisBlock.Size <= MaxMergedBlockSize);

			ThisBlock.BlockRange.BeginPtr = PrevBlock.BlockRange.BeginPtr;

			// Invalidate previous block
			PrevBlock.BlockRange.EndPtr = PrevBlock.BlockRange.BeginPtr;
			PrevBlock.Size				= 0;
		}
	}

	for (uint64 I = 0; I < Result.size(); ++I)
	{
		UNSYNC_ASSERT(Result[I].Size <= MaxMergedBlockSize);
	}

	auto It = std::remove_if(Result.begin(), Result.end(), [](const FCopyCommand& Block) { return Block.Size == 0; });

	Result.erase(It, Result.end());

	{
		uint64 BlockCount = 0;
		for (uint64 I = 0; I < Result.size(); ++I)
		{
			BlockCount += Result[I].BlockRange.Size();
		}
		UNSYNC_ASSERT(BlockCount == Input.size());
	}

	return Result;
}

FScavengedBuildTargetResult
BuildTargetFromScavengedData(FIOWriter&						Output,
							 const std::vector<FNeedBlock>& NeedList,
							 const FScavengeDatabase&		ScavengeDatabase,
							 EStrongHashAlgorithmID			StrongHasher,
							 THashSet<FHash128>&			OutScavengedBlocks)
{
	FScavengedBuildTargetResult BuildResult;

	std::vector<FNeedBlock> ScavengeNeedList;

	THashMap<FScavengeBlockSource, uint64, FScavengeBlockSource::FHash> PossibleSources;

	const FScavengeBlockMap& ScavengeBlockMap = ScavengeDatabase.GetBlockMap();
	uint64					 TotalCopySize	  = 0;
	for (const FNeedBlock& SourceNeedBlock : NeedList)
	{
		FHash128 NeedBlockHash = SourceNeedBlock.Hash.ToHash128();
		TotalCopySize += SourceNeedBlock.Size;
		const auto Sources = ScavengeBlockMap.equal_range(NeedBlockHash);
		if (Sources.first != Sources.second)
		{
			ScavengeNeedList.push_back(SourceNeedBlock);
			for (auto SourceIt = Sources.first; SourceIt != Sources.second; ++SourceIt)
			{
				const FScavengeBlockSource& BlockSource = SourceIt->second;
				PossibleSources[BlockSource] += 1;
			}
		}
	}

	struct FPossibleSource : FScavengeBlockSource
	{
		uint64 NumHits = 0;
		FPath  FileName;
		FPath  FullSourceFilePath;
	};

	std::vector<FPossibleSource> SortedPossibleSources;

	for (const auto& It : PossibleSources)
	{
		if (ScavengeDatabase.IsSourceValid(It.first))
		{
			FPossibleSource Entry;
			Entry.Bits				 = It.first.Bits;
			Entry.NumHits			 = It.second;
			Entry.FileName			 = ScavengeDatabase.GetPartialSourceFilePath(It.first);
			Entry.FullSourceFilePath = ScavengeDatabase.GetFullSourceFilePath(It.first);

			SortedPossibleSources.push_back(Entry);
		}
	};

	std::sort(SortedPossibleSources.begin(), SortedPossibleSources.end(), [](const FPossibleSource& A, const FPossibleSource& B) {
		return A.NumHits > B.NumHits;
	});

	FLogProgressScope ProgressLogger(TotalCopySize, ELogProgressUnits::MB);

	const uint64 ScavengeSizeThreshold = uint64(double(TotalCopySize) * 0.01);

	std::vector<FNeedBlock> LocalNeedList;
	for (const FPossibleSource& PossibleSource : SortedPossibleSources)
	{
		LocalNeedList.clear();

		if (ScavengeNeedList.empty())
		{
			break;
		}

		const FFileManifest& ScavengeFileManifest = ScavengeDatabase.GetFileManifest(PossibleSource);

		FNativeFile LocalSourceFile = FNativeFile(PossibleSource.FullSourceFilePath, EFileMode::ReadOnlyUnbuffered);
		if (LocalSourceFile.IsValid())
		{
			THashMap<FGenericHash, uint64> BlockOffsetMap;
			for (const FGenericBlock& Block : ScavengeFileManifest.Blocks)
			{
				BlockOffsetMap[Block.HashStrong] = Block.Offset;
			}

			for (const FNeedBlock& NeedBlock : ScavengeNeedList)
			{
				auto OffsetIt = BlockOffsetMap.find(NeedBlock.Hash);
				if (OffsetIt != BlockOffsetMap.end())
				{
					FNeedBlock LocalNeedBlock	= NeedBlock;
					LocalNeedBlock.SourceOffset = OffsetIt->second;
					LocalNeedList.push_back(LocalNeedBlock);
				}
			}

			const uint64 LocalNeedListSize = ComputeSize(LocalNeedList);
			if (LocalNeedListSize < ScavengeSizeThreshold)
			{
				continue;
			}

			UNSYNC_VERBOSE(L"Scavenging data from '%ls'", PossibleSource.FullSourceFilePath.wstring().c_str());

			std::sort(LocalNeedList.begin(), LocalNeedList.end(), FNeedBlock::FCompareBySourceOffset());

			std::vector<FCopyCommandWithBlockRange> CopyCommands = OptimizeNeedListWithBlockRange(LocalNeedList, 1_MB);

			bool bFoundInvalidBlock = false;

			auto ReadCallback = [&Output, &OutScavengedBlocks, &LocalSourceFile, &ProgressLogger, &BuildResult, &bFoundInvalidBlock, StrongHasher](
									FIOBuffer Buffer,
									uint64	  SourceOffset,
									uint64	  ReadSize,
									uint64	  UserData) {
				const FCopyCommandWithBlockRange& CopyCommand = *reinterpret_cast<const FCopyCommandWithBlockRange*>(UserData);

				UNSYNC_ASSERT(CopyCommand.BlockRange.Size() != 0);

				const FNeedBlock& FirstBlock = CopyCommand.BlockRange.BeginPtr[0];

				uint64 BlockOffset	= 0;
				bool   bBlockHashOk = true;
				for (const FNeedBlock& Block : CopyCommand.BlockRange)
				{
					const uint8* BlockData	  = Buffer.GetData() + BlockOffset;
					FGenericHash ActualHash	  = ComputeHash(BlockData, Block.Size, StrongHasher);
					FGenericHash ExpectedHash = Block.Hash;

					UNSYNC_ASSERT(Block.TargetOffset == FirstBlock.TargetOffset + BlockOffset);

					if (ActualHash != ExpectedHash)
					{
						bBlockHashOk = false;
						break;
					}

					BlockOffset += Block.Size;
				}

				if (CopyCommand.Size != ReadSize)
				{
					bBlockHashOk = false;
				}

				if (bBlockHashOk)
				{
					Output.WriteFrom(LocalSourceFile, CopyCommand, Buffer.GetBufferView(), &BuildResult.InSituBytes);

					for (const FNeedBlock& Block : CopyCommand.BlockRange)
					{
						OutScavengedBlocks.insert(Block.Hash.ToHash128());
					}

					BuildResult.ScavengedBytes += ReadSize;
					AddGlobalProgress(ReadSize, EBlockListType::Source);
					ProgressLogger.Add(ReadSize);
				}
				else
				{
					bFoundInvalidBlock = true;
				}
				
			};

			std::unique_ptr<FAsyncReader> AsyncReader = LocalSourceFile.CreateAsyncReader();

			for (const FCopyCommandWithBlockRange& Command : CopyCommands)
			{
				AsyncReader->EnqueueRead(Command.SourceOffset, Command.Size, (uint64)(&Command), ReadCallback);
				if (bFoundInvalidBlock)
				{
					// If an invalid block is encountered in a source file, then assume that is corrupt and should not be used from this point.
					// TODO: could also mark the source file as "bad" globally, to avoid using it when patching other files.
					break;
				}
			}

			AsyncReader->Flush();
		}

		auto FilterResult =
			std::remove_if(ScavengeNeedList.begin(), ScavengeNeedList.end(), [&OutScavengedBlocks](const FNeedBlock& Block) {
				return OutScavengedBlocks.find(Block.Hash.ToHash128()) != OutScavengedBlocks.end();
			});

		ScavengeNeedList.erase(FilterResult, ScavengeNeedList.end());
	}

	return BuildResult;
}

}  // namespace unsync
