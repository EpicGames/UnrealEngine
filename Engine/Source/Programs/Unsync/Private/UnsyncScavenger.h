// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncCore.h"
#include "UnsyncHashTable.h"
#include "UnsyncUtil.h"

#include <unordered_map>

namespace unsync {

struct FIOWriter;
struct FFileSyncTask;
struct FNeedBlock;

struct FScavengedManifest
{
	bool			   bValid = false;
	FPath			   Root;
	FPath			   ManifestPath;
	FDirectoryManifest Manifest;
	std::vector<FPath> FileList;
};

struct FScavengeBlockSource
{
	union
	{
		struct
		{
			uint32 ManifestIndex;
			uint32 FileIndex;
		} Data;
		uint64 Bits = ~0ull;
	};

	bool operator==(const FScavengeBlockSource& Other) const { return Bits == Other.Bits; }

	struct FHash
	{
		uint64 operator()(const FScavengeBlockSource& X) const { return X.Bits; }
	};
};

using FScavengeBlockMap = std::unordered_multimap<FHash128, FScavengeBlockSource>;

class FScavengeDatabase
{
public:

	static FScavengeDatabase* BuildFromFileSyncTasks(const FSyncDirectoryOptions& SyncOptions, TArrayView<FFileSyncTask> AllFileTasks);

	const FScavengeBlockMap&  GetBlockMap() const { return BlockMap; }
	const FPath&			  GetPartialSourceFilePath(FScavengeBlockSource SourceId) const;
	FPath					  GetFullSourceFilePath(FScavengeBlockSource SourceId) const;
	const FScavengedManifest& GetScavengedManifest(FScavengeBlockSource SourceId) const { return Manifests[SourceId.Data.ManifestIndex]; }
	const FFileManifest&	  GetFileManifest(FScavengeBlockSource SourceId) const;
	bool					  IsSourceValid(FScavengeBlockSource SourceId) const;

private:
	std::vector<FScavengedManifest> Manifests;

	THashSet<FHash128> UniqueBlockHashes;
	THashSet<FHash128> UniqueUsableBlockHashes;

	// Which local files may contain blocks with this hash
	FScavengeBlockMap BlockMap;

	// Holds source locations that are detected to be inaccessible or corrupted.
	// This can be used to skip trying a known-bad source over and over.
	// THashSet<FScavengeBlockSource, FScavengeBlockSource::FHash> BannedBlockSources;
};

struct FScavengedBuildTargetResult
{
	uint64 ScavengedBytes = 0;
	// Bytes reused on disk via block cloning. Subset of ScavengedBytes.
	uint64 InSituBytes	  = 0;
};

FScavengedBuildTargetResult BuildTargetFromScavengedData(FIOWriter&						Output,
														 const std::vector<FNeedBlock>& NeedList,
														 const FScavengeDatabase&		ScavengeDatabase,
														 EStrongHashAlgorithmID			StrongHasher,
														 THashSet<FHash128>&			OutScavengedBlocks);

}  // namespace unsync
