// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncScheduler.h"

namespace unsync {

FScheduler* GScheduler = nullptr;

FScheduler::FScheduler(uint32 InNumWorkerThreads)
: NumWorkerThreads(InNumWorkerThreads)
, NetworkSemaphore(*this, MAX_NETWORK_TASKS)
{
	ThreadPool.StartWorkers(NumWorkerThreads);
}

FScheduler::~FScheduler()
{
}

FTaskGroup
FScheduler::CreateTaskGroup(FSchedulerSemaphore* ConcurrencyLimiter)
{
	return FTaskGroup(ThreadPool, ConcurrencyLimiter);
}

FSchedulerSemaphore::FSchedulerSemaphore(FScheduler& InScheduler, uint32 MaxCount)
: Scheduler(InScheduler)
, Native(std::min(MaxCount, 1 + InScheduler.NumWorkerThreads)) // 1 for main thread + N workers
{
}

void
FSchedulerSemaphore::Acquire(bool bAllowTaskExecution)
{
	if (bAllowTaskExecution)
	{
		while (!Native.try_acquire())
		{
			Scheduler.TryExecuteTask();
		}
	}
	else
	{
		Native.acquire();
	}
}

void
FSchedulerSemaphore::Release()
{
	Native.release();
}

}  // namespace unsync
