// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncThread.h"

namespace unsync {

class FScheduler;
struct FTaskGroup;

extern FScheduler* GScheduler;

struct FSchedulerSemaphore
{
	UNSYNC_DISALLOW_COPY_ASSIGN(FSchedulerSemaphore)

	FSchedulerSemaphore(FScheduler& InScheduler, uint32 MaxCount);

	bool TryAcquire() { return Native.try_acquire(); }

	void Acquire(bool bAllowTaskExecution = true);
	void Release();

	FScheduler& Scheduler;

	std::counting_semaphore<UNSYNC_MAX_TOTAL_THREADS> Native;
};

class FScheduler
{
public:
	UNSYNC_DISALLOW_COPY_ASSIGN(FScheduler)

	using FTaskFunction = FThreadPool::FTaskFunction;

	static constexpr uint32 MAX_NETWORK_TASKS = 8;

	FScheduler(uint32 InNumWorkerThreads);
	~FScheduler();

	const uint32 NumWorkerThreads;

	FSchedulerSemaphore NetworkSemaphore;

	FTaskGroup CreateTaskGroup(FSchedulerSemaphore* ConcurrencyLimiter = nullptr);

	void TryExecuteTask() { ThreadPool.TryExecuteTask(); }

	bool ExecuteTasksUntilIdle()
	{
		uint64 NumExecuted = 0;
		while (ThreadPool.TryExecuteTask())
		{
			NumExecuted++;
		}
		return NumExecuted != 0;
	}

	template<typename TaskFunction>
	void PushTask(TaskFunction&& Fun, bool bAllowImmediateExecution = true)
	{
		ThreadPool.PushTask(std::forward<TaskFunction>(Fun), bAllowImmediateExecution);
	}

private:
	FThreadPool ThreadPool;
};

struct FTaskGroup
{
	UNSYNC_DISALLOW_COPY_ASSIGN(FTaskGroup)

	template<typename F>
	void run(F InFunction)
	{
		++NumStartedTasks;

		const bool bAcquired = Semaphore && Semaphore->TryAcquire();

		if (!Semaphore || bAcquired)
		{
			ThreadPool.PushTask(
				[Semaphore = this->Semaphore,
				 bAcquired,
				 &NumStartedTasks  = this->NumStartedTasks,
				 &NumFinishedTasks = this->NumFinishedTasks,
				 Function		   = std::forward<F>(InFunction)]() -> void
				{
					Function();
					if (Semaphore && bAcquired)
					{
						Semaphore->Release();
					}
					++NumFinishedTasks;
				});
		}
		else
		{
			InFunction();
			++NumFinishedTasks;
		}
	}

	void wait()
	{
		while (NumFinishedTasks.load() != NumStartedTasks.load())
		{
			ThreadPool.TryExecuteTask();
		}
	}

	FThreadPool&		ThreadPool;
	std::atomic<uint64> NumStartedTasks;
	std::atomic<uint64> NumFinishedTasks;
	FSchedulerSemaphore* Semaphore = nullptr;

	~FTaskGroup() { wait(); };

private:
	friend FScheduler;
	FTaskGroup(FThreadPool& InThreadPool, FSchedulerSemaphore* InSemaphore) : ThreadPool(InThreadPool), Semaphore(InSemaphore) {}
};

template<typename IT, typename FT>
inline void
ParallelForEach(IT ItBegin, IT ItEnd, FT F)
{
	FTaskGroup TaskGroup = GScheduler->CreateTaskGroup();

	for (; ItBegin != ItEnd; ++ItBegin)
	{
		auto* It = &(*ItBegin);
		TaskGroup.run([&F, It]() { F(*It); });
	}

	TaskGroup.wait();
}

template<typename T, typename FT>
inline void
ParallelForEach(T& Container, FT F)
{
	ParallelForEach(std::begin(Container), std::end(Container), F);
}

}  // namespace unsync
