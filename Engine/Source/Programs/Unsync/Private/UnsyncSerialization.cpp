// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncSerialization.h"
#include "UnsyncFile.h"
#include "UnsyncVersion.h"

namespace unsync {

enum class EAlgorithmCompatibilityResult
{
	Ok,
	ErrorChunking,
	ErrorStrong,
	ErrorWeak,
};

static const char*
ToString(EAlgorithmCompatibilityResult V)
{
	switch (V)
	{
		case EAlgorithmCompatibilityResult::Ok:
			return "Ok";
			break;
		case EAlgorithmCompatibilityResult::ErrorChunking:
			return "Unsupported chunking algorithm";
			break;
		case EAlgorithmCompatibilityResult::ErrorStrong:
			return "Unsupported strong hash algorithm";
			break;
		case EAlgorithmCompatibilityResult::ErrorWeak:
			return "Unsupported weak hash algorithm";
			break;
		default:
			return nullptr;
	}
}

template<typename T>
void
Serialize(FVectorStreamOut& S, const T& V)
{
	S.Write((const char*)&V, sizeof(V));
}

template<typename T>
void
Serialize(FIOReaderStream& S, T& V)
{
	S.Read(&V, sizeof(V));
}

void
Serialize(FVectorStreamOut& S, const std::string& V)
{
	uint32 Len = uint32(V.length());
	Serialize(S, Len);
	S.Write(&V[0], Len);
}

void
Serialize(FIOReaderStream& S, std::string& V)
{
	uint32 Len = 0;
	Serialize(S, Len);
	V.resize(Len);
	S.Read(&V[0], Len);
}

static bool
IsCompatibleManifestVersion(uint64 V)
{
	return V == FDirectoryManifest::EVersions::V4 || V == FDirectoryManifest::EVersions::V5 || V == FDirectoryManifest::EVersions::Latest;
}

static FGenericBlock
ToGenericBlock(const FBlock128& Block, EHashType HashType)
{
	FGenericBlock Result;

	Result.HashStrong = FGenericHash::FromHash128(Block.HashStrong, HashType);
	Result.HashWeak	  = Block.HashWeak;
	Result.Offset	  = Block.Offset;
	Result.Size		  = Block.Size;

	return Result;
}

static FGenericBlock
ToGenericBlock(const FBlock128& Block, EStrongHashAlgorithmID StrongHasher)
{
	return ToGenericBlock(Block, ToHashType(StrongHasher));
}

template<typename BlockType>
static FGenericBlockArray
ToGenericBlock(const std::vector<BlockType>& Blocks, EHashType HashType)
{
	FGenericBlockArray result;

	result.reserve(Blocks.size());

	size_t HashSize = GetHashSize(HashType);

	for (const BlockType& block : Blocks)
	{
		FGenericBlock GenericBlock;
		memcpy(GenericBlock.HashStrong.Data, block.HashStrong.Data, HashSize);
		GenericBlock.HashStrong.Type = HashType;
		GenericBlock.HashWeak		 = block.HashWeak;
		GenericBlock.Offset			 = block.Offset;
		GenericBlock.Size			 = block.Size;
		result.push_back(GenericBlock);
	}

	return result;
}

bool
SaveBlocks(const std::vector<FBlock128>& Blocks, uint32 BlockSize, const FPath& Filename)
{
	const uint64 OutputSize = sizeof(FBlockFileHeader) + sizeof(FBlock128) * Blocks.size();

	FNativeFile File(Filename, EFileMode::CreateReadWrite, OutputSize);

	if (File.IsValid())
	{
		uint64 Offset = 0;

		FBlockFileHeader Header;
		Header.BlockSize = BlockSize;
		Header.NumBlocks = Blocks.size();

		Offset += File.Write(&Header, Offset, sizeof(Header));
		Offset += File.Write(Blocks.data(), Offset, sizeof(FBlock128) * Blocks.size());

		if (Offset != OutputSize)
		{
			UNSYNC_ERROR(L"Expected to write %lld bytes, but wrote %lld", OutputSize, Offset);
		}

		return Offset == OutputSize;
	}
	else
	{
		UNSYNC_ERROR(L"Failed to open file '%ls' for writing", Filename.wstring().c_str());
		return false;
	}
}

bool
LoadBlocks(FGenericBlockArray& OutBlocks, uint32& OutBlockSize, const FPath& Filename)
{
	UNSYNC_LOG_INDENT;

	FBuffer File = ReadFileToBuffer(Filename);

	if (!File.Empty())
	{
		const uint8* Ptr = File.Data();

		FBlockFileHeader Header;
		memcpy(&Header, Ptr, sizeof(Header));
		Ptr += sizeof(Header);

		if (Header.Magic != FBlockFileHeader::MAGIC)
		{
			UNSYNC_ERROR(L"Manifest file header mismatch. Expected %llX, got %llX",
						 (long long)FBlockFileHeader::MAGIC,
						 (long long)Header.Magic);
			return false;
		}

		if (Header.Version != FBlockFileHeader::VERSION)
		{
			UNSYNC_ERROR(L"Manifest file version mismatch. Expected %lld, got %lld",
						 (long long)FBlockFileHeader::VERSION,
						 (long long)Header.Version);
			return false;
		}

		OutBlockSize = uint32(Header.BlockSize);

		// #wip-widehash
		std::vector<FBlock128> TempBlocks;
		TempBlocks.resize(Header.NumBlocks);
		memcpy(&TempBlocks[0], Ptr, sizeof(FBlock128) * Header.NumBlocks);

		OutBlocks.reserve(TempBlocks.size());

		for (const FBlock128& Block : TempBlocks)
		{
			FGenericBlock GenericBlock = ToGenericBlock(Block, EStrongHashAlgorithmID::Blake3_128);	 // #wip-widehash
			OutBlocks.push_back(GenericBlock);
		}

		return true;
	}
	else
	{
		UNSYNC_ERROR(L"Failed to open file '%ls' for reading", Filename.wstring().c_str());
		return false;
	}
}

static EAlgorithmCompatibilityResult
IsCompatibleAlgorithm(const FAlgorithmOptionsV5& Options)
{
	switch (Options.ChunkingAlgorithmId)
	{
		default:
			return EAlgorithmCompatibilityResult::ErrorChunking;
		case EChunkingAlgorithmID::FixedBlocks:
		case EChunkingAlgorithmID::VariableBlocks:
			break;
	}

	switch (Options.StrongHashAlgorithmId)
	{
		default:
			return EAlgorithmCompatibilityResult::ErrorStrong;
		case EStrongHashAlgorithmID::Blake3_128:
		case EStrongHashAlgorithmID::Blake3_160:
		case EStrongHashAlgorithmID::Blake3_256:
		case EStrongHashAlgorithmID::MD5:
			break;
	}

	switch (Options.WeakHashAlgorithmId)
	{
		default:
			return EAlgorithmCompatibilityResult::ErrorWeak;
		case EWeakHashAlgorithmID::Naive:
		case EWeakHashAlgorithmID::BuzHash:
			break;
	}

	return EAlgorithmCompatibilityResult::Ok;
}

static std::unordered_map<FHash256, FGenericBlockArray>
LoadMacroBlocks(FIOReaderStream& Reader, uint64 Version)
{
	std::unordered_map<FHash256, FGenericBlockArray> Result;

	if (Version == 1)
	{
		UNSYNC_ERROR(L"Found macro block section version 1, but only versions 2+ are supported.");
		return Result;
	}

	uint64 NumFiles = 0;
	Serialize(Reader, NumFiles);

	const EHashType MacroBlockHashType = MACRO_BLOCK_HASH_TYPE;
	const uint32	MacroBlockHashSize = uint32(GetHashSize(MacroBlockHashType));

	for (uint64 FileIndex = 0; FileIndex < NumFiles; ++FileIndex)
	{
		FHash256 NameHash = {};
		Serialize<FHash256>(Reader, NameHash);

		uint64 NumBlocks = 0;
		Serialize<uint64>(Reader, NumBlocks);

		FGenericBlockArray TempMacroBlocks;
		TempMacroBlocks.reserve(NumBlocks);

		for (uint64 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
		{
			FGenericBlock Block;
			Block.HashStrong.Type = MacroBlockHashType;
			Serialize<uint64>(Reader, Block.Offset);
			Serialize<uint32>(Reader, Block.Size);
			Reader.Read((char*)Block.HashStrong.Data, MacroBlockHashSize);

			TempMacroBlocks.push_back(Block);
		}

		Result[NameHash] = std::move(TempMacroBlocks);
	}

	return Result;
}

static FBuffer
LoadFileReadOnlyMask(FIOReaderStream& Reader, FSerializedSectionHeader Header)
{
	FBuffer Result;

	if (Header.Version != FFileReadOnlyMaskSection::VERSION)
	{
		UNSYNC_ERROR(L"Found read-only file mask section version %llu, but only version %llu is supported.",
					 llu(Header.Version),
					 llu(FFileReadOnlyMaskSection::VERSION));
		return Result;
	}

	Result.Resize(Header.Size);
	uint8* ResultData = Result.Data();

	Reader.Read(ResultData, Header.Size);

	return Result;
}

static FBuffer
LoadFileExecutableBit(FIOReaderStream& Reader, FSerializedSectionHeader Header)
{
	FBuffer Result;

	if (Header.Version != FFileExecutableBitSection::VERSION)
	{
		UNSYNC_ERROR(L"Found executable file bit section version %llu, but only version %llu is supported.",
					 llu(Header.Version),
					 llu(FFileExecutableBitSection::VERSION));
		return Result;
	}

	Result.Resize(Header.Size);
	uint8* ResultData = Result.Data();

	Reader.Read(ResultData, Header.Size);

	return Result;
}

static FBuffer
SaveMacroBlocks(const FDirectoryManifest& Manifest)
{
	FBuffer			 Result;
	FVectorStreamOut Writer(Result);

	const EHashType MacroBlockHashType = MACRO_BLOCK_HASH_TYPE;
	const uint32	MacroBlockHashSize = uint32(GetHashSize(MacroBlockHashType));

	Writer.WriteT<uint64>(Manifest.Files.size());

	for (auto& It : Manifest.Files)
	{
		std::string NameUtf8 = ConvertWideToUtf8(It.first);

		FHash256 NameHash = HashBlake3String<FHash256>(NameUtf8);
		Writer.WriteT<FHash256>(NameHash);

		const FFileManifest& FileManifest = It.second;
		Writer.WriteT<uint64>(FileManifest.MacroBlocks.size());

		for (const FGenericBlock& Block : FileManifest.MacroBlocks)
		{
			UNSYNC_ASSERT(Block.HashStrong.Type == MacroBlockHashType);

			Writer.WriteT<uint64>(Block.Offset);
			Writer.WriteT<uint32>(Block.Size);
			Writer.Write(Block.HashStrong.Data, MacroBlockHashSize);
		}
	}

	return Result;
}

static FBuffer
SaveFileReadOnlyMask(const FDirectoryManifest& Manifest)
{
	FBuffer			 Result;
	FVectorStreamOut Writer(Result);

	const uint64 NumFiles		  = Manifest.Files.size();
	const uint64 NumBitArrayBytes = DivUp(NumFiles, 8);

	Result.Resize(NumBitArrayBytes);
	memset(Result.Data(), 0, Result.Size());

	uint64 FileIndex = 0;
	for (const auto& FileIt : Manifest.Files)
	{
		const FFileManifest& FileManifest = FileIt.second;
		BitArraySet(Result.Data(), FileIndex, FileManifest.bReadOnly);
		++FileIndex;
	}

	return Result;
}

static FBuffer
SaveFileExecutableBit(const FDirectoryManifest& Manifest)
{
	FBuffer			 Result;
	FVectorStreamOut Writer(Result);

	const uint64 NumFiles		  = Manifest.Files.size();
	const uint64 NumBitArrayBytes = DivUp(NumFiles, 8);

	Result.Resize(NumBitArrayBytes);
	memset(Result.Data(), 0, Result.Size());

	uint64 FileIndex = 0;
	for (const auto& FileIt : Manifest.Files)
	{
		const FFileManifest& FileManifest = FileIt.second;
		BitArraySet(Result.Data(), FileIndex, FileManifest.bIsExecutable);
		++FileIndex;
	}

	return Result;
}

static FBuffer
SaveFileRevisionControl(const FDirectoryManifest& Manifest)
{
	FBuffer			 Result;
	FVectorStreamOut Writer(Result);

	const uint64 NumFiles = Manifest.Files.size();
	Writer.WriteT(NumFiles);
	for (const auto& FileIt : Manifest.Files)
	{
		Writer.WriteString(FileIt.second.RevisionControlIdentity);
	}

	return Result;
}

std::vector<std::string>
LoadFileRevisionControl(FIOReaderStream& Reader, FSerializedSectionHeader Header)
{
	std::vector<std::string> Result;

	uint64 NumFiles = 0;
	Serialize(Reader, NumFiles);

	Result.resize(NumFiles);

	for (uint64 FileIndex = 0; FileIndex < NumFiles; ++FileIndex)
	{
		Serialize(Reader, Result[FileIndex]);
	}

	return Result;
}

static FBuffer
SavePackReferences(const FDirectoryManifest& Manifest)
{
	FBuffer			 Result;
	FVectorStreamOut Writer(Result);

	const uint64 NumEntries = Manifest.PackReferences.size();
	Writer.WriteT(NumEntries);
	for (const FPackReference& Hash : Manifest.PackReferences)
	{
		Writer.WriteT(Hash);
	}

	return Result;
}

static std::vector<FPackReference>
LoadPackReferences(FIOReaderStream& Reader, FSerializedSectionHeader Header)
{
	std::vector<FPackReference> Result;

	uint64 NumEntries = 0;
	Serialize(Reader, NumEntries);

	Result.resize(NumEntries);

	for (uint64 i = 0; i < NumEntries; ++i)
	{
		Serialize(Reader, Result[i]);
	}

	return Result;
}

bool  // TODO: return a TResult
LoadDirectoryManifest(FDirectoryManifest& OutManifest, const FPath& Root, FIOReaderStream& Stream)
{
	OutManifest = FDirectoryManifest();

	if (!Stream.IsValid())
	{
		return false;
	}

	uint64 Magic = 0;
	Serialize(Stream, Magic);
	if (Magic != FDirectoryManifest::MAGIC)
	{
		UNSYNC_ERROR(L"Directory manifest header mismatch");
		return false;
	}
	uint64 Version = 0;
	Serialize(Stream, Version);
	if (!IsCompatibleManifestVersion(Version))
	{
		UNSYNC_ERROR(L"Directory manifest version mismatch");
		return false;
	}

	UNSYNC_VERBOSE(L"Manifest format version: %llu", Version);
	OutManifest.Version = Version;

	EAlgorithmCompatibilityResult AlgorithmCompatibility = EAlgorithmCompatibilityResult::Ok;

	auto LoadOptionsSectionV5 = [](FIOReaderStream& Stream, FAlgorithmOptionsV5& OutOptions) {
		FAlgorithmOptionsV5 OptionsV5;
		Serialize(Stream, OptionsV5);

		OutOptions = OptionsV5;

		return IsCompatibleAlgorithm(OptionsV5);
	};

	size_t HashSize = FHash128::Size();

	if (Version >= FDirectoryManifest::EVersions::V6_VariableHash)
	{
		AlgorithmCompatibility = LoadOptionsSectionV5(Stream, OutManifest.Algorithm);
		HashSize			   = GetHashSize(ToHashType(OutManifest.Algorithm.StrongHashAlgorithmId));
	}

	std::unordered_map<FHash256, FGenericBlockArray> MacroBlocks;

	FBuffer FileReadOnlyMask;
	FBuffer FileExecutableMask;
	std::vector<std::string> FileRevisions; // TODO: use linear arena to store all strings for a manifest

	if (Version >= FDirectoryManifest::EVersions::V7_OptionalSections)
	{
		// Load optional sections until terminator is encountered

		FSerializedSectionHeader SectionHeader				  = {};
		bool					 bDoneLoadingOptionalSections = false;
		while (!bDoneLoadingOptionalSections)
		{
			Serialize(Stream, SectionHeader);
			if (SectionHeader.Magic == FSerializedSectionHeader::MAGIC)
			{
				const uint64 StreamPosBeforeSection = Stream.Tell();
				const uint64 StreamPosAfterSection	= StreamPosBeforeSection + SectionHeader.Size;

				switch (SectionHeader.Id)
				{
					default:
						// Skip unknown optional sections
						break;
					case SERIALIZED_SECTION_ID_METADATA_STRING:
						{
							FMetadataStringSection Section;
							Serialize(Stream, Section.NameUtf8);
							Serialize(Stream, Section.ValueUtf8);
							UNSYNC_VERBOSE(L"Metadata: %hs = %hs",
										   Section.NameUtf8.c_str(),
										   Section.ValueUtf8.c_str());	// TODO: print as utf8
							break;
						}
					case SERIALIZED_SECTION_ID_MACRO_BLOCK:
						{
							MacroBlocks = LoadMacroBlocks(Stream, SectionHeader.Version);
							break;
						}
					case SERIALIZED_SECTION_ID_FILE_READ_ONLY_MASK:
						{
							FileReadOnlyMask = LoadFileReadOnlyMask(Stream, SectionHeader);
							break;
						}
					case SERIALIZED_SECTION_ID_FILE_REVISION_CONTROL:
						{
							FileRevisions = LoadFileRevisionControl(Stream, SectionHeader);
							OutManifest.bHasFileRevisionControl = true;
							break;
						}
					case SERIALIZED_SECTION_ID_PACK_REFERENCE:
						{
							OutManifest.PackReferences = LoadPackReferences(Stream, SectionHeader);
							break;
						}
					case SERIALIZED_SECTION_ID_FILE_EXECUTABLE_BIT:
						{
							FileExecutableMask = LoadFileExecutableBit(Stream, SectionHeader);
							break;
						}
					case SERIALIZED_SECTION_ID_TERMINATOR:
						bDoneLoadingOptionalSections = true;
						break;
				}

				uint64 StreamPos = Stream.Tell();
				UNSYNC_ASSERT(StreamPos <= StreamPosAfterSection);

				if (StreamPos != StreamPosAfterSection)
				{
					Stream.Seek(StreamPosAfterSection);
				}
			}
			else
			{
				UNSYNC_ERROR(L"Unexpected serialized section header magic identifier. Expected %llX, got %llX.",
							 FSerializedSectionHeader::MAGIC,
							 SectionHeader.Magic);
				return false;
			}
		}
	}

	uint64 NumFiles = 0;

	if (AlgorithmCompatibility == EAlgorithmCompatibilityResult::Ok)
	{
		Serialize(Stream, NumFiles);

		const bool bReadOnlyMaskValid = NumFiles <= FileReadOnlyMask.Size() * 8;
		const bool bExecutableBitValid = NumFiles <= FileExecutableMask.Size() * 8;

		std::string FilenameUtf8; // shared buffer to avoid some of the reallocations
		for (uint64 FileIndex = 0; FileIndex < NumFiles; ++FileIndex)
		{
			Serialize(Stream, FilenameUtf8);
			ConvertDirectorySeparatorsToNative(FilenameUtf8);
			
			FFileManifest FileManifest;
			Serialize(Stream, FileManifest.Mtime);
			Serialize(Stream, FileManifest.Size);
			Serialize(Stream, FileManifest.BlockSize);
			uint64 NumBlocks = 0;
			Serialize(Stream, NumBlocks);
			if (NumBlocks)
			{
				FileManifest.Blocks.reserve(NumBlocks);

				for (uint64 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
				{
					// TODO: batch-read the blocks
					FGenericBlock Block;

					Serialize(Stream, Block.Offset);
					Serialize(Stream, Block.Size);
					Serialize(Stream, Block.HashWeak);
					Stream.Read((char*)Block.HashStrong.Data, HashSize);

					FileManifest.Blocks.push_back(Block);
				}
			}

			FHash256 NameHash = HashBlake3String<FHash256>(FilenameUtf8);

			auto MacroBlockListIt = MacroBlocks.find(NameHash);
			if (MacroBlockListIt != MacroBlocks.end())
			{
				FileManifest.MacroBlocks = MacroBlockListIt->second;
			}

			std::wstring Filename = ConvertUtf8ToWide(FilenameUtf8);

			if (FileManifest.Mtime == 0)
			{
				UNSYNC_ERROR(L"Invalid manifest entry for file '%ls' (Mtime is 0)", Filename.c_str());
				return false;
			}

			if (FileManifest.BlockSize == 0)
			{
				UNSYNC_ERROR(L"Invalid manifest entry for file '%ls' (BlockSize is 0)", Filename.c_str());
				return false;
			}

			FileManifest.CurrentPath = Root / Filename;

			if (bReadOnlyMaskValid)
			{
				FileManifest.bReadOnly = BitArrayGet(FileReadOnlyMask.Data(), FileIndex);
			}

			if (bExecutableBitValid)
			{
				FileManifest.bIsExecutable = BitArrayGet(FileExecutableMask.Data(), FileIndex);
			}

			if (OutManifest.bHasFileRevisionControl)
			{
				FileManifest.RevisionControlIdentity = std::move(FileRevisions[FileIndex]);
			}

			// Store output
			OutManifest.Files[Filename] = std::move(FileManifest);
		}
	}

	// Old manifest versions always stored options after file blocks
	if (Version < FDirectoryManifest::EVersions::V6_VariableHash)
	{
		AlgorithmCompatibility = LoadOptionsSectionV5(Stream, OutManifest.Algorithm);
	}

	if (AlgorithmCompatibility != EAlgorithmCompatibilityResult::Ok)
	{
		UNSYNC_ERROR(L"%hs", ToString(AlgorithmCompatibility));
		return false;
	}

	// Set the hash type in all file manifests
	const EHashType HashType = ToHashType(OutManifest.Algorithm.StrongHashAlgorithmId);
	for (auto& It : OutManifest.Files)
	{
		FFileManifest& FileManifest = It.second;
		for (FGenericBlock& Block : FileManifest.Blocks)
		{
			Block.HashStrong.Type = HashType;
		}
	}

	return true;
}

bool  // TODO: return a TResult
LoadDirectoryManifest(FDirectoryManifest& OutManifest, const FPath& Root, const FPath& ManifestFilename)
{
	// TODO: use compact binary to store the manifest
	UNSYNC_LOG_INDENT;
	UNSYNC_VERBOSE(L"Loading directory manifest from '%ls'", ManifestFilename.wstring().c_str());

	// FNativeFile manifest_file(manifest_filename, FileMode::ReadOnly);
	FBuffer ManifestBuffer = ReadFileToBuffer(ManifestFilename);

	if (ManifestBuffer.Size() != 0)
	{
		FMemReader		ManifestReader(ManifestBuffer);
		FIOReaderStream ManifestStream(ManifestReader);
		return LoadDirectoryManifest(OutManifest, Root, ManifestStream);
	}
	else
	{
		UNSYNC_ERROR(L"Failed to open manifest file");
		OutManifest = FDirectoryManifest();
		return false;
	}
}

static void
WriteSection(FVectorStreamOut& Stream, uint64 SectionMagic, uint64 SectionVersion, FBufferView SectionData)
{
	FSerializedSectionHeader Header;
	Header.Id	   = SectionMagic;
	Header.Version = SectionVersion;
	Header.Size	   = SectionData.Size;

	Serialize(Stream, Header);
	Stream.Write((const char*)SectionData.Data, (size_t)SectionData.Size);
}

static void
WriteSection(FVectorStreamOut& Stream, const FMetadataStringSection& Section)
{
	FBuffer			 StreamBuffer;
	FVectorStreamOut SectionStream(StreamBuffer);
	SectionStream.WriteString(Section.NameUtf8);
	SectionStream.WriteString(Section.ValueUtf8);
	WriteSection(Stream, FMetadataStringSection::MAGIC, FMetadataStringSection::VERSION, StreamBuffer.View());
}

template<typename SectionHeaderType>
static void
WriteSection(FVectorStreamOut& Stream, FBufferView SectionData)
{
	WriteSection(Stream, SectionHeaderType::MAGIC, SectionHeaderType::VERSION, SectionData);
}

bool
ManifestHasMacroBlocks(const FDirectoryManifest& Manifest)
{
	// TODO: store the macro block count in the manifest runtime data
	for (const auto& FileIt : Manifest.Files)
	{
		if (!FileIt.second.MacroBlocks.empty())
		{
			return true;
		}
	}
	return false;
}

bool
SaveDirectoryManifest(const FDirectoryManifest& Manifest, FVectorStreamOut& Stream)
{
	// TODO: use compact binary to store the manifest

	const size_t HashSize = GetHashSize(ToHashType(Manifest.Algorithm.StrongHashAlgorithmId));

	FBuffer			 BlockStreamBuffer;
	FVectorStreamOut BlockStream(BlockStreamBuffer);

	// Save mandatory fields

	Serialize(Stream, FDirectoryManifest::MAGIC);
	Serialize(Stream, FDirectoryManifest::VERSION);

	Serialize(Stream, Manifest.Algorithm);

	// Save optional sections

	{
		FMetadataStringSection Section;
		Section.NameUtf8  = std::string("unsync.version");
		Section.ValueUtf8 = GetVersionString();
		WriteSection(Stream, Section);
	}

	{
		FHash160			   StableSignature	  = ToHash160(ComputeManifestStableSignature(Manifest));
		std::string			   StableSignatureStr = HashToHexString(StableSignature);
		FMetadataStringSection Section;
		Section.NameUtf8  = std::string("unsync.signature");
		Section.ValueUtf8 = StableSignatureStr;
		WriteSection(Stream, Section);
	}

	if (ManifestHasMacroBlocks(Manifest))
	{
		FBuffer SectionBuffer = SaveMacroBlocks(Manifest);
		WriteSection<FMacroBlockSection>(Stream, SectionBuffer.View());
	}

	{
		FBuffer SectionBuffer = SaveFileReadOnlyMask(Manifest);
		WriteSection<FFileReadOnlyMaskSection>(Stream, SectionBuffer.View());
	}

	{
		FBuffer SectionBuffer = SaveFileExecutableBit(Manifest);
		WriteSection<FFileExecutableBitSection>(Stream, SectionBuffer.View());
	}

	if (Manifest.bHasFileRevisionControl)
	{
		FBuffer SectionBuffer = SaveFileRevisionControl(Manifest);
		WriteSection<FFileRevisionControlSection>(Stream, SectionBuffer.View());
	}

	if (!Manifest.PackReferences.empty())
	{
		FBuffer SectionBuffer = SavePackReferences(Manifest);
		WriteSection<FPackReferenceSection>(Stream, SectionBuffer.View());
	}

	// End with the terminator section (default-constructed);
	FSerializedSectionHeader TerminatorSection;
	Serialize(Stream, TerminatorSection);

	// Save the file manifests (must always be last)

	uint64 NumFiles = Manifest.Files.size();
	Serialize(Stream, NumFiles);
	for (const auto& It : Manifest.Files)
	{
		std::string FilenameUtf8 = ConvertWideToUtf8(It.first);
		ConvertDirectorySeparatorsToUnix(FilenameUtf8);

		Serialize(Stream, FilenameUtf8);  // name as utf8

		const FFileManifest& FileManifest = It.second;
		Serialize(Stream, FileManifest.Mtime);
		Serialize(Stream, FileManifest.Size);
		Serialize(Stream, FileManifest.BlockSize);
		uint64 NumBlocks = FileManifest.Blocks.size();
		Serialize(Stream, NumBlocks);
		if (NumBlocks)
		{
			BlockStream.Output.Clear();

			for (uint64 I = 0; I < NumBlocks; ++I)
			{
				const FGenericBlock& Block = FileManifest.Blocks[I];
				BlockStream.WriteT(Block.Offset);
				BlockStream.WriteT(Block.Size);
				BlockStream.WriteT(Block.HashWeak);
				BlockStream.Write(Block.HashStrong.Data, HashSize);
			}

			Stream.Write((const char*)BlockStream.Output.Data(), (size_t)BlockStream.Output.Size());
		}
	}

	return true;
}

bool
SaveDirectoryManifest(const FDirectoryManifest& Manifest, const FPath& Filename, bool bAllowInDryRun)
{
	FBuffer			 OutputBuffer;
	FVectorStreamOut OutputStream(OutputBuffer);

	UNSYNC_LOG_INDENT;

	bool bSerialized = SaveDirectoryManifest(Manifest, OutputStream);
	UNSYNC_ASSERT(bSerialized);

	EFileMode FileMode = EFileMode ::CreateWriteOnly;
	if (bAllowInDryRun)
	{
		FileMode = FileMode | EFileMode::IgnoreDryRun;
	}

	FNativeFile OutputFile(Filename, FileMode, OutputBuffer.Size());
	if (OutputFile.IsValid())
	{
		uint64 WroteBytes = OutputFile.Write(OutputBuffer.Data(), 0, OutputBuffer.Size());
		return WroteBytes == OutputBuffer.Size();
	}
	else
	{
		UNSYNC_ERROR(L"Failed to open manifest output file '%ls' for writing", Filename.wstring().c_str());
		return false;
	}
}

}  // namespace unsync
