// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCore.h"

namespace unsync {

struct FIOReaderStream;
struct FVectorStreamOut;

bool LoadBlocks(FGenericBlockArray& OutBlocks, uint32& OutBlockSize, const FPath& Filename);
bool SaveBlocks(const std::vector<FBlock128>& Blocks, uint32 BlockSize, const FPath& Filename);

bool LoadDirectoryManifest(FDirectoryManifest& OutManifest, const FPath& Root, FIOReaderStream& Stream);
bool LoadDirectoryManifest(FDirectoryManifest& OutManifest, const FPath& Root, const FPath& Filename);

bool SaveDirectoryManifest(const FDirectoryManifest& Manifest, FVectorStreamOut& Stream);
bool SaveDirectoryManifest(const FDirectoryManifest& Manifest, const FPath& Filename, bool bAllowInDryRun = false);

}  // namespace unsync
