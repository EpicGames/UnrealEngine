// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncCommon.h"

UNSYNC_THIRD_PARTY_INCLUDES_START

#if UNSYNC_PLATFORM_WINDOWS
#	include <winsock2.h>  // must be first
#	include <ws2tcpip.h>
#	pragma comment(lib, "Ws2_32.lib")
#endif	// UNSYNC_PLATFORM_WINDOWS

#if UNSYNC_PLATFORM_UNIX
#	include <sys/socket.h>
#	include <sys/types.h>
#	include <netinet/in.h>
#	include <netdb.h>
#	include <arpa/inet.h>
#	include <unistd.h>
#	define SOCKET		   int
#	define INVALID_SOCKET (SOCKET)(~0)
#	define SOCKET_ERROR   (-1)
#	define closesocket(x) close(x)
#endif	// UNSYNC_PLATFORM_UNIX

#include <unordered_set>
#include <limits>

#include <tls.h>

UNSYNC_THIRD_PARTY_INCLUDES_END

#include "UnsyncBuffer.h"
#include "UnsyncHash.h"
#include "UnsyncLog.h"
#include "UnsyncSocket.h"
#include "UnsyncUtil.h"

namespace unsync {

static_assert(sizeof(FSocketAddress) >= sizeof(sockaddr), "SocketAddress is too small");
static_assert(sizeof(FSocketAddress) >= sizeof(sockaddr_in), "SocketAddress is too small");

const FSocketHandle InvalidSocketHandle = INVALID_SOCKET;

struct FSocketInitHelper
{
	FSocketInitHelper()
	{
#if UNSYNC_PLATFORM_WINDOWS
		WSADATA Wsa;
		if (WSAStartup(MAKEWORD(2, 2), &Wsa) != 0)
		{
			UNSYNC_ERROR(L"WSAStartup failed: %d", WSAGetLastError());
		}
#endif	// UNSYNC_PLATFORM_WINDOWS

		tls_init();
	}
	~FSocketInitHelper()
	{
#if UNSYNC_PLATFORM_WINDOWS
		WSACleanup();
#endif	// UNSYNC_PLATFORM_WINDOWS
	}
};

static void
LazyInitSockets()
{
	static FSocketInitHelper InitHelper;
}

std::string
GetCurrentHostName()
{
	LazyInitSockets();

	char Buffer[1024] = {};

	if (gethostname(Buffer, (int)sizeof(Buffer)) == 0)
	{
		return std::string(Buffer);
	}
	else
	{
		return {};
	}
}

static int32
GetLastSocketError()
{
#if UNSYNC_PLATFORM_WINDOWS
	return WSAGetLastError();
#else
	return -1; // TODO: report unix socket error
#endif
}

static void
ReportSocketError(int32 ErrorCode, ELogLevel LogLevel)
{
#if UNSYNC_PLATFORM_WINDOWS
	ErrorCode = WSAGetLastError();
#endif

	LogPrintf(LogLevel, L"Socket error: %d\n", ErrorCode);
	UNSYNC_BREAK_ON_ERROR_LEVEL(LogLevel);
}

static void
ReportSocketError(tls* TlsCtx, ELogLevel LogLevel)
{
	const char* ErrorString = tls_error(TlsCtx);
	LogPrintf(LogLevel, L"TLS error: %hs\n", ErrorString);
	UNSYNC_BREAK_ON_ERROR_LEVEL(LogLevel);
}

FSocketHandle
SocketListenTcp(const char* Address, uint16 Port)
{
	LazyInitSockets();

	SOCKET ListenSocket = socket(AF_INET, SOCK_STREAM, 0);

	if (ListenSocket == InvalidSocketHandle)
	{
		UNSYNC_ERROR(L"Failed to create TCP socket (error code %d)", GetLastSocketError());
		return InvalidSocketHandle;
	}

	sockaddr_in Service;
	Service.sin_family		= AF_INET;
	Service.sin_addr.s_addr = inet_addr(Address);
	Service.sin_port		= htons(Port);

	int32 BindResult = bind(ListenSocket, (sockaddr*)&Service, sizeof(Service));
	if (BindResult == SOCKET_ERROR)
	{
		UNSYNC_ERROR(L"Failed to bind TCP socket (error code %d)", GetLastSocketError());
		SocketClose(ListenSocket);
		return InvalidSocketHandle;
	}

	int32 ListenResult = listen(ListenSocket, 1);
	if (ListenResult == SOCKET_ERROR)
	{
		UNSYNC_ERROR(L"Failed to listen on TCP socket (error code %d)", GetLastSocketError());
		SocketClose(ListenSocket);
		return InvalidSocketHandle;
	}

	return ListenSocket;
}

FSocketHandle
SocketAccept(FSocketHandle ListenSocket)
{
	SOCKET AcceptSocket = accept(ListenSocket, nullptr, nullptr);
	if (AcceptSocket == InvalidSocketHandle)
	{
		UNSYNC_ERROR(L"Failed to accept connection on TCP socket (error code %d)", GetLastSocketError());
		return InvalidSocketHandle;
	}
	return AcceptSocket;
}

FSocketHandle
SocketConnectTcp(const char* DestAddress, uint16 Port)
{
	LazyInitSockets();

	SOCKET Sock = socket(AF_INET, SOCK_STREAM, 0);

	if (Sock == InvalidSocketHandle)
	{
		UNSYNC_ERROR(L"Failed to create TCP socket (error code %d)", GetLastSocketError());
		return InvalidSocketHandle;
	}

	sockaddr_in SockAddr = {};
	SockAddr.sin_family	 = AF_INET;
	SockAddr.sin_port	 = htons(Port);

	struct addrinfo AddrHints = {};
	AddrHints.ai_family		  = PF_UNSPEC;
	AddrHints.ai_socktype	  = SOCK_STREAM;
	AddrHints.ai_flags		  = AI_CANONNAME;

	struct addrinfo* Addr = {};

	char ResolvedAddress[100] = {};

	int Err = getaddrinfo(DestAddress, nullptr, &AddrHints, &Addr);
	if (Err == 0)
	{
		struct addrinfo* Curr = Addr;
		while (Curr)
		{
			if (Curr->ai_family == SockAddr.sin_family)
			{
				void* AddrData = nullptr;
				AddrData	   = &((struct sockaddr_in*)Curr->ai_addr)->sin_addr;
				if (inet_ntop(Curr->ai_family, AddrData, ResolvedAddress, sizeof(ResolvedAddress)))
				{
					DestAddress = ResolvedAddress;
					break;
				}
			}
			Curr = Curr->ai_next;
		}
	}
	else
	{
		UNSYNC_ERROR(L"Invalid address '%hs'", DestAddress);
		return InvalidSocketHandle;
	}

	if (Addr)
	{
		freeaddrinfo(Addr);
	}

	if (inet_pton(SockAddr.sin_family, DestAddress, &SockAddr.sin_addr) <= 0)
	{
		UNSYNC_ERROR(L"Invalid address '%hs'", DestAddress);
		return InvalidSocketHandle;
	}

	int32 ConnectResult = connect(Sock, (struct sockaddr*)&SockAddr, sizeof(SockAddr));
	if (ConnectResult < 0)
	{
		closesocket(Sock);
		UNSYNC_LOG(L"Warning: Failed to connect to '%hs' on port %d", DestAddress, Port);
		return InvalidSocketHandle;
	}

	return Sock;
}

void
SocketClose(FSocketHandle Socket)
{
	if (Socket != InvalidSocketHandle)
	{
		closesocket(Socket);
	}
}

int
SocketSend(FSocketHandle Socket, const void* Data, size_t DataSize)
{
	if (DataSize == 0)
	{
		return 0;
	}

	UNSYNC_ASSERT(DataSize < INT_MAX);

	int Result = send(Socket, reinterpret_cast<const char*>(Data), (int)DataSize, 0);

	if (Result < 0)
	{
		ReportSocketError(Result, ELogLevel::Warning);
		return 0;
	}
	else
	{
		return Result;
	}
}

int
SocketRecvAll(FSocketHandle Socket, void* Data, size_t DataSize)
{
	if (DataSize == 0)
		return 0;

	UNSYNC_ASSERT(DataSize < INT_MAX);

	// If socket timeout is set, recv may read partial data and return the
	// number of bytes read before timeout or <=0 if another error occurred.

	int ProcessedBytes = 0;
	while (ProcessedBytes < int(DataSize))
	{
		char* BatchPtr		= reinterpret_cast<char*>(Data) + ProcessedBytes;
		int	  BatchSize		= (int)DataSize - ProcessedBytes;
		int	  Res = recv(Socket, BatchPtr, BatchSize, MSG_WAITALL);
		if (Res <= 0)
		{
			if (Res < 0)
			{
				ReportSocketError(Res, ELogLevel::Warning);
			}
			break;
		}
		ProcessedBytes += Res;
	}

	return ProcessedBytes;
}

int
SocketRecvAny(FSocketHandle Socket, void* Data, size_t DataSize)
{
	if (DataSize == 0)
		return 0;

	UNSYNC_ASSERT(DataSize < INT_MAX);

	int Res = recv(Socket, reinterpret_cast<char*>(Data), (int)DataSize, 0);

	if (Res < 0)
	{
		ReportSocketError(Res, ELogLevel::Warning);
		return 0;
	}

	return Res;
}

bool
SocketSetRecvTimeout(FSocketHandle Socket, uint32 Seconds)
{
#if UNSYNC_PLATFORM_WINDOWS
	uint32 Timeout = Seconds * 1000;  // Winsock uses timeout in milliseconds
#else
	struct timeval Timeout;
	Timeout.tv_sec	= long(Seconds);
	Timeout.tv_usec = 0;
#endif

	int Result = setsockopt(Socket, SOL_SOCKET, SO_RCVTIMEO, reinterpret_cast<const char*>(&Timeout), int(sizeof(Timeout)));

	return Result == 0;
}

bool
SocketValid(FSocketHandle Socket)
{
	if (Socket == INVALID_SOCKET)
	{
		return false;
	}

	int RecvResult = send(Socket, nullptr, 0, 0);
	return RecvResult != SOCKET_ERROR;
}

FSocketAddress
SocketMakeAddress(const char* Address, uint16 Port)
{
	sockaddr_in Addr	 = {};
	Addr.sin_family		 = AF_INET;
	Addr.sin_addr.s_addr = inet_addr(Address);
	Addr.sin_port		 = htons(Port);
	FSocketAddress Result;
	memcpy(&Result, &Addr, sizeof(Addr));
	return Result;
}

int
FSocketRaw::Send(const void* Data, size_t DataSize)
{
	return SocketSend(Handle, Data, DataSize);
}

int
FSocketRaw::RecvAll(void* Data, size_t DataSize)
{
	return SocketRecvAll(Handle, Data, DataSize);
}

int
FSocketRaw::RecvAny(void* Data, size_t DataSize)
{
	return SocketRecvAny(Handle, Data, DataSize);
}

static ssize_t
TlsReadCallback(struct tls* TlsCtx, void* Buffer, size_t BufferSize, void* UserData)
{
	FSocketHandle Socket = reinterpret_cast<FSocketTls*>(UserData)->Handle;
	return SocketRecvAll(Socket, Buffer, BufferSize);
}

static ssize_t
TlsWriteCallback(struct tls* TlsCtx, const void* Buffer, size_t BufferSize, void* UserData)
{
	FSocketHandle Socket = reinterpret_cast<FSocketTls*>(UserData)->Handle;
	return SocketSend(Socket, Buffer, BufferSize);
}

FSocketTls::FSocketTls(FSocketHandle InHandle, FTlsClientSettings ClientSettings) : FSocketBase(InHandle)
{
	Security = ESocketSecurity::Unknown;

	tls_config* TlsCfg = tls_config_new();

	if (ClientSettings.CACert.Data)
	{
		tls_config_set_ca_mem(TlsCfg, ClientSettings.CACert.Data, (size_t)ClientSettings.CACert.Size);
	}
	else
	{
		const FBuffer& SystemRootCerts = GetSystemRootCerts();
		tls_config_set_ca_mem(TlsCfg, SystemRootCerts.Data(), SystemRootCerts.Size());
	}

	if (!ClientSettings.bVerifyCertificate)
	{
		tls_config_insecure_noverifycert(TlsCfg);
	}

	if (!ClientSettings.bVerifySubject)
	{
		tls_config_insecure_noverifyname(TlsCfg);
	}

	TlsCtx = tls_client();

	int Err = 0;

	if (Err == 0)
	{
		tls_configure(TlsCtx, TlsCfg);
	}

	tls_config_free(TlsCfg);

	if (Err == 0)
	{
		UNSYNC_ASSERT(!ClientSettings.Subject.empty());
		std::string TlsSubject(ClientSettings.Subject);
		Err = tls_connect_cbs(TlsCtx, TlsReadCallback, TlsWriteCallback, this, TlsSubject.c_str());
	}

	if (Err == 0)
	{
		for(;;)
		{
			Err = tls_handshake(TlsCtx);
			if (Err != TLS_WANT_POLLIN && Err != TLS_WANT_POLLOUT)
			{
				break;
			}
		}
	}

	if (Err)
	{
		const char* TlsErrorMsg = tls_error(TlsCtx);
		if (TlsErrorMsg)
		{
			UNSYNC_LOG(L"Warning: Failed to establish TLS connection: %hs", TlsErrorMsg);
		}
		else
		{
			UNSYNC_LOG(L"Warning: Failed to establish TLS connection");
		}
		tls_free(TlsCtx);
		TlsCtx = nullptr;
	}
	else
	{
		const char* ConnVersion = tls_conn_version(TlsCtx);
		if (ConnVersion)
		{
			if (!strcmp(ConnVersion, "TLSv1.3"))
			{
				Security = ESocketSecurity::TLSv1_3;
			}
			else if (!strcmp(ConnVersion, "TLSv1.2"))
			{
				Security = ESocketSecurity::TLSv1_2;
			}
		}
	}
}

FSocketTls::~FSocketTls()
{
	if (TlsCtx)
	{
		for(;;)
		{
			int Err = tls_close(TlsCtx);
			if (Err != TLS_WANT_POLLIN && Err != TLS_WANT_POLLOUT)
			{
				break;
			}
		}
		tls_free(TlsCtx);
	}
}

static ssize_t 
TlsWrite(struct tls* TlsCtx, const char* Buffer, size_t BufferSize)
{
	ssize_t ErrorCode;
	do
	{
		ErrorCode = tls_write(TlsCtx, Buffer, BufferSize);
	} 
	while(ErrorCode == TLS_WANT_POLLIN || ErrorCode == TLS_WANT_POLLOUT);

	return ErrorCode;
}

int
FSocketTls::Send(const void* Data, size_t DataSize)
{
	if (DataSize == 0)
		return 0;

	UNSYNC_ASSERT(DataSize < INT_MAX);

	int ProcessedBytes = 0;
	while (ProcessedBytes < DataSize)
	{
		ssize_t ErrorCode = TlsWrite(TlsCtx, (const char*)Data + ProcessedBytes, DataSize - ProcessedBytes);
		if (ErrorCode <= 0)
		{
			if (ErrorCode < 0)
			{
				ReportSocketError(TlsCtx, ELogLevel::Warning);
			}
			break;
		}
		int Res = CheckedNarrow(ErrorCode);
		ProcessedBytes += Res;
	}

	return ProcessedBytes;
}

static ssize_t 
TlsRead(struct tls* TlsCtx, char* Buffer, size_t BufferSize)
{
	ssize_t ErrorCode;
	do
	{
		ErrorCode = tls_read(TlsCtx, Buffer, BufferSize);
	} 
	while(ErrorCode == TLS_WANT_POLLIN || ErrorCode == TLS_WANT_POLLOUT);

	return ErrorCode;
}

int
FSocketTls::RecvAll(void* Data, size_t DataSize)
{
	if (DataSize == 0)
		return 0;

	UNSYNC_ASSERT(DataSize < INT_MAX);

	int ProcessedBytes = 0;
	while (ProcessedBytes < DataSize)
	{
		ssize_t ErrorCode = TlsRead(TlsCtx, (char*)Data + ProcessedBytes, DataSize - ProcessedBytes);
		if (ErrorCode <= 0)
		{
			if (ErrorCode < 0)
			{
				ReportSocketError(TlsCtx, ELogLevel::Warning);
			}
			break;
		}
		int Res = CheckedNarrow(ErrorCode);
		ProcessedBytes += Res;
	}

	return ProcessedBytes;
}

int
FSocketTls::RecvAny(void* Data, size_t DataSize)
{
	if (DataSize == 0)
		return 0;

	UNSYNC_ASSERT(DataSize < INT_MAX);

	int ProcessedBytes = 0;
	do
	{
		ssize_t ErrorCode = TlsRead(TlsCtx, (char*)Data + ProcessedBytes, DataSize - ProcessedBytes);
		if (ErrorCode <= 0)
		{
			if (ErrorCode < 0)
			{
				ReportSocketError(TlsCtx, ELogLevel::Warning);
			}
			break;
		}
		int Res = CheckedNarrow(ErrorCode);
		ProcessedBytes += Res;
	} while (false);

	return ProcessedBytes;
}

FSocketBase::~FSocketBase()
{
	SocketClose(Handle);
}

const char*
ToString(ESocketSecurity Security)
{
	switch (Security)
	{
		case ESocketSecurity::None:
			return "None";
		case ESocketSecurity::TLSv1_2:
			return "TLS 1.2";
		case ESocketSecurity::TLSv1_3:
			return "TLS 1.3";
		default:
			return "Unknown";
	}
}

bool
SendBuffer(FSocketBase& Socket, const FBufferView& Data)
{
	UNSYNC_ASSERT(Data.Size <= std::numeric_limits<int32>::max());
	int32 Size = int32(Data.Size);
	bool  bOk  = true;
	bOk &= SocketSendT(Socket, Size);
	bOk &= (SocketSend(Socket, Data.Data, Size) == Size);
	return bOk;
}

bool
SendBuffer(FSocketBase& Socket, const FBuffer& Data)
{
	return SendBuffer(Socket, Data.View());
}

}  // namespace unsync
