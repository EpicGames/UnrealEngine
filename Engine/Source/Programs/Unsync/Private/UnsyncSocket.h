// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <stdint.h>

#include "UnsyncCommon.h"
#include "UnsyncBuffer.h"

struct tls;

namespace unsync {

class FBuffer;
struct FBufferView;
using FSocketHandle = uintptr_t;

extern const FSocketHandle InvalidSocketHandle;

struct FSocketAddress  // opaque buffer that holds socket address
{
	FSocketAddress()
	{
		for (uint64& X : Data64)
			X = 0;
	}

	union
	{
		char   Data[16];
		uint64 Data64[2];
	};

	bool operator==(const FSocketAddress& Other) const { return Data64[0] == Other.Data64[0] && Data64[1] == Other.Data64[1]; }
};

// Returns current host name or empty string if it could not be determined.
std::string GetCurrentHostName();

FSocketHandle SocketListenTcp(const char* Address, uint16 Port);
FSocketHandle SocketAccept(FSocketHandle ListenSocket);

FSocketHandle SocketConnectTcp(const char* DestAddress, uint16 Port);
void		  SocketClose(FSocketHandle Socket);

int			   SocketSend(FSocketHandle Socket, const void* Data, size_t DataSize);
int			   SocketRecvAll(FSocketHandle Socket, void* Data, size_t DataSize);
int			   SocketRecvAny(FSocketHandle Socket, void* Data, size_t DataSize);
FSocketAddress SocketMakeAddress(const char* Address, uint16 Port);

bool SocketSetRecvTimeout(FSocketHandle Socket, uint32 Seconds);

bool SocketValid(FSocketHandle Socket);

template<typename T>
bool
SocketSendT(FSocketHandle Socket, const T& Data)
{
	int SentBytes = SocketSend(Socket, &Data, sizeof(Data));
	return SentBytes == sizeof(Data);
}

template<typename T>
bool
SocketRecvT(FSocketHandle Socket, T& Data)
{
	return SocketRecvAll(Socket, &Data, sizeof(Data)) == sizeof(Data);
}

enum class ESocketSecurity
{
	None,
	TLSv1_2,
	TLSv1_3,
	Unknown
};

const char* ToString(ESocketSecurity Security);

struct FSocketBase
{
	UNSYNC_DISALLOW_COPY_ASSIGN(FSocketBase)

	explicit FSocketBase(FSocketHandle InHandle) : Handle(InHandle) {}

	virtual ~FSocketBase();

	virtual int Send(const void* Data, size_t DataSize) = 0;
	virtual int RecvAll(void* Data, size_t DataSize)	= 0;
	virtual int RecvAny(void* Data, size_t DataSize)	= 0;

	virtual bool IsEncrypted() const = 0;

	FSocketHandle	Handle	 = {};
	ESocketSecurity Security = ESocketSecurity::None;
};

struct FSocketRaw : FSocketBase
{
	explicit FSocketRaw(FSocketHandle InHandle) : FSocketBase(InHandle) {}

	virtual int Send(const void* Data, size_t DataSize) override;
	virtual int RecvAll(void* Data, size_t DataSize) override;
	virtual int RecvAny(void* Data, size_t DataSize) override;

	virtual bool IsEncrypted() const override { return false; }
};

enum class ETlsRequirement : uint8 {
	None,		// Use defaults for the underlying protocol
	Preferred,	// Try TLS connection, but allow falling back to plaintext if server supports it
	Required,	// Only allow connections with TLS
};

struct FTlsClientSettings
{
	std::string_view Subject			= {};
	FBufferView		 CACert				= {};
	bool			 bVerifyCertificate = true;
	bool			 bVerifySubject		= true;
};

struct FSocketTls : FSocketBase
{
	FSocketTls(FSocketHandle InHandle, FTlsClientSettings ClientSettings);
	~FSocketTls();

	bool IsTlsValid() const { return TlsCtx != nullptr; }

	virtual int Send(const void* Data, size_t DataSize) override;
	virtual int RecvAll(void* Data, size_t DataSize) override;
	virtual int RecvAny(void* Data, size_t DataSize) override;

	virtual bool IsEncrypted() const override { return IsTlsValid(); }

	tls* TlsCtx = {};
};

template<typename T>
bool
SocketSendT(FSocketBase& Socket, const T& Data)
{
	int SentBytes = Socket.Send(&Data, sizeof(Data));
	return SentBytes == sizeof(Data);
}

template<typename T>
bool
SocketRecvT(FSocketBase& Socket, T& Data)
{
	return Socket.RecvAll(&Data, sizeof(Data)) == sizeof(Data);
}

inline int
SocketSend(FSocketBase& Socket, const void* Data, size_t DataSize)
{
	return Socket.Send(Data, DataSize);
}

inline int
SocketRecvAll(FSocketBase& Socket, void* Data, size_t DataSize)
{
	return Socket.RecvAll(Data, DataSize);
}

inline int
SocketRecvAny(FSocketBase& Socket, void* Data, size_t DataSize)
{
	return Socket.RecvAny(Data, DataSize);
}

inline bool
SocketValid(FSocketBase& Socket)
{
	return SocketValid(Socket.Handle);
}

template<typename T>
bool
SendStruct(FSocketBase& Socket, const T& Data)
{
	int32 Size = int32(sizeof(Data));
	bool  bOk  = true;
	bOk &= SocketSendT(Socket, Size);
	bOk &= SocketSendT(Socket, Data);
	return bOk;
}

bool SendBuffer(FSocketBase& Socket, const FBufferView& Data);
bool SendBuffer(FSocketBase& Socket, const FBuffer& Data);

template<typename T>
bool
RecvStruct(FSocketBase& Socket, const T& Data)
{
	int32 Size = int32(sizeof(Data));
	bool  bOk  = true;
	bOk &= SocketRecvT(Socket, Size);
	bOk &= SocketRecvT(Socket, Data);
	return bOk;
}

}  // namespace unsync
