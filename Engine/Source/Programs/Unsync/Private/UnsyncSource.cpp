// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncSource.h"
#include "UnsyncUtil.h"

#if UNSYNC_PLATFORM_WINDOWS
#	include <Windows.h>
#	include <shellapi.h>
#	include <lm.h>
#	include <lmdfs.h>
#	include <wincrypt.h>
#	include <winnetwk.h>  // for WNetGetUniversalName
#endif					   // UNSYNC_PLATFORM_WINDOWS

namespace unsync {

FDfsMirrorInfo
DfsEnumerate(const FPath& Root)
{
	FDfsMirrorInfo Result;

#if UNSYNC_PLATFORM_WINDOWS

	std::wstring RootPathLower = StringToLower(Root.native());

	LPWSTR RootPathCstr = (LPWSTR)Root.c_str();

	DWORD ResumeHandle = 0;

	std::vector<PDFS_INFO_3> InfosToFree;

	PDFS_INFO_3	 BestMatchEntry = nullptr;
	std::wstring BestMatchPath;

	for (;;)
	{
		DWORD		EntriesRead = 0;
		PDFS_INFO_3 DfsInfoRoot = nullptr;
		DWORD		Res			= NetDfsEnum(RootPathCstr, 3, MAX_PREFERRED_LENGTH, (LPBYTE*)&DfsInfoRoot, &EntriesRead, &ResumeHandle);
		if (Res == ERROR_NO_MORE_ITEMS)
		{
			break;
		}
		else if (Res == RPC_S_INVALID_NET_ADDR)
		{
			// Not a network share root, so nothing to do
			break;
		}
		else if (Res != ERROR_SUCCESS)
		{
			UNSYNC_LOG(L"DFS enumeration failed with error: %d", Res);
			break;
		}

		PDFS_INFO_3 DfsInfo = DfsInfoRoot;

		for (DWORD I = 0; I < EntriesRead; I++)
		{
			std::wstring EntryPathLower = StringToLower(DfsInfo->EntryPath);

			// entry prefix must match requested root path
			if (RootPathLower.find(EntryPathLower) == 0 && (RootPathLower.length() > BestMatchPath.length()))
			{
				DWORD			  NumOnlineStorages = 0;
				PDFS_STORAGE_INFO StorageInfo		= DfsInfo->Storage;
				for (DWORD J = 0; J < DfsInfo->NumberOfStorages; J++)
				{
					if (StorageInfo->State != DFS_STORAGE_STATE_OFFLINE)
					{
						NumOnlineStorages++;
					}
					++StorageInfo;
				}

				BestMatchPath  = DfsInfo->EntryPath;
				BestMatchEntry = DfsInfo;
			}

			++DfsInfo;
		}

		InfosToFree.push_back(DfsInfoRoot);
	}

	if (BestMatchEntry)
	{
		Result.Root					  = BestMatchPath;
		PDFS_STORAGE_INFO StorageInfo = BestMatchEntry->Storage;
		Result.Storages.reserve(BestMatchEntry->NumberOfStorages);
		for (DWORD J = 0; J < BestMatchEntry->NumberOfStorages; J++)
		{
			if (StorageInfo->State != DFS_STORAGE_STATE_OFFLINE)
			{
				FDfsStorageInfo ResultEntry;
				ResultEntry.Server = StorageInfo->ServerName;
				ResultEntry.Share  = StorageInfo->ShareName;
				Result.Storages.push_back(ResultEntry);
			}
			++StorageInfo;
		}
	}

	for (auto It : InfosToFree)
	{
		NetApiBufferFree(It);
	}

#endif	// UNSYNC_PLATFORM_WINDOWS

	return Result;
}

}  // namespace unsync
