// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncProxy.h"

namespace unsync {

enum class ESourceType : uint8 {
	Unknown,
	FileSystem,
	Server,
	ServerWithManifestId,
};

inline bool
IsFileSystemSource(ESourceType SourceType)
{
	return SourceType == ESourceType::FileSystem;
}

struct FSourcePath
{
	ESourceType Type = ESourceType::Unknown;
	std::string Location;  // utf-8

	bool operator==(const FSourcePath& Other) const { return Type == Other.Type && Location == Other.Location; }
};

// Returns a list of alternative DFS paths for a given root
struct FDfsStorageInfo
{
	std::wstring Server;
	std::wstring Share;

	bool IsValid() const { return !Server.empty() && !Share.empty(); }
};
struct FDfsMirrorInfo
{
	std::wstring				 Root;
	std::vector<FDfsStorageInfo> Storages;
};
FDfsMirrorInfo DfsEnumerate(const FPath& Root);

struct FDfsAlias
{
	FPath Source;
	FPath Target;
};

}  // namespace unsync
