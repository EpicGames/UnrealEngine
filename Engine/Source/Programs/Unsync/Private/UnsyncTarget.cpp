// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncTarget.h"
#include "UnsyncCompression.h"
#include "UnsyncCore.h"
#include "UnsyncError.h"
#include "UnsyncHashTable.h"
#include "UnsyncProgress.h"
#include "UnsyncProxy.h"
#include "UnsyncScavenger.h"
#include "UnsyncThread.h"
#include "UnsyncScheduler.h"
#include "UnsyncChunking.h"

namespace unsync {

void
DownloadBlocks(FProxyPool&					  ProxyPool,
			   const std::vector<FNeedBlock>& OriginalUniqueNeedBlocks,
			   const FBlockDownloadCallback&  CompletionCallback)
{
	const uint32		ParentThreadIndent	 = GLogIndent;
	const bool			bParentThreadVerbose = GLogVerbose;
	std::atomic<uint64> NumActiveLogThreads	 = {};

	THashSet<FHash128>			   DownloadedBlocks;
	std::vector<FNeedBlock>		   RemainingNeedBlocks;
	const std::vector<FNeedBlock>* NeedBlocks = &OriginalUniqueNeedBlocks;

	FAtomicError Error;

	const uint32 MaxAttempts = 30;
	for (uint32 Attempt = 0; Attempt <= MaxAttempts && !Error; ++Attempt)
	{
		if (!ProxyPool.IsValid())
		{
			UNSYNC_WARNING(L"FProxy connection is not valid.")
		}

		if (Attempt != 0)
		{
			uint64 NumRemainingBlocks = RemainingNeedBlocks.size();
			UNSYNC_WARNING(L"Failed to download %lld blocks from proxy. Retry attempt %d of %d ...",
						   NumRemainingBlocks,
						   Attempt,
						   MaxAttempts);

			SchedulerSleep(1000);
		}

		struct FDownloadBatch
		{
			uint64 Begin;
			uint64 End;
			uint64 SizeBytes;
		};
		std::vector<FDownloadBatch> Batches;
		Batches.push_back(FDownloadBatch{});

		const uint64 MaxBytesPerBatch = ProxyPool.RemoteDesc.Protocol == EProtocolFlavor::Jupiter ? 16_MB : 128_MB;

		for (uint64 BlockIndex = 0; BlockIndex < NeedBlocks->size() && !Error; ++BlockIndex)
		{
			const FNeedBlock& Block = (*NeedBlocks)[BlockIndex];

			if (Batches.back().SizeBytes + Block.Size < MaxBytesPerBatch)
			{
				Batches.back().End = BlockIndex + 1;
			}
			else
			{
				UNSYNC_ASSERT(Batches.back().End == BlockIndex);
				FDownloadBatch NewBatch = {};
				NewBatch.Begin			= BlockIndex;
				NewBatch.End			= BlockIndex + 1;
				NewBatch.SizeBytes		= 0;
				Batches.push_back(NewBatch);
			}

			Batches.back().SizeBytes += Block.Size;
		}

		UNSYNC_VERBOSE2(L"Download batches: %lld", Batches.size());

		FTaskGroup DownloadTasks = GScheduler->CreateTaskGroup(&GScheduler->NetworkSemaphore);
		std::mutex DownloadedBlocksMutex;

		for (FDownloadBatch Batch : Batches)
		{
			DownloadTasks.run(
				[NeedBlocks,
				 Batch,
				 ParentThreadIndent,
				 bParentThreadVerbose,
				 &DownloadedBlocksMutex,
				 &ProxyPool,
				 &DownloadedBlocks,
				 &CompletionCallback,
				 &NumActiveLogThreads,
				 &Error]
				{
					if (Error)
					{
						return;
					}

					FThreadElectScope  AllowVerbose(NumActiveLogThreads, bParentThreadVerbose);
					FLogVerbosityScope VerboseScope(AllowVerbose);
					FLogIndentScope	   IndentScope(ParentThreadIndent, true);

					auto Proxy = ProxyPool.Alloc();
					if (Proxy)
					{
						const FNeedBlock*	   Blocks		  = NeedBlocks->data();
						TArrayView<FNeedBlock> NeedBlocksView = {Blocks + Batch.Begin, Blocks + Batch.End};

						FDownloadResult DownloadResult =
							Proxy->Download(NeedBlocksView,
											[&DownloadedBlocksMutex, &CompletionCallback, &DownloadedBlocks](const FDownloadedBlock& Block,
																											 FHash128 BlockHash)
											{
												{
													std::lock_guard<std::mutex> LockGuard(DownloadedBlocksMutex);
													DownloadedBlocks.insert(BlockHash);
												}
												CompletionCallback(Block, BlockHash);
											});

						ProxyPool.Dealloc(std::move(Proxy));

						if (const FDownloadError* DownloadError = DownloadResult.TryError())
						{
							if (!DownloadError->CanRetry())
							{
								Error.Set(FError(*DownloadError));
							}

							if (DownloadError->RetryMode == EDownloadRetryMode::Disconnect)
							{
								ProxyPool.Invalidate();
							}
						}
					}
				});
		}

		DownloadTasks.wait();

		if (DownloadedBlocks.size() == OriginalUniqueNeedBlocks.size() || Error)
		{
			break;
		}

		RemainingNeedBlocks.clear();
		for (const FNeedBlock& Block : OriginalUniqueNeedBlocks)
		{
			if (DownloadedBlocks.find(Block.Hash.ToHash128()) == DownloadedBlocks.end())  // #wip-widehash
			{
				RemainingNeedBlocks.push_back(Block);
			}
		}
		NeedBlocks = &RemainingNeedBlocks;
	}
}

FBuildTargetResult
BuildTarget(FIOWriter& Output, FIOReader& Source, FIOReader& Base, const FNeedList& NeedList, const FBuildTargetParams& Params)
{
	UNSYNC_LOG_INDENT;

	FBuildTargetResult BuildResult;

	if (Params.SourceType == FBuildTargetParams::ESourceType::Server)
	{
		if (Params.ProxyPool == nullptr)
		{
			UNSYNC_ERROR(L"Connection pool must be provided when syncing from server");
			return BuildResult;
		}

		if (!Params.ProxyPool->IsValid())
		{
			UNSYNC_ERROR(L"Server connection cannot be established because connection pool is invalid");
			return BuildResult;
		}
	}

	auto TimeBegin = TimePointNow();
	double ElapsedTime = 0;

	const FNeedListSize			 SizeInfo		  = ComputeNeedListSize(NeedList);
	const EStrongHashAlgorithmID StrongHasher	  = Params.StrongHasher;
	FProxyPool*					 ProxyPool		  = Params.ProxyPool;
	FScavengeDatabase*			 ScavengeDatabase = Params.ScavengeDatabase;
	FBlockCache*				 BlockCache		  = Params.BlockCache;

	if (SizeInfo.TotalBytes != Output.GetSize())
	{
		UNSYNC_ERROR(L"Output size is %llu, but expected to be %llu", llu(Output.GetSize()), llu(SizeInfo.TotalBytes));
		return BuildResult;
	}

	UNSYNC_ASSERT(SizeInfo.TotalBytes == Output.GetSize());

	FAtomicError Error;

	std::atomic<bool> bBaseDataCopyTaskDone	  = false;
	std::atomic<bool> bSourceDataCopyTaskDone = false;
	std::atomic<bool> bWaitingForBaseData	  = false;

	struct FStats
	{
		std::atomic<uint64> WrittenBytesFromSource = {};
		std::atomic<uint64> WrittenBytesFromBase   = {};
		// Bytes reused on disk via block cloning (if available).
		// Subset of WrittenBytesFromBase.
		std::atomic<uint64> BytesUsedInSitu		   = {};
	};
	FStats Stats;

	const uint32 MaxWriteTasks = 64;

	FSchedulerSemaphore WriteSemaphore(*GScheduler, MaxWriteTasks); // // throttle writing tasks to avoid memory bloat
	FTaskGroup			WriteTasks = GScheduler->CreateTaskGroup(&WriteSemaphore);

	// Remember if parent thread has verbose logging and indentation
	const bool	 bAllowVerboseLog = GLogVerbose;
	const uint32 LogIndent		  = GLogIndent;

	// Copy the need list as it will be progressively filtered by various sources:
	// - block cache
	// - scevenge from external local files
	// - download from proxy
	// - read from source file
	// TODO: could use a bit array to indicate if the entry is valid instead of full copy
	std::vector<FNeedBlock> FilteredSourceNeedList;

	if (ScavengeDatabase)
	{
		THashSet<FHash128> ScavengedBlocks;

		FScavengedBuildTargetResult ScavengeResult =
			BuildTargetFromScavengedData(Output, NeedList.Source, *ScavengeDatabase, StrongHasher, ScavengedBlocks);

		// Treat scavenged data the same as base for stats purposes
		Stats.WrittenBytesFromBase += ScavengeResult.ScavengedBytes;
		Stats.BytesUsedInSitu += ScavengeResult.InSituBytes;

		FilteredSourceNeedList = NeedList.Source;
		auto FilterResult	   = std::remove_if(FilteredSourceNeedList.begin(),
											FilteredSourceNeedList.end(),
											[&ScavengedBlocks](const FNeedBlock& Block)
											{ return ScavengedBlocks.find(Block.Hash.ToHash128()) != ScavengedBlocks.end(); });
		FilteredSourceNeedList.erase(FilterResult, FilteredSourceNeedList.end());
	}
	else
	{
		FilteredSourceNeedList = NeedList.Source;
	}

	auto ProcessNeedList =
		[bAllowVerboseLog, LogIndent, &Output, &Error, &WriteTasks, &bWaitingForBaseData, &Stats, &Params, SizeInfo](
			FIOReader&					   DataProvider,
			const std::vector<FNeedBlock>& NeedBlocks,
			uint64						   TotalCopySize,
			EBlockListType				   ListType,
			std::atomic<bool>&			   bCompletionFlag)
	{
		FLogIndentScope IndentScope(LogIndent, true);
		uint64			ReadBytesTotal = 0;

		const wchar_t* ListName = ListType == EBlockListType::Source ? L"source" : L"base";

		if (!DataProvider.IsValid())
		{
			UNSYNC_ERROR(L"Failed to read blocks from %ls. Stream is invalid.", ListName);
			Error.Set(AppError(L"Failed to read blocks. Stream is invalid."));
			return;
		}

		if (ListType == EBlockListType::Source && Params.SourceType == FBuildTargetParams::ESourceType::File &&
			DataProvider.GetSize() != SizeInfo.TotalBytes)
		{
			UNSYNC_ERROR(L"File size is %llu, but expected to be %llu. File may have changed after manifest was generated.",
						 llu(DataProvider.GetSize()),
						 llu(SizeInfo.TotalBytes));
			Error.Set(AppError(L"Failed to read source blocks. Size mismatch."));
			return;
		}

		if (DataProvider.GetError())
		{
			UNSYNC_ERROR(L"Failed to read blocks from %ls. %hs", ListName, FormatSystemErrorMessage(DataProvider.GetError()).c_str());
			Error.Set(AppError(L"Failed to read blocks"));
			return;
		}

		UNSYNC_VERBOSE(L"Reading blocks from %ls", ListName);

		FLogProgressScope ProgressLogger(TotalCopySize, ELogProgressUnits::MB);

		FReadSchedule ReadSchedule = BuildReadSchedule(NeedBlocks);

		UNSYNC_LOG_INDENT;
		UNSYNC_VERBOSE(L"Using %d requests for %d total blocks (%.2f MB)",
					   (int)ReadSchedule.Requests.size(),
					   (int)NeedBlocks.size(),
					   SizeMb(TotalCopySize));

		static constexpr uint32 MaxActiveLargeRequests							= 2;
		uint64					ActiveLargeRequestSizes[MaxActiveLargeRequests] = {};

		std::unique_ptr<FAsyncReader> AsyncReader = DataProvider.CreateAsyncReader();

		while (!ReadSchedule.Requests.empty() && !Error)
		{
			uint64 BlockIndex = ~0ull;

			uint32 LargeRequestSlot = ~0u;
			for (uint32 I = 0; I < MaxActiveLargeRequests; ++I)
			{
				if (ActiveLargeRequestSizes[I] == 0)
				{
					LargeRequestSlot = I;
					break;
				}
			}

			if (LargeRequestSlot != ~0u)
			{
				BlockIndex = ReadSchedule.Requests.back();
				ReadSchedule.Requests.pop_back();
			}
			else
			{
				BlockIndex = ReadSchedule.Requests.front();
				ReadSchedule.Requests.pop_front();
			}

			const FCopyCommand& Block = ReadSchedule.Blocks[BlockIndex];

			if (LargeRequestSlot != ~0u)
			{
				ActiveLargeRequestSizes[LargeRequestSlot] = Block.Size;
			}

			UNSYNC_ASSERTF(Block.SourceOffset + Block.Size <= DataProvider.GetSize(),
						   L"Copy command is out of bounds. Offset %llu, size %llu ([%llu..%llu]), input size %llu.",
						   llu(Block.SourceOffset),
						   llu(Block.Size),
						   llu(Block.SourceOffset),
						   llu(Block.SourceOffset + Block.Size),
						   DataProvider.GetSize());

			uint64 ReadBytes = 0;

			auto ReadCallback = [&DataProvider, &ReadBytes, &Output, &WriteTasks, &Error, &Stats, Block, ListType](FIOBuffer CmdBuffer,
																												   uint64	  CmdOffset,
																												   uint64	  CmdReadSize,
																												   uint64	  CmdUserData)
			{
				if (CmdReadSize == uint64(-1))
				{
					UNSYNC_ERROR(L"Read failed: %hs", FormatSystemErrorMessage(DataProvider.GetError()).c_str());
					Error.Set(AppError(L"Failed to read blocks"));
					return;
				}

				WriteTasks.run(
					[Buffer = MakeShared(std::move(CmdBuffer)), CmdReadSize, Block, &DataProvider, &Output, &Error, &Stats, ListType]()
					{
						uint64		InSituBytes	 = 0;
						const int64 WrittenBytes = Output.WriteFrom(DataProvider, Block, Buffer->GetBufferView(), &InSituBytes);
						Stats.BytesUsedInSitu += InSituBytes;

						if (WrittenBytes < 0 || uint64(WrittenBytes) != CmdReadSize)
						{
							UNSYNC_FATAL(L"Expected to write %llu bytes, but written %llu", CmdReadSize, WrittenBytes);
							Error.Set(AppError(L"Failed to write output"));
							return;
						}

						if (ListType == EBlockListType::Source)
						{
							Stats.WrittenBytesFromSource += WrittenBytes;
						}
						else if (ListType == EBlockListType::Base)
						{
							Stats.WrittenBytesFromBase += WrittenBytes;
						}
						else
						{
							UNSYNC_FATAL(L"Unexpected block list type");
						}
					});

				ReadBytes = CmdReadSize;

				AddGlobalProgress(CmdReadSize, ListType);
			};

			AsyncReader->EnqueueRead(Block.SourceOffset, Block.Size, 0, ReadCallback);

			for (uint32 I = 0; I < MaxActiveLargeRequests; ++I)
			{
				if (ActiveLargeRequestSizes[I] == ReadBytes)
				{
					ActiveLargeRequestSizes[I] = 0;
					break;
				}
			}

			ReadBytesTotal += ReadBytes;

			FLogVerbosityScope VerbosityScope(GLogVerbose ||
											  (bAllowVerboseLog && (ListType == EBlockListType::Base && bWaitingForBaseData)));

			ProgressLogger.Add(ReadBytes);
		}

		AsyncReader->Flush();

		ProgressLogger.Complete();

		bCompletionFlag = true;
	};

	FTaskGroup BackgroundTasks = GScheduler->CreateTaskGroup();
	BackgroundTasks.run(
		[SizeInfo, ProcessNeedList, &NeedList, &Base, &bBaseDataCopyTaskDone]()
		{
			// Silence log messages on background task.
			// Logging will be enabled if we're waiting for base data.
			FLogVerbosityScope VerbosityScope(false);

			if (SizeInfo.BaseBytes)
			{
				ProcessNeedList(Base, NeedList.Base, SizeInfo.BaseBytes, EBlockListType::Base, bBaseDataCopyTaskDone);
			}
		});

	THashSet<FHash128> CachedBlocks;

	if (BlockCache)
	{
		UNSYNC_VERBOSE(L"Writing blocks from cache");
		UNSYNC_LOG_INDENT;

		uint64 BytesFromCache = 0;

		for (const FNeedBlock NeedBlock : FilteredSourceNeedList)
		{
			auto It = BlockCache->BlockMap.find(NeedBlock.Hash.ToHash128());
			if (It != BlockCache->BlockMap.end())
			{
				FBufferView BlockBuffer = It->second;

				const uint64 WrittenBytes = Output.Write(BlockBuffer.Data, NeedBlock.TargetOffset, BlockBuffer.Size);

				Stats.WrittenBytesFromSource += WrittenBytes;
				BytesFromCache += WrittenBytes;

				AddGlobalProgress(NeedBlock.Size, EBlockListType::Source);
			}
		}

		auto FilterResult = std::remove_if(FilteredSourceNeedList.begin(),
										   FilteredSourceNeedList.end(),
										   [BlockCache](const FNeedBlock& Block)
										   { return BlockCache->BlockMap.find(Block.Hash.ToHash128()) != BlockCache->BlockMap.end(); });


		// Add some virtual time cost to account for block cache creation
		if (BytesFromCache != 0)
		{
			const uint64 BlockCacheTotalSize = BlockCache->BlockData.Size();
			const double CacheUsedFraction	 = double(BytesFromCache) / double(BlockCacheTotalSize);
			const double EstimatedTimeCost	 = CacheUsedFraction * BlockCache->InitDuration.count();

			ElapsedTime += EstimatedTimeCost;
		}

		FilteredSourceNeedList.erase(FilterResult, FilteredSourceNeedList.end());
	}

	if (FilteredSourceNeedList.size() == 0)
	{
		// No more data is needed from source file
	}
	else if (ProxyPool && ProxyPool->IsValid())
	{
		UNSYNC_VERBOSE(L"Downloading blocks");
		UNSYNC_LOG_INDENT;

		struct FBlockWriteCmd
		{
			uint64 TargetOffset = 0;
			uint64 Size			= 0;
		};
		std::mutex								DownloadedBlocksMutex;
		THashSet<FHash128>						DownloadedBlocks;
		THashMap<FHash128, FBlockWriteCmd>		BlockWriteMap;
		THashMap<FHash128, std::vector<uint64>> BlockScatterMap;
		std::vector<FNeedBlock>					UniqueNeedList;

		uint64 EstimatedDownloadSize = 0;
		for (const FNeedBlock& Block : FilteredSourceNeedList)
		{
			FBlockWriteCmd Cmd;
			Cmd.TargetOffset  = Block.TargetOffset;
			Cmd.Size		  = Block.Size;
			auto InsertResult = BlockWriteMap.insert(std::make_pair(Block.Hash.ToHash128(), Cmd));	// #wip-widehash

			if (InsertResult.second)
			{
				UniqueNeedList.push_back(Block);
				EstimatedDownloadSize += Block.Size;
			}
			else
			{
				BlockScatterMap[Block.Hash.ToHash128()].push_back(Cmd.TargetOffset);  // #wip-widehash
			}
		}

		// limit how many decompression tasks can be queued up to avoid memory bloat
		const uint64		MaxConcurrentDecompressionTasks = 64;
		FSchedulerSemaphore DecompressionSemaphore(*GScheduler, MaxConcurrentDecompressionTasks);

		FTaskGroup DecompressTasks = GScheduler->CreateTaskGroup(&DecompressionSemaphore);

		FLogProgressScope DownloadProgressLogger(EstimatedDownloadSize, ELogProgressUnits::MB);

		const bool			bParentThreadVerbose = GLogVerbose;
		const uint32		ParentThreadIndent	 = GLogIndent;
		std::atomic<uint64> NumActiveLogThreads	 = {};
		std::atomic<uint64> NumHashMismatches	 = {};

		DownloadBlocks(
			*ProxyPool,
			UniqueNeedList,
			[&NumActiveLogThreads,
			 &DownloadProgressLogger,
			 &BlockWriteMap,
			 &DecompressionSemaphore,
			 &DecompressTasks,
			 &Output,
			 StrongHasher,
			 &BlockScatterMap,
			 &DownloadedBlocksMutex,
			 &DownloadedBlocks,
			 &Error,
			 &NumHashMismatches,
			 &ParentThreadIndent,
			 &bParentThreadVerbose,
			 &Stats](const FDownloadedBlock& Block, FHash128 BlockHash)
			{
				FLogIndentScope IndentScope(ParentThreadIndent, true);

				{
					FThreadElectScope  AllowVerbose(NumActiveLogThreads, bParentThreadVerbose);
					FLogVerbosityScope VerboseScope(AllowVerbose);
					DownloadProgressLogger.Add(Block.DecompressedSize);
				}

				auto WriteIt = BlockWriteMap.find(BlockHash);
				if (WriteIt != BlockWriteMap.end())
				{
					FBlockWriteCmd Cmd = WriteIt->second;

					// TODO: avoid this copy by storing IOBuffer in DownloadedBlock
					const bool	 bCompressed	= Block.bCompressed;
					const uint64 DownloadedSize = bCompressed ? Block.CompressedSize : Block.DecompressedSize;
					FIOBuffer	 DownloadedData = FIOBuffer::Alloc(DownloadedSize, L"downloaded_data");

					memcpy(DownloadedData.GetData(), Block.Data, DownloadedSize);

					DecompressTasks.run(
						[&Output,
						 BlockHashUnaligned = BlockHash,
						 Cmd,
						 DownloadedData	  = std::make_shared<FIOBuffer>(std::move(DownloadedData)),
						 DecompressedSize = Block.DecompressedSize,
						 bCompressed,
						 ParentThreadIndent,
						 StrongHasher,
						 &BlockScatterMap,
						 &DownloadedBlocksMutex,
						 &DownloadedBlocks,
						 &Error,
						 &NumHashMismatches,
						 &Stats]()
						{
							FLogIndentScope IndentScope(ParentThreadIndent, true);

							// captured objects don't respect large alignment requirements
							FHash128 BlockHash = BlockHashUnaligned;

							bool	  bOk = false;
							FIOBuffer DecompressedData;
							if (DecompressedSize == Cmd.Size)
							{
								if (bCompressed)
								{
									DecompressedData = FIOBuffer::Alloc(Cmd.Size, L"decompressed_data");

									bOk = Decompress(DownloadedData->GetData(),
													 DownloadedData->GetSize(),
													 DecompressedData.GetData(),
													 Cmd.Size);
								}
								else
								{
									DecompressedData = std::move(*DownloadedData);

									bOk = true;
								}

								if (bOk)
								{
									FHash128 ActualBlockHash = ComputeHash(DecompressedData.GetData(), Cmd.Size, StrongHasher).ToHash128();

									bOk = ActualBlockHash == BlockHash;

									if (bOk)
									{
										uint64 WrittenBytes = Output.Write(DecompressedData.GetData(), Cmd.TargetOffset, Cmd.Size);

										Stats.WrittenBytesFromSource += WrittenBytes;

										AddGlobalProgress(Cmd.Size, EBlockListType::Source);

										if (WrittenBytes != Cmd.Size)
										{
											bOk = false;
											UNSYNC_FATAL(L"Expected to write %llu bytes, but written %llu", Cmd.Size, WrittenBytes);
											Error.Set(AppError(L"Failed to write output"));
										}
									}
									else
									{
										NumHashMismatches++;
									}
								}
								else
								{
									UNSYNC_FATAL(L"Failed to decompress downloaded block");
									Error.Set(AppError(L"Failed to decompress downloaded block"));
								}
							}

							if (bOk)
							{
								auto ScatterIt = BlockScatterMap.find(BlockHash);
								if (ScatterIt != BlockScatterMap.end())
								{
									const std::vector<uint64>& ScatterList = ScatterIt->second;
									for (uint64 ScatterOffset : ScatterList)
									{
										uint64 WrittenBytes = Output.Write(DecompressedData.GetData(), ScatterOffset, Cmd.Size);

										Stats.WrittenBytesFromSource += WrittenBytes;

										if (WrittenBytes != Cmd.Size)
										{
											UNSYNC_FATAL(L"Expected to write %llu bytes, but written %llu", Cmd.Size, WrittenBytes);
											Error.Set(AppError(L"Failed to write output"));
										}
									}

									AddGlobalProgress(Cmd.Size * ScatterList.size(), EBlockListType::Source);
								}
							}

							if (bOk)
							{
								std::lock_guard<std::mutex> LockGuard(DownloadedBlocksMutex);
								DownloadedBlocks.insert(BlockHash);
							}
						});
				}
			});

		DecompressTasks.wait();

		DownloadProgressLogger.Complete();

		if (NumHashMismatches.load() != 0)
		{
			UNSYNC_WARNING(L"Found block hash mismatches while downloading data from proxy. Mismatching blocks: %llu.",
						   NumHashMismatches.load());
		}

		{
			auto FilterResult = std::remove_if(FilteredSourceNeedList.begin(),
											   FilteredSourceNeedList.end(),
											   [&DownloadedBlocks](const FNeedBlock& Block)
											   { return DownloadedBlocks.find(Block.Hash.ToHash128()) != DownloadedBlocks.end(); });
			FilteredSourceNeedList.erase(FilterResult, FilteredSourceNeedList.end());
		}

		if (FilteredSourceNeedList.empty())
		{
			bSourceDataCopyTaskDone = true;
		}
		else if (Params.SourceType == FBuildTargetParams::ESourceType::Server)
		{
			Error.Set(AppError(L"Could not download all required data from the server"));
			bSourceDataCopyTaskDone = true;
		}
		else
		{
			uint64 RemainingBytes = ComputeSize(FilteredSourceNeedList);

			UNSYNC_VERBOSE(L"Did not receive %d blocks from proxy. Must read %.2f MB from source.",
						   FilteredSourceNeedList.size(),
						   SizeMb(RemainingBytes));

			UNSYNC_LOG_INDENT;

			ProcessNeedList(Source, FilteredSourceNeedList, RemainingBytes, EBlockListType::Source, bSourceDataCopyTaskDone);
		}
	}
	else if (SizeInfo.SourceBytes)
	{
		ProcessNeedList(Source, FilteredSourceNeedList, SizeInfo.SourceBytes, EBlockListType::Source, bSourceDataCopyTaskDone);
	}

	if (!bBaseDataCopyTaskDone)
	{
		UNSYNC_VERBOSE(L"Reading blocks from base");
	}

	bWaitingForBaseData = true;
	BackgroundTasks.wait();
	WriteTasks.wait();

	ElapsedTime += DurationSec(TimeBegin, TimePointNow());
	UNSYNC_VERBOSE(L"Done in %.3f sec (%.3f MB / sec)", ElapsedTime, SizeMb(double(SizeInfo.TotalBytes) / ElapsedTime));
	if (Stats.BytesUsedInSitu > 0)
	{
		UNSYNC_VERBOSE2("%.2f MB reused in situ", SizeMb(Stats.BytesUsedInSitu));
	}

	if (GLogVerbose)
	{
		LogGlobalProgress();
	}

	BuildResult.bSuccess	= !Error;
	BuildResult.BaseBytes	= Stats.WrittenBytesFromBase;
	BuildResult.SourceBytes = Stats.WrittenBytesFromSource;
	BuildResult.InSituBytes = Stats.BytesUsedInSitu;

	return BuildResult;
}

FBuffer
BuildTargetBuffer(FIOReader& SourceProvider, FIOReader& BaseProvider, const FNeedList& NeedList, const FBuildTargetParams& Params)
{
	FBuffer				Result;
	const FNeedListSize SizeInfo = ComputeNeedListSize(NeedList);
	Result.Resize(SizeInfo.TotalBytes);
	FMemReaderWriter ResultWriter(Result.Data(), Result.Size());
	BuildTarget(ResultWriter, SourceProvider, BaseProvider, NeedList, Params);
	return Result;
}

FBuffer
BuildTargetBuffer(const uint8*				SourceData,
				  uint64					SourceSize,
				  const uint8*				BaseData,
				  uint64					BaseSize,
				  const FNeedList&			NeedList,
				  const FBuildTargetParams& Params)
{
	FMemReader SourceReader(SourceData, SourceSize);
	FMemReader BaseReader(BaseData, BaseSize);
	return BuildTargetBuffer(SourceReader, BaseReader, NeedList, Params);
}

struct FMemStream
{
	FMemStream(const uint8* InData, uint64 InDataSize) : Data(InData), DataRw(nullptr), SIZE(InDataSize), Offset(0) {}

	FMemStream(uint8* InData, uint64 InDataSize) : Data(InData), DataRw(InData), SIZE(InDataSize), Offset(0) {}

	bool Read(void* Dest, uint64 ReadSize)
	{
		if (Offset + ReadSize <= SIZE)
		{
			uint8* DestBytes = reinterpret_cast<uint8*>(Dest);
			memcpy(DestBytes, Data + Offset, ReadSize);
			Offset += ReadSize;
			return true;
		}
		else
		{
			return false;
		}
	}

	bool Write(const void* Source, uint64 WriteSize)
	{
		if (Offset + WriteSize <= SIZE)
		{
			const uint8* SourceBytes = reinterpret_cast<const uint8*>(Source);
			memcpy(DataRw + Offset, SourceBytes, WriteSize);
			Offset += WriteSize;
			return true;
		}
		else
		{
			return false;
		}
	}

	const uint8* Data;
	uint8*		 DataRw;
	const uint64 SIZE;
	uint64		 Offset;
};

template<typename BlockType>
static bool
BlocksEqual(const std::vector<BlockType>& A, std::vector<BlockType>& B)
{
	if (A.size() != B.size())
	{
		return false;
	}

	for (uint64 I = 0; I < A.size(); ++I)
	{
		if (A[I].HashStrong != B[I].HashStrong || A[I].HashWeak != B[I].HashWeak || A[I].Size != B[I].Size || A[I].Offset != B[I].Offset)
		{
			return false;
		}
	}

	return true;
}

FBuffer
BuildTargetWithPatch(const uint8* PatchData, uint64 PatchSize, const uint8* BaseData, uint64 BaseDataSize)
{
	const EChunkingAlgorithmID ChunkingMode = EChunkingAlgorithmID::FixedBlocks;

	FBuffer Result;

	FBuffer	   DecompressedPatch = Decompress(PatchData, PatchSize);
	FMemStream Stream(DecompressedPatch.Data(), DecompressedPatch.Size());

	FPatchHeader Header;
	Stream.Read(&Header, sizeof(Header));

	if (Header.Magic != FPatchHeader::MAGIC)
	{
		UNSYNC_ERROR(L"Patch file header mismatch. Expected %llX, got %llX", (long long)FPatchHeader::MAGIC, (long long)Header.Magic);
		return Result;
	}

	if (Header.Version != FPatchHeader::VERSION)
	{
		UNSYNC_ERROR(L"Patch file version mismatch. Expected %lld, got %lld", (long long)FPatchHeader::VERSION, (long long)Header.Version);
		return Result;
	}

	FHash128 ExpectedHeaderHash = HashBlake3Bytes<FHash128>(Stream.Data, Stream.Offset);
	FHash128 HeaderHash;
	Stream.Read(&HeaderHash, sizeof(HeaderHash));
	if (HeaderHash != ExpectedHeaderHash)
	{
		UNSYNC_ERROR(L"Patch header hash mismatch");
		return Result;
	}

	FGenericBlockArray SourceValidationExpected(Header.NumSourceValidationBlocks);
	FGenericBlockArray BaseValidationExpected(Header.NumBaseValidationBlocks);

	Stream.Read(&SourceValidationExpected[0], SourceValidationExpected.size() * sizeof(SourceValidationExpected[0]));
	Stream.Read(&BaseValidationExpected[0], BaseValidationExpected.size() * sizeof(BaseValidationExpected[0]));

	FGenericBlockArray BaseValidation;

	{
		FLogVerbosityScope VerbosityScope(false);
		FAlgorithmOptions  Algorithm;
		Algorithm.ChunkingAlgorithmId	= ChunkingMode;
		Algorithm.WeakHashAlgorithmId	= Header.WeakHashAlgorithmId;
		Algorithm.StrongHashAlgorithmId = Header.StrongHashAlgorithmId;

		BaseValidation = ComputeBlocks(BaseData, BaseDataSize, FPatchHeader::VALIDATION_BLOCK_SIZE, Algorithm);
	}

	if (!BlocksEqual(BaseValidation, BaseValidationExpected))
	{
		UNSYNC_ERROR(L"Base file mismatch");
		return Result;
	}

	FPatchCommandList PatchCommands;
	PatchCommands.Source.resize(Header.NumSourceBlocks);
	PatchCommands.Base.resize(Header.NumBaseBlocks);

	Stream.Read(&PatchCommands.Source[0], Header.NumSourceBlocks * sizeof(PatchCommands.Source[0]));
	Stream.Read(&PatchCommands.Base[0], Header.NumBaseBlocks * sizeof(PatchCommands.Base[0]));

	FHash128 ExpectedBlockHash = HashBlake3Bytes<FHash128>(Stream.Data, Stream.Offset);
	FHash128 BlockHash;
	Stream.Read(&BlockHash, sizeof(BlockHash));
	if (BlockHash != ExpectedBlockHash)
	{
		UNSYNC_ERROR(L"Patch block hash mismatch");
		return Result;
	}

	const uint8* SourceData		= Stream.Data + Stream.Offset;
	uint64		 SourceDataSize = Stream.SIZE - Stream.Offset;

	// TODO: generate the proper source need list from the start
	uint64 SourceOffset = 0;
	for (FCopyCommand& Block : PatchCommands.Source)
	{
		Block.SourceOffset = SourceOffset;
		SourceOffset += Block.Size;
	}

	FNeedList NeedList;
	NeedList.Base.reserve(PatchCommands.Base.size());
	NeedList.Source.reserve(PatchCommands.Source.size());
	NeedList.Sequence.reserve(PatchCommands.Sequence.size());

	for (const FCopyCommand& Cmd : PatchCommands.Base)
	{
		FNeedBlock Block;
		static_cast<FCopyCommand&>(Block) = Cmd;
		NeedList.Base.push_back(Block);
	}

	for (const FCopyCommand& Cmd : PatchCommands.Source)
	{
		FNeedBlock Block;
		static_cast<FCopyCommand&>(Block) = Cmd;
		NeedList.Source.push_back(Block);
	}

	for (const FHash128& Hash : PatchCommands.Sequence)
	{
		NeedList.Sequence.push_back(Hash);
	}

	FBuildTargetParams BuildParams;
	BuildParams.StrongHasher = Header.StrongHashAlgorithmId;
	BuildParams.SourceType	 = FBuildTargetParams::ESourceType::Patch;

	Result = BuildTargetBuffer(SourceData, SourceDataSize, BaseData, BaseDataSize, NeedList, BuildParams);

	FGenericBlockArray SourceValidation;

	{
		FLogVerbosityScope VerbosityScope(false);

		FAlgorithmOptions Algorithm;
		Algorithm.ChunkingAlgorithmId	= ChunkingMode;
		Algorithm.WeakHashAlgorithmId	= Header.WeakHashAlgorithmId;
		Algorithm.StrongHashAlgorithmId = Header.StrongHashAlgorithmId;

		SourceValidation = ComputeBlocks(Result.Data(), Result.Size(), FPatchHeader::VALIDATION_BLOCK_SIZE, Algorithm);
	}

	if (!BlocksEqual(SourceValidation, SourceValidationExpected))
	{
		UNSYNC_ERROR(L"Patched size file mismatch");
		Result.Clear();
		return Result;
	}

	return Result;
}

}  // namespace unsync
