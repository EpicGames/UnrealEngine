// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncBuffer.h"
#include "UnsyncCommon.h"
#include "UnsyncFile.h"

namespace unsync {

struct FNeedList;
class FProxyPool;
class FBlockCache;
class FScavengeDatabase;

struct FBuildTargetResult
{
	bool   bSuccess	   = false;
	uint64 SourceBytes = 0;
	uint64 BaseBytes   = 0;
	uint64 InSituBytes = 0;
};

struct FBuildTargetParams
{
	EStrongHashAlgorithmID StrongHasher;
	FProxyPool*			   ProxyPool		= nullptr;
	FBlockCache*		   BlockCache		= nullptr;
	FScavengeDatabase*	   ScavengeDatabase = nullptr;

	enum class ESourceType {
		File,
		Patch,
		Server,
	};

	ESourceType SourceType = ESourceType::File;
};

FBuildTargetResult BuildTarget(FIOWriter&				 Result,
							   FIOReader&				 Source,
							   FIOReader&				 Base,
							   const FNeedList&			 NeedList,
							   const FBuildTargetParams& Params);

FBuffer BuildTargetBuffer(FIOReader& SourceProvider, FIOReader& BaseProvider, const FNeedList& NeedList, const FBuildTargetParams& Params);

FBuffer BuildTargetBuffer(const uint8*				SourceData,
						  uint64					SourceSize,
						  const uint8*				BaseData,
						  uint64					BaseSize,
						  const FNeedList&			NeedList,
						  const FBuildTargetParams& Params);

FBuffer BuildTargetWithPatch(const uint8* PatchData, uint64 PatchSize, const uint8* BaseData, uint64 BaseSize);

}  // namespace unsync
