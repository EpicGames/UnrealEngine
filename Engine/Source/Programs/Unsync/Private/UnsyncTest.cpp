// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncTest.h"
#include "UnsyncChunking.h"
#include "UnsyncCore.h"
#include "UnsyncDiff.h"
#include "UnsyncFile.h"
#include "UnsyncHash.h"
#include "UnsyncScan.h"
#include "UnsyncTarget.h"
#include "UnsyncTest.h"
#include "UnsyncThread.h"
#include "UnsyncUtil.h"

UNSYNC_THIRD_PARTY_INCLUDES_START
#include <md5-sse2.h>
#include <chrono>
#include <filesystem>
#include <unordered_map>
UNSYNC_THIRD_PARTY_INCLUDES_END

namespace unsync {

namespace test {

class FTestFile : public FNativeFile
{
public:
	explicit FTestFile(const FPath& Filename, EFileMode Mode = EFileMode::ReadOnly, uint64 InSize = 0)
		: FNativeFile(Filename, Mode, InSize)
	{
#if UNSYNC_PLATFORM_WINDOWS
		if (DWORD FileAttributes = GetFileAttributesW(Filename.c_str()); FileAttributes != INVALID_FILE_ATTRIBUTES)
		{
			FileAttributes |= FILE_ATTRIBUTE_TEMPORARY;
			SetFileAttributesW(Filename.c_str(), FileAttributes);
		}
#endif
	}

	virtual ~FTestFile() override {
		if (!Filename.empty())
		{
			Close();

			std::error_code Error{};
			std::filesystem::remove(Filename.native(), Error);
			if (Error)
			{
				UNSYNC_ERROR("Failed to remove test file %s: %s", Filename.c_str(), Error.message().c_str());
			}
		}
	}
};

} // namespace test

static bool
VerifyRandomBytes(const uint8* Data, uint64 Size, uint32 Seed)
{
	for (uint64 I = 0; I < Size; ++I)
	{
		uint8 Expected = Xorshift32(Seed) & 0xFF;
		if (Data[I] != Expected)
		{
			UNSYNC_ERROR(L"Expected byte %llu to be %d, but found %d.", I, Expected, Data[I]);
			return false;
		}
	}

	return true;
}

static bool
TestRollingSum()
{
	UNSYNC_LOG(L"TestRollingSum()");

	// sanity check
	{
		uint8			 Data[] = {0, 1, 2, 3, 4, 5, 6, 7, 128, 255, 123, 19, 84};
		FRollingChecksum Hash;
		Hash.Update(Data, sizeof(Data));

		{
			UNSYNC_ASSERT(Hash.A == 1040);
			UNSYNC_ASSERT(Hash.B == 5196);

			uint32 HashValue = Hash.Get();
			UNSYNC_ASSERT(HashValue == 340526096);
		}

		Hash.Sub(0);
		Hash.Sub(1);

		{
			uint32 HashValue = Hash.Get();
			UNSYNC_ASSERT(HashValue == 288949201);
		}

		Hash.Add(121);

		{
			uint32 HashValue = Hash.Get();
			UNSYNC_ASSERT(HashValue == 362939497);
		}

		Hash.Sub(255);

		{
			uint32 HashValue = Hash.Get();
			UNSYNC_ASSERT(HashValue == 138019659);
		}

		for (uint32 I = 0; I < 123512; ++I)
		{
			Hash.Add(uint8(I * 123));
		}

		{
			uint32 HashValue = Hash.Get();
			UNSYNC_ASSERT(HashValue == 1223342943);
		}

		for (uint32 I = 0; I < 123512; ++I)
		{
			Hash.Sub(uint8(I * 21));
		}

		{
			uint32 HashValue = Hash.Get();
			UNSYNC_ASSERT(HashValue == 1339426083);
		}
	}

	// basic rolling sum
	{
		uint64 WindowSize = 64;
		uint32 BlockSize  = 65536;

		FBuffer Data;
		for (uint32 I = 0; I < BlockSize * 3; ++I)
		{
			std::hash<uint32> Hasher;
			Data.PushBack(uint8(Hasher(I)));
		}

		FRollingChecksum Hash1;
		Hash1.Update(&Data[0], WindowSize);

		for (uint32 I = 1; I < BlockSize; ++I)
		{
			Hash1.Sub(Data[I - 1]);
			Hash1.Add(Data[I + WindowSize - 1]);

			FRollingChecksum Hash2;
			Hash2.Update(&Data[I], WindowSize);

			UNSYNC_ASSERT(Hash1.Get() == Hash2.Get());
		}
	}

	// basic serial test
	{
		auto DoTest = [](uint64 BlockSize, uint64 DataSize) {
			FBuffer Data(DataSize);
			FillRandomBytes(&Data[0], Data.Size(), 1234);
			const uint8* DataBegin = Data.Data();

			std::vector<uint32> HashReference(Data.Size());
			std::vector<uint32> HashRolling;
			HashRolling.reserve(Data.Size());

			for (uint64 I = 0; I < Data.Size(); ++I)
			{
				uint64			 ThisBlockSize = std::min(BlockSize, Data.Size() - I);
				FRollingChecksum Hash;
				Hash.Update(&Data[I], ThisBlockSize);
				HashReference[I] = Hash.Get();
			}

			auto ScanFn = [DataBegin, &HashReference, &HashRolling](const uint8* WindowBegin, const uint8* WindowEnd, uint32 WindowHash) {
				uint64 Idx = WindowBegin - DataBegin;
				UNSYNC_ASSERT(HashReference[Idx] == WindowHash);
				HashRolling.push_back(WindowHash);
				return false;
			};

			HashScan<FRollingChecksum>(Data.Data(), Data.Size(), BlockSize, ScanFn);

			UNSYNC_ASSERT(HashReference.size() == HashRolling.size());

			for (uint32 I = 0; I < DataSize; ++I)
			{
				UNSYNC_ASSERT(HashReference[I] == HashRolling[I]);
			}
		};

		DoTest(4, 4);
		DoTest(4, 1);
		DoTest(4, 5);
		DoTest(4, 7);
		DoTest(4, 128);
		DoTest(4, 127);
		DoTest(4, 128 * 1024);
		DoTest(1024, 128 * 1024);
	}

	// simulated parallel test
	{
		auto DoTest = [](uint64 BlockSize, uint64 DataSize) {
			FBuffer Data(DataSize);
			FillRandomBytes(&Data[0], Data.Size(), 1234);
			const uint8*		DataBegin = Data.Data();
			const uint8*		DataEnd	  = DataBegin + Data.Size();
			std::vector<uint32> HashReference(Data.Size());
			std::vector<uint32> HashRolling;
			HashRolling.reserve(Data.Size());

			for (uint64 I = 0; I < Data.Size(); ++I)
			{
				uint64			 ThisBlockSize = std::min(BlockSize, Data.Size() - I);
				FRollingChecksum Hash;
				Hash.Update(&Data[I], ThisBlockSize);
				HashReference[I] = Hash.Get();
			}

			struct Task
			{
				const uint8* Begin = nullptr;
				const uint8* End   = nullptr;
			};

			std::vector<Task> Tasks;
			uint64			  BytesPerTask = std::max<uint64>(BlockSize, 8);
			uint64			  NumTasks	   = DivUp(DataSize, BytesPerTask);

			for (uint64 I = 0; I < NumTasks; ++I)
			{
				Task Task;

				Task.Begin = DataBegin + I * BytesPerTask;
				Task.End   = Task.Begin + BytesPerTask;

				Task.Begin = Task.Begin - (BlockSize - 1);

				Task.Begin = std::max(Task.Begin, DataBegin);
				Task.End   = std::min(Task.End, DataEnd);

				Tasks.push_back(Task);
			}

			uint64 ExpectedIdx = 0;
			for (uint64 TaskIdx = 0; TaskIdx < Tasks.size(); ++TaskIdx)
			{
				const Task& Task	 = Tasks[TaskIdx];
				uint64		TaskSize = Task.End - Task.Begin;

				auto ScanFn = [&HashReference, &HashRolling, &Task, DataBegin, DataEnd, &ExpectedIdx](
								  const uint8* WindowBegin,
								  const uint8* WindowEnd,
								  uint32	   WindowHash) {
					uint64 Idx = WindowBegin - DataBegin;
					UNSYNC_ASSERT(Idx == ExpectedIdx);
					UNSYNC_ASSERT(HashReference[Idx] == WindowHash);
					HashRolling.push_back(WindowHash);
					ExpectedIdx++;
					return WindowEnd == Task.End && Task.End != DataEnd;
				};

				HashScan<FRollingChecksum>(Task.Begin, TaskSize, BlockSize, ScanFn);
			}

			UNSYNC_ASSERT(HashReference.size() == HashRolling.size());

			for (uint32 I = 0; I < DataSize; ++I)
			{
				UNSYNC_ASSERT(HashReference[I] == HashRolling[I]);
			}
		};

		DoTest(4, 4);
		DoTest(4, 1);
		DoTest(4, 5);
		DoTest(4, 7);
		DoTest(4, 8);
		DoTest(4, 9);
		DoTest(4, 16);
		DoTest(4, 31);
		DoTest(4, 32);
		DoTest(4, 33);
		DoTest(4, 128);
		DoTest(4, 127);
		DoTest(4, 128 * 1024);
		DoTest(1024, 128 * 1024);
	}

	return true;
}

static bool
TestSync(EWeakHashAlgorithmID WeakHasher, EStrongHashAlgorithmID StrongHasher)
{
	// TODO: test different chunk modes
	const EChunkingAlgorithmID ChunkingMode = EChunkingAlgorithmID::FixedBlocks;

	FAlgorithmOptions Algorithm;
	Algorithm.ChunkingAlgorithmId	= ChunkingMode;
	Algorithm.WeakHashAlgorithmId	= WeakHasher;
	Algorithm.StrongHashAlgorithmId = StrongHasher;
	
	FBuildTargetParams BuildParams;
	BuildParams.StrongHasher = StrongHasher;

	UNSYNC_LOG(L"TestSync(%hs, %hs)", ToString(WeakHasher), ToString(StrongHasher));

	{
		uint32	BlockSize	 = 4;
		FBuffer Source		 = {0, 0, 0, 0, 1, 1, 1, 1};
		FBuffer Base		 = {0, 0, 0, 0, 1, 1, 1, 1};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		UNSYNC_ASSERT(SourceBlocks.size() == 2);
		UNSYNC_ASSERT(SourceBlocks[0].Offset == 0);
		UNSYNC_ASSERT(SourceBlocks[1].Offset == 4);

		auto NeedList = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		UNSYNC_ASSERT(NeedList.Source.size() == 0);
		UNSYNC_ASSERT(NeedList.Base.size() == 2);
		UNSYNC_ASSERT(IsSynchronized(NeedList, SourceBlocks));
	}

	{
		uint32	BlockSize	 = 4;
		FBuffer Source		 = {0, 0, 0, 0, 1, 1, 1, 1};
		FBuffer Base		 = {1, 2, 3, 4, 1, 1, 1, 1};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		auto	NeedList	 = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		UNSYNC_ASSERT(NeedList.Source.size() == 1);
		UNSYNC_ASSERT(NeedList.Base.size() == 1);
	}

	{
		uint32	BlockSize	 = 4;
		FBuffer Source		 = {0, 0, 0, 0, 1, 1, 1, 1};
		FBuffer Base		 = {1, 1, 1, 1, 0, 0, 0, 0};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		auto	NeedBlocks	 = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		UNSYNC_ASSERT(NeedBlocks.Source.size() == 0);
		UNSYNC_ASSERT(NeedBlocks.Base.size() == 2);
		UNSYNC_ASSERT(!IsSynchronized(NeedBlocks, SourceBlocks));

		FMemReader SourceReader(Source);
		FMemReader BaseReader(Base);
		auto	   Target = BuildTargetBuffer(SourceReader, BaseReader, NeedBlocks, BuildParams);

		UNSYNC_ASSERT(Target.Size() == Source.Size());
		UNSYNC_ASSERT(!memcmp(Target.Data(), Source.Data(), Source.Size()));
	}

	{
		uint32	BlockSize	 = 2;
		FBuffer Source		 = {0, 0, 0, 0, 1, 1, 1, 1};
		FBuffer Base		 = {0, 0, 1, 1, 0, 0, 1, 1};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		auto	NeedBlocks	 = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		UNSYNC_ASSERT(!IsSynchronized(NeedBlocks, SourceBlocks));

		auto Target = BuildTargetBuffer(Source.Data(), Source.Size(), Base.Data(), Base.Size(), NeedBlocks, BuildParams);

		UNSYNC_ASSERT(Target.Size() == Source.Size());
		UNSYNC_ASSERT(!memcmp(Target.Data(), Source.Data(), Source.Size()));
	}

	{
		uint32	BlockSize	 = 2;
		FBuffer Source		 = {0, 0, 0, 0, 1, 1, 1, 1};
		FBuffer Base		 = {0, 0, 1, 1};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		auto	NeedBlocks	 = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		UNSYNC_ASSERT(!IsSynchronized(NeedBlocks, SourceBlocks));

		auto Target = BuildTargetBuffer(Source.Data(), Source.Size(), Base.Data(), Base.Size(), NeedBlocks, BuildParams);

		UNSYNC_ASSERT(Target.Size() == Source.Size());
		UNSYNC_ASSERT(!memcmp(Target.Data(), Source.Data(), Source.Size()));
	}

	{
		uint32	BlockSize	 = 2;
		FBuffer Source		 = {0, 0, 0, 0, 1, 1, 1, 1};
		FBuffer Base		 = {0, 0, 0, 0, 1, 1, 1, 1, 0, 0, 0, 0};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		auto	NeedBlocks	 = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		UNSYNC_ASSERT(!IsSynchronized(NeedBlocks, SourceBlocks));

		auto Target = BuildTargetBuffer(Source.Data(), Source.Size(), Base.Data(), Base.Size(), NeedBlocks, BuildParams);

		UNSYNC_ASSERT(Target.Size() == Source.Size());
		UNSYNC_ASSERT(!memcmp(Target.Data(), Source.Data(), Source.Size()));
	}

	{
		uint32	BlockSize	 = 2;
		FBuffer Source		 = {0, 0, 0, 0, 1, 1, 1, 1};
		FBuffer Base		 = {};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		auto	NeedBlocks	 = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		UNSYNC_ASSERT(!IsSynchronized(NeedBlocks, SourceBlocks));

		auto Target = BuildTargetBuffer(Source.Data(), Source.Size(), Base.Data(), Base.Size(), NeedBlocks, BuildParams);

		UNSYNC_ASSERT(Target.Size() == Source.Size());
		UNSYNC_ASSERT(!memcmp(Target.Data(), Source.Data(), Source.Size()));
	}

	{
		uint32	BlockSize	 = 32;
		FBuffer Source		 = {1, 2, 3, 4, 5, 6, 7, 8};
		FBuffer Base		 = {};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		auto	NeedBlocks	 = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		UNSYNC_ASSERT(!IsSynchronized(NeedBlocks, SourceBlocks));

		auto Target = BuildTargetBuffer(Source.Data(), Source.Size(), Base.Data(), Base.Size(), NeedBlocks, BuildParams);

		UNSYNC_ASSERT(Target.Size() == Source.Size());
		UNSYNC_ASSERT(!memcmp(Target.Data(), Source.Data(), Source.Size()));
	}

	{
		uint32	BlockSize	 = 32;
		FBuffer Source		 = {1, 2, 3, 4, 5, 6, 7, 8};
		FBuffer Base		 = {1, 2, 3, 4, 5, 6, 7, 8};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		auto	NeedBlocks	 = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		UNSYNC_ASSERT(IsSynchronized(NeedBlocks, SourceBlocks));

		auto Target = BuildTargetBuffer(Source.Data(), Source.Size(), Base.Data(), Base.Size(), NeedBlocks, BuildParams);

		UNSYNC_ASSERT(Target.Size() == Source.Size());
		UNSYNC_ASSERT(!memcmp(Target.Data(), Source.Data(), Source.Size()));
	}

	{
		uint32	BlockSize	 = 2;
		FBuffer Source		 = {};
		FBuffer Base		 = {1, 2, 4, 4, 5, 6, 7, 8};
		auto	SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
		auto	NeedBlocks	 = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

		auto Target = BuildTargetBuffer(Source.Data(), Source.Size(), Base.Data(), Base.Size(), NeedBlocks, BuildParams);

		UNSYNC_ASSERT(Target.Size() == Source.Size());
		UNSYNC_ASSERT(!memcmp(Target.Data(), Source.Data(), Source.Size()));
	}

	return true;
}

void
TestBuildTarget(EWeakHashAlgorithmID WeakHasher, EStrongHashAlgorithmID StrongHasher)
{
	// TODO: test different chunk modes
	FAlgorithmOptions Algorithm;
	Algorithm.ChunkingAlgorithmId	= EChunkingAlgorithmID::FixedBlocks;
	Algorithm.WeakHashAlgorithmId	= WeakHasher;
	Algorithm.StrongHashAlgorithmId = StrongHasher;

	UNSYNC_LOG(L"TestBuildTarget(%hs, %hs)", ToString(WeakHasher), ToString(StrongHasher));
	uint32	BlockSize = 4;
	FBuffer Source	  = {0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 99, 99, 99, 99};
	FBuffer Base	  = {16, 17, 18, 19, 20, 21, 99, 99, 99, 99, 26, 27, 28, 29, 30, 31};

	auto SourceBlocks = ComputeBlocks(Source.Data(), Source.Size(), BlockSize, Algorithm);
	auto NeedBlocks	  = DiffBlocks(Base.Data(), Base.Size(), BlockSize, WeakHasher, StrongHasher, SourceBlocks);

	UNSYNC_ASSERT(!IsSynchronized(NeedBlocks, SourceBlocks));
	
	FBuildTargetParams BuildParams;
	BuildParams.StrongHasher = StrongHasher;

	auto Target = BuildTargetBuffer(Source.Data(), Source.Size(), Base.Data(), Base.Size(), NeedBlocks, BuildParams);

	UNSYNC_ASSERT(Target.Size() == Source.Size());
	UNSYNC_ASSERT(!memcmp(Target.Data(), Source.Data(), Source.Size()));
}

void
TestFiles()
{
	using test::FTestFile;

	UNSYNC_LOG(L"TestFiles()");

	// TODO: test different chunk modes
	const EChunkingAlgorithmID ChunkingMode = EChunkingAlgorithmID::FixedBlocks;

	FPath TempDirPath = std::filesystem::temp_directory_path() / "unsync_test";
	std::error_code ErrorCode{};
	// Whether to remove the temp dir after running file tests.
	const bool bRemoveTempDir = !std::filesystem::exists(TempDirPath, ErrorCode);
	CreateDirectories(TempDirPath);

	bool bDirectoryExists = PathExists(TempDirPath) && IsDirectory(TempDirPath);
	UNSYNC_ASSERT(bDirectoryExists);

	FPath  TestFilename = TempDirPath / "a.bin";
	uint64 TestFileSize = 32'489'595;

	std::unique_ptr<uint8[]> TempBuffer = std::make_unique<uint8[]>(TestFileSize);

	{
		// overlapped file writing
		FNativeFile TestFile(TestFilename, EFileMode::CreateReadWrite, TestFileSize);
		UNSYNC_ASSERT(TestFile.IsValid());

		FillRandomBytes(TempBuffer.get(), TestFileSize, 321);
		uint64 WrittenBytes = TestFile.Write(TempBuffer.get(), 0, TestFileSize);
		UNSYNC_ASSERT(WrittenBytes == TestFileSize);

		memset(TempBuffer.get(), 0, TestFileSize);

		TestFile.Read(TempBuffer.get(), 0, TestFileSize);
		VerifyRandomBytes(TempBuffer.get(), TestFileSize, 321);
	}

	// CloneFileRange
	{
		constexpr uint64 CloneFileSize = 8 * 1024 * 1024;
		std::unique_ptr<uint8[]> CloneData = std::make_unique<uint8[]>(CloneFileSize);
		UNSYNC_ASSERT(CloneData);
		FTestFile InputFile(TempDirPath / "input.bin", EFileMode::CreateReadWrite, CloneFileSize);
		UNSYNC_ASSERT(InputFile.IsValid());
		FTestFile OutputFile(TempDirPath / "output.bin", EFileMode::CreateReadWrite, CloneFileSize);
		UNSYNC_ASSERT(OutputFile.IsValid());

		const int Seed = __COUNTER__;
		FillRandomBytes(CloneData.get(), CloneFileSize, Seed);
		uint64 Written = InputFile.Write(CloneData.get(), 0, CloneFileSize);
		UNSYNC_ASSERT(Written == CloneFileSize);

		// Copy the file with the 1st/2nd halves swapped.
		{
			const FCopyCommand Copy1{.Size = CloneFileSize / 2, .SourceOffset = 0, .TargetOffset = CloneFileSize / 2};
			const FCopyCommand Copy2{.Size = CloneFileSize / 2, .SourceOffset = CloneFileSize / 2, .TargetOffset = 0};

			if (CloneFileRange(InputFile, OutputFile, Copy1) && CloneFileRange(InputFile, OutputFile, Copy2))
			{
				UNSYNC_VERBOSE("Successfully cloned file contents");
				memset(CloneData.get(), 0, CloneFileSize);
				OutputFile.Read(CloneData.get(), CloneFileSize / 2, CloneFileSize / 2);
				OutputFile.Read(CloneData.get() + CloneFileSize / 2, 0, CloneFileSize / 2);
				VerifyRandomBytes(CloneData.get(), CloneFileSize, Seed);
			}
			else
			{
				UNSYNC_VERBOSE("Unable to clone file contents—probably an unsupported filesystem.");
			}
		}

		// Non-cluster-aligned clone
		{
			constexpr int	   UnalignedOffset		 = 3;
			constexpr int	   UnalignedCloneSize	 = CloneFileSize / 2 - UnalignedOffset;
			constexpr int	   UnalignedSourceOffset = UnalignedOffset;
			constexpr int	   UnalignedTargetOffset = CloneFileSize / 2 + UnalignedOffset;
			const FCopyCommand CopyUnaligned{.Size		   = UnalignedCloneSize,
											 .SourceOffset = UnalignedSourceOffset,
											 .TargetOffset = UnalignedTargetOffset};
			if (CloneFileRange(InputFile, OutputFile, CopyUnaligned))
			{
				UNSYNC_VERBOSE("Successfully cloned unaligned file contents");
				std::unique_ptr<uint8[]> UnalignedReadData = std::make_unique<uint8[]>(UnalignedCloneSize);
				UNSYNC_ASSERT(UnalignedReadData);
				const uint64 ReadLen = OutputFile.Read(UnalignedReadData.get(), UnalignedTargetOffset, UnalignedCloneSize);
				UNSYNC_ASSERT(ReadLen == UnalignedCloneSize);
				UNSYNC_ASSERT(memcmp(UnalignedReadData.get(), CloneData.get() + UnalignedOffset, UnalignedCloneSize) == 0);
			}
		}

		// Unaligned source/dest. Not supported on ReFS.
		{
			const FCopyCommand UnalignedCopy{.Size = 16384, .SourceOffset = 1, .TargetOffset = 2};
			(void)CloneFileRange(InputFile, OutputFile, UnalignedCopy);
		}

		// Small ranges; too small to block clone on ReFS (with 4 kiB clusters)
		// OK for clones to fail.
		{
			(void)CloneFileRange(InputFile, OutputFile, FCopyCommand{.Size = 100, .SourceOffset = 200, .TargetOffset = 200});
			(void)CloneFileRange(InputFile, OutputFile, FCopyCommand{.Size = 100, .SourceOffset = 4000, .TargetOffset = 4000});
			(void)CloneFileRange(InputFile, OutputFile, FCopyCommand{.Size = 4096, .SourceOffset = 100, .TargetOffset = 100});
			(void)CloneFileRange(InputFile, OutputFile, FCopyCommand{.Size = 5096, .SourceOffset = 100, .TargetOffset = 100});
		}
	}

	{
		memset(TempBuffer.get(), 0, TestFileSize);

		FNativeFile TestFile(TestFilename, EFileMode::ReadOnly);
		UNSYNC_ASSERT(TestFile.IsValid());
		UNSYNC_ASSERT(TestFile.GetSize() == TestFileSize);
		TestFile.Read(TempBuffer.get(), 0, TestFileSize);
		VerifyRandomBytes(TempBuffer.get(), TestFileSize, 321);
	}

	{
		FBuffer TestFile(TestFileSize);
		FillRandomBytes(TestFile.Data(), TestFileSize, 123);
		UNSYNC_ASSERT(WriteBufferToFile(TestFilename, TestFile));
	}

	{
		FBuffer TestFile = ReadFileToBuffer(TestFilename);
		UNSYNC_ASSERT(!TestFile.Empty());
		UNSYNC_ASSERT(TestFile.Size() == TestFileSize);
		VerifyRandomBytes(TestFile.Data(), TestFileSize, 123);
	}

	{
		uint32	BlockSize = uint32(16_KB);
		FBuffer TestFile  = ReadFileToBuffer(TestFilename);

		FAlgorithmOptions Algorithm;
		Algorithm.ChunkingAlgorithmId	= ChunkingMode;
		Algorithm.WeakHashAlgorithmId	= EWeakHashAlgorithmID::Naive;
		Algorithm.StrongHashAlgorithmId = EStrongHashAlgorithmID::MD5;

		FGenericBlockArray Blocks = ComputeBlocks(TestFile.Data(), TestFile.Size(), BlockSize, Algorithm);
		UNSYNC_ASSERT(Blocks.size() == 1984);

		std::vector<FBlock128> Blocks128 = ToBlock128(Blocks);

		FHash128 HashMd5 = HashMd5Bytes((const uint8*)Blocks128.data(), Blocks128.size() * sizeof(*Blocks128.data()));

		uint32 HashU32x4[4];
		memcpy(HashU32x4, HashMd5.Data, 16);

		UNSYNC_ASSERT(HashU32x4[0] == 0x561e130d);
		UNSYNC_ASSERT(HashU32x4[1] == 0xe3fe0e52);
		UNSYNC_ASSERT(HashU32x4[2] == 0xa9658163);
		UNSYNC_ASSERT(HashU32x4[3] == 0x2c317d4b);
	}

	if (bRemoveTempDir)
	{
		std::filesystem::remove_all(TempDirPath, ErrorCode);
		if (ErrorCode)
		{
			UNSYNC_LOG("Failed to remove TestFiles temp dir %s: %s", TempDirPath.c_str(), ErrorCode.message().c_str());
		}
	}
}

void
TestPerfHashWeak(EWeakHashAlgorithmID Algorithm)
{
	UNSYNC_LOG(L"TestPerfHashWeak(%hs)", ToString(Algorithm));
	UNSYNC_LOG_INDENT;

	UNSYNC_LOG(L"Generating data");

	FBuffer Buffer(2_GB + 123, 0);
	for (uint64 I = 0; I < Buffer.Size(); ++I)
	{
		Buffer[I] = uint8(I * 12345671 + 123);
	}

	UNSYNC_LOG(L"Hashing");

	auto TimeBegin = TimePointNow();

	uint32 Result = 0;

	if (Algorithm == EWeakHashAlgorithmID::Naive)
	{
		FRollingChecksum Hasher;
		Hasher.Update(Buffer.Data(), Buffer.Size());
		Result = Hasher.Get();
	}
	else if (Algorithm == EWeakHashAlgorithmID::BuzHash)
	{
		FBuzHash Hasher;
		Hasher.Update(Buffer.Data(), Buffer.Size());
		Result = Hasher.Get();
	}

	auto TimeEnd = TimePointNow();

	double Elapsed = DurationSec(TimeBegin, TimeEnd);
	UNSYNC_LOG(L"Complete in %.2f ms, %.2f MB / sec, hash: 0x%08X", Elapsed * 1000.0, (double(Buffer.Size()) / 1_MB) / Elapsed, Result);
}

void
TestPerfHashStrong(EStrongHashAlgorithmID StrongHasher)
{
	UNSYNC_LOG(L"TestPerfHashStrong(%hs)", ToString(StrongHasher));
	UNSYNC_LOG_INDENT;

	UNSYNC_LOG(L"Generating data");

	FBuffer Buffer(2_GB + 123, 0);
	for (uint64 I = 0; I < Buffer.Size(); ++I)
	{
		Buffer[I] = uint8(I * 12345671 + 123);
	}

	UNSYNC_LOG(L"Hashing");

	auto		 TimeBegin = TimePointNow();
	FGenericHash Result	   = ComputeHash(Buffer.Data(), Buffer.Size(), StrongHasher);
	auto		 TimeEnd   = TimePointNow();

	std::string HashStr = BytesToHexString(Result.Data, GetHashSize(Result.Type));
	double		Elapsed = DurationSec(TimeBegin, TimeEnd);
	UNSYNC_LOG(L"Complete in %.2f ms, %.2f MB / sec, hash: %hs",
			   Elapsed * 1000.0,
			   (double(Buffer.Size()) / 1_MB) / Elapsed,
			   HashStr.c_str());
}

void
TestPerfComputeBlocksVariable(EWeakHashAlgorithmID WeakHasher, EStrongHashAlgorithmID StrongHasher)
{
	UNSYNC_LOG(L"TestPerfComputeBlocksVariable(%hs, %hs)", ToString(WeakHasher), ToString(StrongHasher));
	UNSYNC_LOG_INDENT;

	UNSYNC_LOG(L"Generating data");

	FBuffer Buffer(2_GB, 0);
	FillRandomBytes(Buffer.Data(), Buffer.Size(), 1234);

	UNSYNC_LOG(L"Computing blocks");

	auto	   TimeBegin = TimePointNow();
	FMemReader BufferReader(Buffer);
	auto	   Result  = ComputeBlocksVariable(BufferReader, uint32(64_KB), WeakHasher, StrongHasher);
	auto	   TimeEnd = TimePointNow();

	double Elapsed = DurationSec(TimeBegin, TimeEnd);
	UNSYNC_LOG(L"Complete in %.2f ms, %.2f MB / sec, computed blocks: %llu",
			   Elapsed * 1000.0,
			   (double(Buffer.Size()) / 1_MB) / Elapsed,
			   Result.size());
}

void
TestBuzhash()
{
	UNSYNC_LOG(L"TestBuzhash()");
	UNSYNC_LOG_INDENT;

	// sanity check
	{
		uint8	 Data[] = {0, 1, 2, 3, 4, 5, 6, 7, 128, 255, 123, 19, 84};
		FBuzHash Hash;
		Hash.Update(Data, sizeof(Data));
		UNSYNC_ASSERT(Hash.Get() == 0x876e74b0);
	}

	// basic rolling sum
	{
		uint64 WindowSize = 64;
		uint32 BlockSize  = 65536;

		FBuffer Data;
		for (uint32 I = 0; I < BlockSize * 3; ++I)
		{
			std::hash<uint32> Hasher;
			Data.PushBack(uint8(Hasher(I)));
		}

		FBuzHash Hash1;
		Hash1.Update(&Data[0], WindowSize);

		for (uint32 I = 1; I < BlockSize; ++I)
		{
			Hash1.Sub(Data[I - 1]);
			Hash1.Add(Data[I + WindowSize - 1]);

			FBuzHash Hash2;
			Hash2.Update(&Data[I], WindowSize);

			UNSYNC_ASSERT(Hash1.Get() == Hash2.Get());
		}
	}
}

void
TestBasicHash()
{
	UNSYNC_LOG(L"TestBasicHash()");
	UNSYNC_LOG_INDENT;

	{
		FGenericHash Hash	 = HashBytes((const uint8*)"Blake3", 6, EHashType::Blake3_128);
		std::string	 HashStr = BytesToHexString(Hash.Data, Hash.Size());
		UNSYNC_ASSERT(HashStr == "2c4c1fa09b1a3459bc56ac6af6b446c8");
	}

	{
		FGenericHash Hash	 = HashBytes((const uint8*)"Blake3", 6, EHashType::Blake3_160);
		std::string	 HashStr = BytesToHexString(Hash.Data, Hash.Size());
		UNSYNC_ASSERT(HashStr == "2c4c1fa09b1a3459bc56ac6af6b446c89c784cf9");
	}

	{
		FGenericHash Hash	 = HashBytes((const uint8*)"Blake3", 6, EHashType::Blake3_256);
		std::string	 HashStr = BytesToHexString(Hash.Data, Hash.Size());
		UNSYNC_ASSERT(HashStr == "2c4c1fa09b1a3459bc56ac6af6b446c89c784cf9399825f2bede910bed452abe");
	}

	{
		std::unordered_map<FGenericHash, uint32> HashMap;
		FGenericHash							 K1 = ComputeHash((const uint8*)"123", 4, EStrongHashAlgorithmID::Blake3_128);
		FGenericHash							 K2 = ComputeHash((const uint8*)"123", 4, EStrongHashAlgorithmID::Blake3_160);
		FGenericHash							 K3 = ComputeHash((const uint8*)"123", 4, EStrongHashAlgorithmID::Blake3_256);

		HashMap[K1] = 1;
		HashMap[K2] = 2;
		HashMap[K3] = 3;

		UNSYNC_ASSERT(HashMap[K1] == 1);
		UNSYNC_ASSERT(HashMap[K2] == 2);
		UNSYNC_ASSERT(HashMap[K3] == 3);
	}

	// ParseHashFromHexString
	{
		FGenericHash Parsed;
		UNSYNC_ASSERT(ParseHashFromHexString(EStrongHashAlgorithmID::Blake3_128, "2c4c1fa09b1a3459bc56ac6af6b446c8", Parsed));
		std::string RoundTrip = BytesToHexString(Parsed.Data, Parsed.Size());
		UNSYNC_ASSERT(RoundTrip == "2c4c1fa09b1a3459bc56ac6af6b446c8");

		UNSYNC_ASSERT(ParseHashFromHexString(EStrongHashAlgorithmID::Blake3_160, "2c4c1fa09b1a3459bc56ac6af6b446c89c784cf9", Parsed));
		RoundTrip = BytesToHexString(Parsed.Data, Parsed.Size());
		UNSYNC_ASSERT(RoundTrip == "2c4c1fa09b1a3459bc56ac6af6b446c89c784cf9");

		// Wrong length
		UNSYNC_ASSERT(!ParseHashFromHexString(EStrongHashAlgorithmID::Blake3_128, "2c4c1fa0", Parsed));
		// Invalid hex char
		UNSYNC_ASSERT(!ParseHashFromHexString(EStrongHashAlgorithmID::Blake3_128, "zz4c1fa09b1a3459bc56ac6af6b446c8", Parsed));
	}

	// ParseUint64FromHexString
	{
		uint64 Value = 0;
		UNSYNC_ASSERT(ParseUint64FromHexString("0000000000000000", Value));
		UNSYNC_ASSERT(Value == 0);

		UNSYNC_ASSERT(ParseUint64FromHexString("0000000000000001", Value));
		UNSYNC_ASSERT(Value == 1);

		UNSYNC_ASSERT(ParseUint64FromHexString("ffffffffffffffff", Value));
		UNSYNC_ASSERT(Value == UINT64_MAX);

		UNSYNC_ASSERT(ParseUint64FromHexString("1584c5a558e0bc7f", Value));
		UNSYNC_ASSERT(Value == 0x1584c5a558e0bc7f);

		UNSYNC_ASSERT(ParseUint64FromHexString("ABCDEF0123456789", Value));
		UNSYNC_ASSERT(Value == 0xABCDEF0123456789);

		// Wrong length
		UNSYNC_ASSERT(!ParseUint64FromHexString("1234", Value));
		UNSYNC_ASSERT(!ParseUint64FromHexString("00000000000000001", Value));
		// Invalid hex char
		UNSYNC_ASSERT(!ParseUint64FromHexString("zzzzzzzzzzzzzzzz", Value));
	}
}

void
TestBuffer()
{
	UNSYNC_LOG(L"TestBuffer()");
	UNSYNC_LOG_INDENT;

	FBuffer Buf1;
	Buf1.Resize(1000);
	UNSYNC_ASSERT(Buf1.Size() == 1000);
	for (auto& It : Buf1)
	{
		It = 123;
	}
	for (const auto& It : Buf1)
	{
		UNSYNC_ASSERT(It == 123);
	}

	FBuffer Buf2(std::move(Buf1));

	UNSYNC_ASSERT(Buf1.Empty());
	UNSYNC_ASSERT(Buf2.Size() == 1000);

	FBuffer Buf3;
	std::swap(Buf3, Buf2);

	UNSYNC_ASSERT(Buf2.Empty());
	UNSYNC_ASSERT(Buf3.Size() == 1000);

	FBuffer Buf4;
	Buf4.Resize(123);
	Buf4 = std::move(Buf3);

	UNSYNC_ASSERT(Buf3.Empty());
	UNSYNC_ASSERT(Buf4.Size() == 1000);

	Buf4.Resize(500);
	UNSYNC_ASSERT(Buf4.Size() == 500);
	UNSYNC_ASSERT(Buf4.Capacity() == 1000);

	Buf4.Clear();
	UNSYNC_ASSERT(Buf4.Size() == 0);
	UNSYNC_ASSERT(Buf4.Capacity() == 1000);

	Buf4.Shrink();
	UNSYNC_ASSERT(Buf4.Size() == 0);
	UNSYNC_ASSERT(Buf4.Capacity() == 0);
}

void
TestMisc()
{
	UNSYNC_LOG(L"TestMisc()");
	UNSYNC_LOG_INDENT;

	{
		UNSYNC_LOG(L"OptimizeNeedList");
		std::vector<FNeedBlock> NeedBlocks;

		uint32 Rng = 123;

		uint64 CurrentOffset = 0;
		for (uint64 I = 0; I < 1024; ++I)
		{
			FNeedBlock Block;
			Block.Size		   = 32_KB + (Xorshift32(Rng) % 256_KB);
			Block.SourceOffset = CurrentOffset;
			Block.TargetOffset = CurrentOffset;
			NeedBlocks.push_back(Block);
			CurrentOffset += Block.Size;
		}

		std::vector<FCopyCommand> CopyCommands = OptimizeNeedList(NeedBlocks, ~0ull);
		UNSYNC_ASSERT(CopyCommands.size() == 1);
	}
}

void
TestLog()
{
	GLogVeryVerbose = true;
	GBreakOnError	= false;
	GBreakOnWarning = false;
	GLogProgress	= true;

	LogGlobalStatus(L"Running logging test");

	UNSYNC_LOG(L"Info text");
	UNSYNC_WARNING(L"Warning text");
	UNSYNC_ERROR(L"Error text");
	UNSYNC_VERBOSE(L"Debug text");
	UNSYNC_VERBOSE2(L"Trace text");

	const uint32 ProgressMax = 10;
	for (uint32 I = 0; I < ProgressMax; ++I)
	{
		LogGlobalProgress(I, ProgressMax);
		SchedulerSleep(20);
	}
}

void
RunTests(const std::string& Preset)
{
	FLogVerbosityScope VerboseLog(true);

	EWeakHashAlgorithmID WeakList[]{
		EWeakHashAlgorithmID::Naive,
		EWeakHashAlgorithmID::BuzHash,
	};

	EStrongHashAlgorithmID StrongList[]{
		EStrongHashAlgorithmID::MD5,
		EStrongHashAlgorithmID::Blake3_128,
		EStrongHashAlgorithmID::Blake3_160,
		EStrongHashAlgorithmID::Blake3_256,
	};

	if (Preset == "thread" || Preset == "all")
	{
		extern void TestThread();
		TestThread();
	}

	if (Preset == "misc" || Preset == "all")
	{
		TestMisc();
	}

	if (Preset == "buffer" || Preset == "all")
	{
		TestBuffer();
	}

	if (Preset == "files" || Preset == "all")
	{
		TestFiles();
	}

	if (Preset == "file_async_read" || Preset == "all")
	{
		extern void TestFileAsyncRead();
		TestFileAsyncRead();
	}

	if (Preset == "build_target" || Preset == "all")
	{
		for (EWeakHashAlgorithmID Weak : WeakList)
		{
			for (EStrongHashAlgorithmID Strong : StrongList)
			{
				TestBuildTarget(Weak, Strong);
			}
		}
	}

	if (Preset == "rolling_sum" || Preset == "all")
	{
		TestRollingSum();
	}

	if (Preset == "buzhash" || Preset == "all")
	{
		TestBuzhash();
	}

	if (Preset == "basic_hash" || Preset == "all")
	{
		TestBasicHash();
	}

	if (Preset == "chunking" || Preset == "all")
	{
		extern void TestChunking();
		TestChunking();
	}

	if (Preset == "sync" || Preset == "all")
	{
		for (auto Weak : WeakList)
		{
			for (auto Strong : StrongList)
			{
				TestSync(Weak, Strong);
			}
		}
	}

	if (Preset == "perf" || Preset == "all")
	{
		TestPerfComputeBlocksVariable(EWeakHashAlgorithmID::Naive, EStrongHashAlgorithmID::Blake3_128);
		TestPerfComputeBlocksVariable(EWeakHashAlgorithmID::BuzHash, EStrongHashAlgorithmID::Blake3_128);
		TestPerfComputeBlocksVariable(EWeakHashAlgorithmID::BuzHash, EStrongHashAlgorithmID::Blake3_160);

		for (auto Strong : StrongList)
		{
			TestPerfHashStrong(Strong);
		}

		for (auto Weak : WeakList)
		{
			TestPerfHashWeak(Weak);
		}
	}

	if (Preset == "log" || Preset == "all")
	{
		TestLog();
	}

	if (Preset == "parse_remote" || Preset == "all")
	{
		extern void TestParseRemote();
		TestParseRemote();
	}

	if (Preset == "minicb" || Preset == "all")
	{
		extern void TestMiniCb();
		TestMiniCb();
	}

	if (Preset == "filetime" || Preset == "all")
	{
		extern void TestFileTime();
		TestFileTime();
	}

	if (Preset == "fileattrib" || Preset == "all")
	{
		extern void TestFileAttrib();
		TestFileAttrib();
	}

	if (Preset == "pathutil" || Preset == "all")
	{
		extern void TestPathUtil();
		TestPathUtil();
	}

	if (Preset == "horde_manifest_decode" || Preset == "all")
	{
		extern void TestHordeManifestDecode();
		TestHordeManifestDecode();
	}

	if (Preset == "horde_artifact_format" || Preset == "all")
	{
		extern void TestHordeArtifactFormat();
		TestHordeArtifactFormat();
	}
}

}  // namespace unsync
