// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <string>

namespace unsync {

void RunTests(const std::string& Preset);

}  // namespace unsync
