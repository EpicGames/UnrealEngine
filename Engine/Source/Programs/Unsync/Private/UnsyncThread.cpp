// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncThread.h"
#include "UnsyncUtil.h"

#include <chrono>

namespace unsync {

uint32 GMaxThreads = std::min<uint32>(UNSYNC_MAX_TOTAL_THREADS, std::thread::hardware_concurrency());

void
FThreadPool::StartWorkers(uint32 NumWorkers)
{
	std::unique_lock<std::mutex> LockScope(Mutex);

	while (Threads.size() < NumWorkers)
	{
		Threads.emplace_back(
			[this]()
			{
				while (DoWorkInternal(true))
				{
				}
			});
	}
}

FThreadPool::~FThreadPool()
{
	bShutdown = true;
	WakeCondition.notify_all();

	for (std::thread& Thread : Threads)
	{
		Thread.join();
	}
}

FThreadPool::FTaskFunction
FThreadPool::PopTask(bool bWaitForSignal)
{
	std::unique_lock<std::mutex> LockScope(Mutex);

	if (bWaitForSignal)
	{
		auto WaitUntil = [this]() { return bShutdown || !Tasks.empty(); };
		WakeCondition.wait(LockScope, WaitUntil);
	}

	FThreadPool::FTaskFunction Result;

	if (!Tasks.empty())
	{
		Result = std::move(Tasks.back());
		Tasks.pop_back();
	}

	return Result;
}

void
FThreadPool::PushTask(FTaskFunction&& Fun, bool bAllowImmediateExecution)
{
	if (Threads.empty() || (NumRunningTasks.load() == NumWorkerThreads() && bAllowImmediateExecution))
	{
		Fun();
	}
	else
	{
		std::unique_lock<std::mutex> LockScope(Mutex);
		Tasks.push_back(std::forward<FTaskFunction>(Fun));
		WakeCondition.notify_one();
	}
}

bool
FThreadPool::DoWorkInternal(bool bWaitForSignal)
{
	FTaskFunction Task = PopTask(bWaitForSignal);

	if (Task)
	{
		NumRunningTasks++;

		Task();

		NumRunningTasks--;

		return true;
	}
	else
	{
		return false;
	}
}

void
SchedulerSleep(uint32 Milliseconds)
{
	std::this_thread::sleep_for(std::chrono::milliseconds(Milliseconds));
}

void
TestThread()
{
	UNSYNC_LOG(L"TestThread()");
	UNSYNC_LOG_INDENT;

	{
		UNSYNC_LOG(L"PushTask");

		const uint32		NumTasks = 1000;
		std::atomic<uint32> Counter	 = 0;

		{
			FThreadPool ThreadPool;
			ThreadPool.StartWorkers(10);

			uint32 RandomSeed = 1234;
			for (uint32 i = 0; i < NumTasks; ++i)
			{
				uint32 R = Xorshift32(RandomSeed) % 10;
				ThreadPool.PushTask(
					[R, &Counter]
					{
						SchedulerSleep(1 + R);
						Counter++;
					});
			}

			while (ThreadPool.TryExecuteTask())
			{
			}

			// thread pool destructor waits for outstanding tasks
		}

		UNSYNC_ASSERT(Counter == NumTasks)
	}
}

}  // namespace unsync
