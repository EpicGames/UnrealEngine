// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"

#include "UnsyncLog.h"
#include "UnsyncUtil.h"

UNSYNC_THIRD_PARTY_INCLUDES_START
#include <atomic>
#include <condition_variable>
#include <functional>
#include <thread>
#include <vector>
#include <semaphore>
UNSYNC_THIRD_PARTY_INCLUDES_END

namespace unsync {

static constexpr uint32 UNSYNC_MAX_TOTAL_THREADS = 64;

extern uint32 GMaxThreads;

class FScheduler;

struct FThreadElectScope
{
	const bool			 bValue;	  // NOLINT
	const bool			 bCondition;  // NOLINT
	std::atomic<uint64>& Counter;
	FThreadElectScope(std::atomic<uint64>& InCounter, bool bInCondition)
	: bValue(bInCondition && (InCounter.fetch_add(1) == 0))
	, bCondition(bInCondition)
	, Counter(InCounter)
	{
	}
	~FThreadElectScope()
	{
		if (bCondition)
		{
			Counter.fetch_sub(1);
		}
	}

	operator bool() const { return bValue; }
};

struct FThreadLogConfig
{
	FThreadLogConfig() : ParentThreadIndent(GLogIndent), bParentThreadVerbose(GLogVerbose) {}


	uint32				ParentThreadIndent;
	bool				bParentThreadVerbose;
	std::atomic<uint64> NumActiveVerboseLogThreads = {};

	struct FScope
	{
		FScope(FThreadLogConfig& Parent)
		: AllowVerbose(Parent.NumActiveVerboseLogThreads, Parent.bParentThreadVerbose)
		, VerboseScope(AllowVerbose.bValue)
		, IndentScope(Parent.ParentThreadIndent, true)
		{
		}
		FThreadElectScope  AllowVerbose;
		FLogVerbosityScope VerboseScope;
		FLogIndentScope	   IndentScope;
	};
};

void SchedulerSleep(uint32 Milliseconds);

class FThreadPool
{
public:

	using FTaskFunction = std::function<
		void()>;

	FThreadPool() = default;
	~FThreadPool();

	// Launches worker threads until total started worker count reaches NumWorkers.
	// Does nothing if the number of already launched workers is lower than given value.
	void StartWorkers(uint32 NumWorkers);

	// Adds a task to the task list
	void PushTask(FTaskFunction&& Fun, bool bAllowImmediateExecution = true);

	// Try to pop the next task from the list and execute it on the current thread.
	// Returns false if list is empty, which may happen if worker threads have picked up the tasks already.
	bool TryExecuteTask() { return DoWorkInternal(false); }

	uint32 NumWorkerThreads() const { return uint32(Threads.size()); }

private:

	// Try to execute a task and return whether there may be more tasks to run
	bool DoWorkInternal(bool bWaitForSignal);

	FTaskFunction PopTask(bool bWaitForSignal);

	std::vector<std::thread>  Threads;
	std::vector<FTaskFunction> Tasks;

	std::mutex				Mutex;
	std::condition_variable WakeCondition;
	std::atomic<bool>		bShutdown;
	std::atomic<uint64>		NumRunningTasks;
};

}  // namespace unsync
