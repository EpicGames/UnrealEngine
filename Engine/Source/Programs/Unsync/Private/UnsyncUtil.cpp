// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncUtil.h"
#include "UnsyncFile.h"
#include "UnsyncSocket.h"
#include "UnsyncProtocol.h"

#if UNSYNC_PLATFORM_WINDOWS
#	include <Windows.h>
#	include <shellapi.h>
#	include <lm.h>
#	include <lmdfs.h>
#	include <wincrypt.h>
#	include <winnetwk.h> // for WNetGetUniversalName
#	pragma comment(lib, "Netapi32.lib")
#	pragma comment(lib, "Crypt32.lib")
#	pragma comment(lib, "Bcrypt.lib")
#	pragma comment(lib, "Mpr.lib")
#	pragma comment(lib, "Advapi32.lib") // for registry access
#endif	// UNSYNC_PLATFORM_WINDOWS

#include <stdlib.h>
#include <codecvt>
#include <filesystem>
#include <locale>
#include <mutex>
#include <sstream>
#include <unordered_set>
#include <system_error>
#include <fmt/format.h>
#if __has_include(<fmt/xchar.h>)
#	include <fmt/xchar.h>
#endif

#ifdef __GNUC__
#	define _strnicmp strncasecmp
#endif

namespace unsync {

static FBuffer GSystemRootCerts;

static const char G_HEX_CHARS[] = "0123456789abcdef";

template<typename CharT>
uint64
BytesToHexCharsT(CharT* Output, uint64 OutputSize, const uint8* Input, uint64 InputSize)
{
	const uint64 MaxBytes = std::min(OutputSize / 2, InputSize);
	for (uint64 I = 0; I < MaxBytes; ++I)
	{
		uint8 V			  = Input[I];
		Output[I * 2 + 0] = CharT(G_HEX_CHARS[V >> 4]);
		Output[I * 2 + 1] = CharT(G_HEX_CHARS[V & 0xF]);
	}
	return MaxBytes * 2;
}

uint64
BytesToHexChars(char* Output, uint64 OutputSize, const uint8* Input, uint64 InputSize)
{
	return BytesToHexCharsT(Output, OutputSize, Input, InputSize);
}

uint64
BytesToHexChars(wchar_t* Output, uint64 OutputSize, const uint8* Input, uint64 InputSize)
{
	return BytesToHexCharsT(Output, OutputSize, Input, InputSize);
}

std::string
BytesToHexString(const uint8* Data, uint64 Size)
{
	std::string Result;
	Result.resize(Size * 2);
	uint64 WrittenChars = BytesToHexChars(Result.data(), Result.length(), Data, Size);

#ifdef _NDEBUG
	UNSYNC_ASSERT(written_chars == result.length());
#else
	UNSYNC_UNUSED(WrittenChars);
#endif

	return Result;
}


void
FormatJsonKeyValueStr(std::wstring& Output, std::wstring_view K, std::wstring_view V, std::wstring_view Suffix)
{
	fmt::format_to(std::back_inserter(Output), L"\"{}\": \"{}\"{}", K, V, Suffix);
}

void
FormatJsonKeyValueStr(std::string& Output, std::string_view K, std::string_view V, std::string_view Suffix)
{
	fmt::format_to(std::back_inserter(Output), "\"{}\": \"{}\"{}", K, V, Suffix);
}

void
FormatJsonKeyValueUInt(std::wstring& Output, std::wstring_view K, uint64 V, std::wstring_view Suffix)
{
	fmt::format_to(std::back_inserter(Output), L"\"{}\": {}{}", K, V, Suffix);
}

void
FormatJsonKeyValueUInt(std::string& Output, std::string_view K, uint64 V, std::string_view Suffix)
{
	fmt::format_to(std::back_inserter(Output), "\"{}\": {}{}", K, V, Suffix);
}


void
FormatJsonKeyValueBool(std::wstring& Output, std::wstring_view K, bool V, std::wstring_view Suffix)
{
	fmt::format_to(std::back_inserter(Output), L"\"{}\": {}{}", K, V ? L"true" : L"false", Suffix);
}

void
FormatJsonKeyValueBool(std::string& Output, std::string_view K, bool V, std::string_view Suffix)
{
	fmt::format_to(std::back_inserter(Output), "\"{}\": {}{}", K, V ? "true" : "false", Suffix);
}

void
FormatJsonBlock(std::wstring& Output, const FGenericBlock& Block)
{
	Output += L"{";

	static const size_t MaxHashLen = 2 * sizeof(Block.HashStrong.Data);
	wchar_t				HashChars[MaxHashLen];

	uint64			  HashLen = BytesToHexChars(HashChars, MaxHashLen, Block.HashStrong.Data, Block.HashStrong.Size());
	std::wstring_view HashStr = std::wstring_view(HashChars, HashLen);

	FormatJsonKeyValueUInt(Output, L"offset", Block.Offset, L", ");
	FormatJsonKeyValueUInt(Output, L"size", Block.Size, L", ");
	if (Block.HashWeak != 0)
	{
		FormatJsonKeyValueUInt(Output, L"hash_weak", Block.HashWeak, L", ");
	}
	FormatJsonKeyValueStr(Output, L"hash_strong", HashStr);

	Output += L"}";
}

void
FormatJsonBlock(std::string& Output, const FGenericBlock& Block)
{
	Output += "{";

	static const size_t MaxHashLen = 2 * sizeof(Block.HashStrong.Data);
	char				HashChars[MaxHashLen];

	uint64			 HashLen = BytesToHexChars(HashChars, MaxHashLen, Block.HashStrong.Data, Block.HashStrong.Size());
	std::string_view HashStr = std::string_view(HashChars, HashLen);

	FormatJsonKeyValueUInt(Output, "offset", Block.Offset, ", ");
	FormatJsonKeyValueUInt(Output, "size", Block.Size, ", ");
	if (Block.HashWeak != 0)
	{
		FormatJsonKeyValueUInt(Output, "hash_weak", Block.HashWeak, ", ");
	}
	FormatJsonKeyValueStr(Output, "hash_strong", HashStr);

	Output += "}";
}

void
FormatJsonBlockArray(std::wstring& Output, const FGenericBlockArray& Blocks)
{
	Output += L"[\n";
	uint64 BlockIndex = 0;
	for (const FGenericBlock& Block : Blocks)
	{
		if (BlockIndex != 0)
		{
			Output += L",\n";
		}

		FormatJsonBlock(Output, Block);

		++BlockIndex;
	}
	Output += L"]";
}

void
FormatJsonBlockArray(std::string& Output, const FGenericBlockArray& Blocks)
{
	Output += "[\n";
	uint64 BlockIndex = 0;
	for (const FGenericBlock& Block : Blocks)
	{
		if (BlockIndex != 0)
		{
			Output += ",\n";
		}

		FormatJsonBlock(Output, Block);

		++BlockIndex;
	}
	Output += "]";
}


FTimingLogger::FTimingLogger(const char* InName, ELogLevel InLogLevel, bool bInEnabled)
: bEnabled(bInEnabled)
, Name(InName)
, LogLevel(InLogLevel)
{
	TimeBegin = TimePointNow();
}

FTimingLogger::~FTimingLogger()
{
	Finish();
}

void FTimingLogger::Finish()
{
	if (bEnabled)
	{
		FTimePoint	  TimeEnd	   = TimePointNow();
		FTimeDuration Duration	   = FTimeDuration(TimeEnd - TimeBegin);
		double		  TotalSeconds = DurationSec(TimeBegin, TimeEnd);

		int H = std::chrono::duration_cast<std::chrono::hours>(Duration).count();
		int M = std::chrono::duration_cast<std::chrono::minutes>(Duration).count() % 60;
		int S = int(std::chrono::duration_cast<std::chrono::seconds>(Duration).count() % 60);

		if (Name.empty())
		{
			LogPrintf(LogLevel, L"%.3f sec\n", TotalSeconds);
		}
		else
		{
			if (TotalSeconds >= 60.0)
			{
				LogPrintf(LogLevel, L"%hs: %.3f sec (%02d:%02d:%02d)\n", Name.c_str(), TotalSeconds, H, M, S);
			}
			else
			{
				LogPrintf(LogLevel, L"%hs: %.3f sec\n", Name.c_str(), TotalSeconds);
			}
		}

		LogFlush();

		bEnabled = false;
	}
}

template<typename T>
static bool
IsTrivialAsciiString(const T& Input)
{
	for (auto c : Input)
	{
		if ((unsigned)c > 127)
		{
			return false;
		}
	}
	return true;
}

#ifdef __clang__
#	pragma clang diagnostic push
#	pragma clang diagnostic ignored "-Wdeprecated-declarations"  // codecvt_utf8 is deprecated, but there is no trivial replacement
#endif															  // __clang__

std::wstring
ConvertUtf8ToWide(std::string_view StringUtf8)
{
	std::wstring Result;

	if (IsTrivialAsciiString(StringUtf8))
	{
		Result.resize(StringUtf8.length());
		wchar_t* ResultChars = Result.data();
		for (char c : StringUtf8)
		{
			*ResultChars = (wchar_t)c;
			++ResultChars;
		}
	}
	else
	{
		std::wstring_convert<std::codecvt_utf8<wchar_t>> Cvt;
		Result = Cvt.from_bytes(StringUtf8.data(), StringUtf8.data() + StringUtf8.length());
	}

	return Result;
}


void
ConvertWideToUtf8(std::wstring_view StringWide, std::string& Result)
{
	Result.clear();

	if (IsTrivialAsciiString(StringWide))
	{
		Result.resize(StringWide.length());
		char* ResultChars = Result.data();
		for (wchar_t wc : StringWide)
		{
			*ResultChars = (char)wc;
			++ResultChars;
		}
	}
	else
	{
		std::wstring_convert<std::codecvt_utf8<wchar_t>> Cvt;
		Result = Cvt.to_bytes(StringWide.data(), StringWide.data() + StringWide.length());
	}
}

std::string
ConvertWideToUtf8(std::wstring_view StringWide)
{
	std::string Result;
	ConvertWideToUtf8(StringWide, Result);
	return Result;
}

#ifdef __clang__
#	pragma clang diagnostic pop
#endif	// __clang__

const bool
FFileAttributeCache::Exists(const FPath& Path) const
{
	const auto It = Map.find(Path.native());
	return It != Map.end();
}

std::string
ToString(const FPath& Path)
{
	FPathStringView PathView = Path.native();
	return ToString(PathView);
}

std::string
StringToLower(std::string_view Input)
{
	std::string Result(Input);
	std::transform(Result.begin(), Result.end(), Result.begin(), [](int32 C) { return char(::tolower(C)); });
	return Result;
}


std::wstring
StringToLower(std::wstring_view Input)
{
	std::wstring Result(Input);
	std::transform(Result.begin(), Result.end(), Result.begin(), [](int32 C) { return wchar_t(::tolower(C)); });
	return Result;
}

std::wstring
StringToUpper(std::wstring_view Input)
{
	std::wstring Result(Input);
	std::transform(Result.begin(), Result.end(), Result.begin(), [](int32 C) { return wchar_t(::toupper(C)); });
	return Result;
}

bool
StringStartsWith(const std::string_view String, const std::string_view Prefix, bool bCaseSensitive)
{
	if (bCaseSensitive)
	{
		return String.starts_with(Prefix);
	}
	else if (Prefix.length() <= String.length())
	{
		return _strnicmp(String.data(), Prefix.data(), Prefix.length()) == 0;
	}
	else
	{
		return false;
	}
}

bool
StringEquals(const std::string_view A, const std::string_view B, bool bCaseSensitive)
{
	if (bCaseSensitive)
	{
		return A == B;
	}
	else if (A.length() == B.length())
	{
		return _strnicmp(A.data(), B.data(), A.length()) == 0;
	}
	else
	{
		return false;
	}
}

std::string
StringEscape(const std::string_view Input)
{
	// Adapted from Json11

	std::string Result;

	for (size_t i = 0; i < Input.length(); i++)
	{
		const char C = Input[i];
		if (C == '\\')
		{
			Result += "\\\\";
		}
		else if (C == '"')
		{
			Result += "\\\"";
		}
		else if (C == '\b')
		{
			Result += "\\b";
		}
		else if (C == '\f')
		{
			Result += "\\f";
		}
		else if (C == '\n')
		{
			Result += "\\n";
		}
		else if (C == '\r')
		{
			Result += "\\r";
		}
		else if (C == '\t')
		{
			Result += "\\t";
		}
		else if (static_cast<uint8_t>(C) <= 0x1f)
		{
			char buf[8];
			snprintf(buf, sizeof buf, "\\u%04x", C);
			Result += buf;
		}
		else if (static_cast<uint8_t>(C) == 0xe2 && i + 2 < Input.length() &&
				 static_cast<uint8_t>(Input[i + 1]) == 0x80 && static_cast<uint8_t>(Input[i + 2]) == 0xa8)
		{
			Result += "\\u2028";
			i += 2;
		}
		else if (static_cast<uint8_t>(C) == 0xe2 && i + 2 < Input.length() &&
				 static_cast<uint8_t>(Input[i + 1]) == 0x80 && static_cast<uint8_t>(Input[i + 2]) == 0xa9)
		{
			Result += "\\u2029";
			i += 2;
		}
		else
		{
			Result += C;
		}
	}

	return Result;
}

FPath
GetUniversalPath(const FPath& Path)
{
	FPath Result = Path;

#if UNSYNC_PLATFORM_WINDOWS

	// https://docs.microsoft.com/en-us/windows/win32/api/winnetwk/nf-winnetwk-wnetgetuniversalnamea

	static constexpr DWORD MaxBufferSize = 1024;

	WCHAR Buffer[MaxBufferSize] = {};

	DWORD				  BufferSize		= MaxBufferSize;
	UNIVERSAL_NAME_INFOW* UniversalNameInfo = (UNIVERSAL_NAME_INFOW*)Buffer;

	DWORD ErrorCode = WNetGetUniversalNameW(Path.native().c_str(), UNIVERSAL_NAME_INFO_LEVEL, (LPVOID)UniversalNameInfo, &BufferSize);
	if (ErrorCode == NO_ERROR)
	{
		Result = FPath(UniversalNameInfo->lpUniversalName);
	}

#endif	// UNSYNC_PLATFORM_WINDOWS

	return Result;
}

static FPath
GetNormalWeaklyCanonicalAbsolutePath(const FPath& InPath)
{
	FPath NormalPath		 = InPath.lexically_normal();
	FPath CanonicalPath		 = std::filesystem::weakly_canonical(NormalPath);
	FPath AbsoluteNormalPath = std::filesystem::absolute(CanonicalPath);
	return AbsoluteNormalPath;
}

FPath
NormalizeFilenameWide(std::wstring_view Filename)
{
	if (Filename.empty())
	{
		return FPath();
	}

	std::wstring_view FileUrlPrefix = L"file://";
	if (Filename.starts_with(FileUrlPrefix))
	{
		Filename = Filename.substr(FileUrlPrefix.length());
	}

	FPath FilenameAsPath = FPath(Filename);

	FPath AbsoluteNormalPath;
	if (Filename.starts_with(L"\\\\") || Filename.starts_with(L"//"))
	{
		AbsoluteNormalPath = FilenameAsPath;  // Assume network paths are absolute
	}
	else
	{
		AbsoluteNormalPath = GetNormalWeaklyCanonicalAbsolutePath(FilenameAsPath);
	}

	return AbsoluteNormalPath;
}

FPath
NormalizeFilenameUtf8(std::string_view Filename)
{
	if (Filename.empty())
	{
		return FPath();
	}

	std::string_view FileUrlPrefix = "file://";
	if (Filename.starts_with(FileUrlPrefix))
	{
		Filename = Filename.substr(FileUrlPrefix.length());
	}

	FPath FilenameAsPath = ConvertUtf8ToWide(Filename);

	FPath AbsoluteNormalPath;
	if (Filename.starts_with("\\\\") || Filename.starts_with("//"))
	{
		AbsoluteNormalPath = FilenameAsPath; // Assume network paths are absolute
	}
	else
	{
		AbsoluteNormalPath = GetNormalWeaklyCanonicalAbsolutePath(FilenameAsPath);
	}

	return AbsoluteNormalPath;
}

FPath
GetAbsoluteNormalPath(const FPath& InPath)
{
	FPath NormalPath		 = InPath.lexically_normal();
	FPath AbsoluteNormalPath = std::filesystem::absolute(NormalPath);
	return AbsoluteNormalPath;
}

const FBuffer&
GetSystemRootCerts()
{
	static bool IsInitialized = false;
	if (IsInitialized)
	{
		return GSystemRootCerts;
	}

	IsInitialized = true;

#if UNSYNC_PLATFORM_WINDOWS
	HCERTSTORE CertStore = CertOpenSystemStoreA((HCRYPTPROV_LEGACY) nullptr, "ROOT");
	if (!CertStore)
	{
		UNSYNC_ERROR(L"Failed to open root system certificate storage");
		return GSystemRootCerts;
	}

	PCCERT_CONTEXT CertContext = CertEnumCertificatesInStore(CertStore, nullptr);

	GSystemRootCerts.Clear();

	std::unordered_set<FHash128> UniqueCerts;

	uint32 NumDuplicateCerts = 0;

	FBuffer TempCert;
	while (CertContext)
	{
		DWORD CertLen = 0;
		CryptBinaryToStringA(CertContext->pbCertEncoded, CertContext->cbCertEncoded, CRYPT_STRING_BASE64HEADER, nullptr, &CertLen);

		TempCert.Resize(CertLen);
		CryptBinaryToStringA(CertContext->pbCertEncoded,
							 CertContext->cbCertEncoded,
							 CRYPT_STRING_BASE64HEADER,
							 (char*)TempCert.Data(),
							 &CertLen);

		FHash128 CertHash = HashBlake3Bytes<FHash128>(TempCert.Data(), TempCert.Size());

		auto InsertResult = UniqueCerts.insert(CertHash);
		if (InsertResult.second)
		{
			GSystemRootCerts.Append(TempCert.Data(), TempCert.Size() - 1);
		}
		else
		{
			NumDuplicateCerts++;
		}

		CertContext = CertEnumCertificatesInStore(CertStore, CertContext);
	}
	CertCloseStore(CertStore, 0);
#endif	// UNSYNC_PLATFORM_WINDOWS

#if UNSYNC_PLATFORM_UNIX
	{
		const char* PossibleCertsPaths[] = {
			"/etc/ssl/certs/ca-certificates.crt",				  // Debian/Ubuntu/Gentoo etc.
			"/etc/pki/tls/certs/ca-bundle.crt",					  // Fedora/RHEL 6
			"/etc/ssl/ca-bundle.pem",							  // OpenSUSE
			"/etc/pki/tls/cacert.pem",							  // OpenELEC
			"/etc/pki/ca-trust/extracted/pem/tls-ca-bundle.pem",  // CentOS/RHEL 7
			"/etc/ssl/cert.pem",								  // Alpine Linux
		};

		for (const char* CertsPath : PossibleCertsPaths)
		{
			GSystemRootCerts = ReadFileToBuffer(CertsPath);
			if (!GSystemRootCerts.Empty())
			{
				UNSYNC_VERBOSE2(L"Loaded system CA bundle from '%hs'", CertsPath);
				break;
			}
		}

		if (GSystemRootCerts.Empty())
		{
			UNSYNC_WARNING(
				L"Could not find CA certificate bundle in any of the known locations. "
				L"Use --cacert <path> to explicitly specify the CA file.");
		}
	}
#endif	// UNSYNC_PLATFORM_UNIX

	GSystemRootCerts.PushBack(0);

	return GSystemRootCerts;
}

#if UNSYNC_PLATFORM_WINDOWS
void
OpenUrlInDefaultBrowser(const char* Address)
{
	ShellExecuteA(nullptr, "open", Address, nullptr, nullptr, SW_SHOWNORMAL);
}
#else  // UNSYNC_PLATFORM_WINDOWS
void
OpenUrlInDefaultBrowser(const char* Address)
{
#	ifdef __APPLE__
	std::string Command = fmt::format("open \"{}\"", Address);
#	else // assume linux with xdg-utils installed
	std::string Command = fmt::format("xdg-open \"{}\"", Address);
#	endif

	int RetCode = system(Command.c_str());
	if (RetCode != 0)
	{
		UNSYNC_ERROR(L"Failed to run command '%hs'", Command.c_str());
	}
}
#endif // UNSYNC_PLATFORM_WINDOWS


#if UNSYNC_PLATFORM_WINDOWS
FPath GetUserHomeDirectory()
{
	if (const wchar_t* EnvUserProfile = _wgetenv(L"USERPROFILE"))
	{
		return NormalizeFilenameWide(EnvUserProfile);
	}
	else
	{
		return {};
	}
}
#else // UNSYNC_PLATFORM_WINDOWS
FPath
GetUserHomeDirectory()
{
	if (const char* EnvUserProfile = getenv("HOME"))
	{
		return NormalizeFilenameUtf8(EnvUserProfile);
	}
	else
	{
		return {};
	}
}
#endif // UNSYNC_PLATFORM_WINDOWS

std::string
FormatSystemErrorMessage(int32 ErrorCode)
{
	std::string ErrorMessage = std::system_category().message(ErrorCode);
	return fmt::format("Error code {}: {}", ErrorCode, ErrorMessage);
}

FHash256
GetAnonymizedMachineId(std::string_view Salt)
{
	std::string Seed;

	Seed += Salt;
	Seed += GetCurrentHostName();
	Seed += " {22FF4421-8CAC-4A14-9E4C-780AAF8BBF2A}";

#if UNSYNC_PLATFORM_WINDOWS
	{
		HKEY Key = {};
		if (RegOpenKeyA(HKEY_LOCAL_MACHINE, "SOFTWARE\\Microsoft\\Cryptography", &Key) == ERROR_SUCCESS)
		{
			char  Buffer[512] = {};
			DWORD BufferSize  = sizeof(Buffer);
			auto  Status	  = RegQueryValueExA(Key, "MachineGuid", nullptr, nullptr, (LPBYTE)Buffer, &BufferSize);
			if (Status == ERROR_SUCCESS && BufferSize > 1)
			{
				std::string_view MachineGuid = std::string_view(Buffer, BufferSize - 1);
				Seed += " MachineGuid ";
				Seed += MachineGuid;
			}
			RegCloseKey(Key);
		}
	}
#endif	// UNSYNC_PLATFORM_WINDOWS

	// TODO: read `/etc/machine-id` on linux
	// TODO: use `ioreg -rd1 -c IOPlatformExpertDevice` to get IOPlatformUUID on mac

	FHash256 Result = HashBlake3String<FHash256>(Seed);

	return Result;
}

std::string
GetAnonymizedMachineIdString(std::string_view Seed)
{
	FHash256	MachineId = GetAnonymizedMachineId(Seed);
	std::string Result	  = HashToHexString(MachineId);
	return Result;
}

bool
LooksLikeHash160(const std::string_view Str)
{
	if (Str.length() != 40)
	{
		return false;
	}

	const char* PossibleChars = "0123456789abcdefABCDEF";
	if (Str.find_first_not_of(PossibleChars) != std::string::npos)
	{
		return false;
	}

	return true;
}

bool
LooksLikeHash160(const std::wstring_view Str)
{
	if (Str.length() != 40)
	{
		return false;
	}

	const wchar_t* PossibleChars = L"0123456789abcdefABCDEF";
	if (Str.find_first_not_of(PossibleChars) != std::wstring::npos)
	{
		return false;
	}

	return true;
}

bool
LooksLikeUrl(std::string_view Str)
{
	std::string_view Prefixes[] = {
		"http://",
		"https://",
		"unsync://",
		"unsync+tls://",
		"horde+http://",
		"horde+https://",
		"unsync+http://",
		"unsync+https://",
		"jupiter+http://",
		"jupiter+https://",
	};

	for (std::string_view Prefix : Prefixes)
	{
		if (Str.starts_with(Prefix))
		{
			return true;
		}
	}
	
	return false;
}

std::vector<std::string_view>
SplitByAny(std::string_view String, const char* SeparatorCharacters)
{
	std::vector<std::string_view> Result;

	while (!String.empty())
	{
		size_t Pos = String.find_first_of(SeparatorCharacters);
		if (Pos == std::string::npos)
		{
			Result.push_back(String);
			break;
		}

		std::string_view Part = String.substr(0, Pos);

		Result.push_back(Part);

		String = String.substr(Pos + 1);
	}

	return Result;
}

std::string_view
AsStringView(const FBuffer& Buffer)
{
	if (Buffer.Empty())
	{
		return {};
	}
	else
	{
		return std::string_view(reinterpret_cast<const char*>(Buffer.Data()), Buffer.Size());
	}
}

}  // namespace unsync
