// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"

UNSYNC_THIRD_PARTY_INCLUDES_START
#if UNSYNC_PLATFORM_WINDOWS
#	include <Windows.h>
#	include <intrin.h>
#endif	// UNSYNC_PLATFORM_WINDOWS
#include <fmt/format.h>
#include <stdio.h>
#include <chrono>
#include <concepts>
#include <string>
#include <string_view>
#include <unordered_map>
#include <vector>
UNSYNC_THIRD_PARTY_INCLUDES_END

#include "UnsyncHash.h"
#include "UnsyncLog.h"

namespace unsync {

class FBuffer;

using FTimePoint	= std::chrono::time_point<std::chrono::high_resolution_clock>;
using FTimeDuration = std::chrono::duration<double>;

constexpr auto operator""_KB(unsigned long long X) -> uint64
{
	return X * (1ull << 10);
}
constexpr auto operator""_MB(unsigned long long X) -> uint64
{
	return X * (1ull << 20);
}
constexpr auto operator""_GB(unsigned long long X) -> uint64
{
	return X * (1ull << 30);
}

inline FTimePoint
TimePointNow()
{
	return std::chrono::high_resolution_clock::now();
}

inline double
DurationSec(FTimePoint TimeBegin, FTimePoint TimeEnd)
{
	return FTimeDuration(TimeEnd - TimeBegin).count();
}

inline double
DurationMs(FTimePoint TimeBegin, FTimePoint TimeEnd)
{
	return DurationSec(TimeBegin, TimeEnd) * 1000.0;
}

inline uint64
CalcChunkSize(uint64 ChunkIndex, uint64 ChunkSize, uint64 TotalDataSize)
{
	uint64 Begin = ChunkIndex * ChunkSize;
	uint64 End	 = std::min<uint64>(TotalDataSize, (ChunkIndex + 1) * ChunkSize);
	return End - Begin;
}

inline uint64
MakeU64(uint32 H, uint32 L)
{
	return uint64(L) | (uint64(H) << 32);
}

struct FRange
{
	uint64 Offset = 0;
	uint64 Size = 0;
};

struct FTimingLogger
{
	bool		bEnabled  = false;
	FTimePoint	TimeBegin = FTimePoint{};
	std::string Name;
	ELogLevel	LogLevel;

	FTimingLogger(const char* InName, ELogLevel InLogLevel, bool InEnabled = true);
	~FTimingLogger();
	void Finish();
};

inline double
SizeMb(double Size)
{
	return double(Size) / double(1 << 20);
}

inline double
SizeMb(uint64 Size)
{
	return SizeMb(double(Size));
}

inline uint64
DivUp(uint64 Num, uint64 Den)
{
	return (Num + Den - 1) / Den;
}

inline uint32
NextPow2(uint32 V)
{
	V--;
	V |= V >> 1;
	V |= V >> 2;
	V |= V >> 4;
	V |= V >> 8;
	V |= V >> 16;
	V++;
	return V;
}

std::string BytesToHexString(const uint8* Data, uint64 Size);

// Converts input bytes to hexadecimal ACII. Returns how many characters were written to output.
uint64 BytesToHexChars(char* Output, uint64 OutputSize, const uint8* Input, uint64 InputSize);
uint64 BytesToHexChars(wchar_t* Output, uint64 OutputSize, const uint8* Input, uint64 InputSize);

template<typename HashType>
std::string
HashToHexString(const HashType& Hash)
{
	return BytesToHexString(Hash.Data, Hash.Size());
}

std::wstring ConvertUtf8ToWide(std::string_view StringUtf8);
std::string	 ConvertWideToUtf8(std::wstring_view StringWide);
void		 ConvertWideToUtf8(std::wstring_view StringWide, std::string& Result);

std::string_view AsStringView(const FBuffer& Buffer);

inline std::string
ToString(const std::wstring_view WideStringView)
{
	return ConvertWideToUtf8(WideStringView);
}

inline std::string
ToString(const std::string_view StringView)
{
	return std::string(StringView);
}

std::string ToString(const FPath& Path);

std::string StringToLower(std::string_view Input);
std::wstring StringToLower(std::wstring_view Input);
std::wstring StringToUpper(std::wstring_view Input);

std::string StringEscape(const std::string_view Input);

bool StringEquals(const std::string_view A, const std::string_view B, bool bCaseSensitive = true);
bool StringStartsWith(const std::string_view String, const std::string_view Prefix, bool bCaseSensitive = true);

inline bool
UncasedStringEquals(const std::string_view A, const std::string_view B)
{
	return StringEquals(A, B, false);
}

inline bool
IsAsciiAlphabetCharacter(const char C)
{
	return (C >= 'a' && C <= 'z') || (C >= 'A' && C <= 'Z');
}

inline bool
IsAsciiNumericCharacter(const char C)
{
	return C >= '0' && C <= '9';
}

inline bool
IsAsciiAlphaNumericCharacter(const char C)
{
	return IsAsciiAlphabetCharacter(C) || IsAsciiNumericCharacter(C);
}

std::vector<std::string_view> SplitByAny(std::string_view String, const char* SeparatorCharacters);
inline std::vector<std::string_view>

SplitBy(std::string_view String, char SeparatorCharacter)
{
	char Chars[2] = {SeparatorCharacter, 0};
	return SplitByAny(String, Chars);
}

// Takes a drive-based path (e.g. P:/Foo/Bar) and converts it to universal form (e.g. //server/Foo/Bar), if possible.
// Otherwise, returns original path.
FPath GetUniversalPath(const FPath& Path);

template<typename T>
struct TArrayView
{
	const T* BeginPtr = nullptr;
	const T* EndPtr	  = nullptr;

	const T* begin() const { return BeginPtr; }	 // NOLINT
	const T* end() const { return EndPtr; }		 // NOLINT

	size_t Size() const { return end() - begin(); }
};

template<typename T>
TArrayView<T>
MakeView(const T* Ptr, size_t Count)
{
	TArrayView<T> Result;
	Result.BeginPtr = Ptr;
	Result.EndPtr	= Ptr + Count;
	return Result;
}

template<typename T>
TArrayView<T>
MakeView(const std::vector<T>& Container)
{
	return MakeView(Container.data(), Container.size());
}

inline uint64
AlignDownToMultiplePow2(uint64 X, uint64 MultiplePow2)
{
	return X & (~(MultiplePow2 - 1));
}

inline uint64
AlignUpToMultiplePow2(uint64 X, uint64 MultiplePow2)
{
	uint64 Remainder = X & (MultiplePow2 - 1);
	if (Remainder)
	{
		X += MultiplePow2 - Remainder;
	}
	return X;
}

template <class T>
concept TIsIntegral = std::is_integral_v<T>;

template <class T>
concept TIsSignedIntegral = TIsIntegral<T> && static_cast<T>(-1) < static_cast<T>(0);

template <class T>
concept TIsUnsignedIntegral = TIsIntegral<T> && !TIsSignedIntegral<T>;

template<typename T>
requires TIsUnsignedIntegral<T>
inline uint32
CheckedNarrow(T X)
{
	uint32 Narrowed = static_cast<uint32>(X);
	UNSYNC_ASSERTF(X == static_cast<uint64>(Narrowed), L"Value %llu does not fit into uint32", X);
	return uint32(X);
}

template<typename T>
requires TIsSignedIntegral<T>
inline int32
CheckedNarrow(T X)
{
	int32 Narrowed = static_cast<int32>(X);
	UNSYNC_ASSERTF(X == static_cast<int64>(Narrowed), L"Value %lld does not fit into int32", X);
	return Narrowed;
}

inline uint32
Xorshift32(uint32& State)
{
	uint32 X = State;
	X ^= X << 13;
	X ^= X >> 17;
	X ^= X << 5;
	State = X;
	return X;
}

inline void
FillRandomBytes(uint8* Output, uint64 Size, uint32 Seed)
{
	for (uint64 I = 0; I < Size; ++I)
	{
		Output[I] = Xorshift32(Seed) & 0xFF;
	}
}

#if !UNSYNC_COMPILER_MSVC
inline uint8
_BitScanReverse64(unsigned long* Index, uint64 Mask)
{
	if (Mask == 0)
	{
		return 0;
	}

	*Index = 63 - __builtin_clzll(Mask);
	return 1;
}
#endif	// !UNSYNC_COMPILER_MSVC

inline uint64
FloorLog264(uint64 X)
{
	unsigned long XLog2;
	long		  Mask = -long(_BitScanReverse64(&XLog2, X) != 0);
	return XLog2 & Mask;
}

inline uint32
CountLeadingZeros32(uint32 X)
{
	unsigned long XLog2;
	_BitScanReverse64(&XLog2, (uint64(X) << 1) | 1);
	return 32 - XLog2;
}

inline uint64
CountLeadingZeros64(uint64 X)
{
	unsigned long XLog2;
	long		  Mask = -long(_BitScanReverse64(&XLog2, X) != 0);
	return ((63 - XLog2) & Mask) | (64 & ~Mask);
}

template<typename StorageT>
struct TBitArrayInfo
{
	static constexpr size_t ElemSizeInBits	= sizeof(StorageT) * 8;
	const uint64			ElemIndex;
	const StorageT			BitMask;
	TBitArrayInfo(uint64 BitIndex) : ElemIndex(BitIndex / ElemSizeInBits), BitMask(StorageT(1) << (BitIndex % ElemSizeInBits)) {}
};

template<typename StorageT>
inline bool
BitArrayGet(const StorageT* Storage, uint64 BitIndex)
{
	TBitArrayInfo<StorageT> Info(BitIndex);
	return (Storage[Info.ElemIndex] & Info.BitMask) != 0;
}

template<typename StorageT>
inline void
BitArraySet(StorageT* Storage, uint64 BitIndex, bool bValue)
{
	TBitArrayInfo<StorageT> Info(BitIndex);
	if (bValue)
	{
		Storage[Info.ElemIndex] |= Info.BitMask;
	}
	else
	{
		Storage[Info.ElemIndex] &= ~Info.BitMask;
	}
}

FPath NormalizeFilenameWide(std::wstring_view InFilename);
FPath NormalizeFilenameUtf8(std::string_view InFilename);
FPath GetAbsoluteNormalPath(const FPath& InPath);

const FBuffer& GetSystemRootCerts();

template<typename T>
inline std::string_view
ToStringView(const T& Buf)
{
	return std::string_view(Buf.data(), Buf.size());
}

inline std::string
ToString(const fmt::memory_buffer& Buf)
{
	return std::string(ToStringView(Buf));
}

inline void
Append(fmt::memory_buffer& Buf, std::string_view V)
{
	Buf.append(V);
}

inline void
Append(fmt::memory_buffer& Buf, const char* S)
{
	Buf.append(std::string_view(S));
}

void OpenUrlInDefaultBrowser(const char* Address);

FPath GetUserHomeDirectory();

FHash256	GetAnonymizedMachineId(std::string_view Salt = {});
std::string GetAnonymizedMachineIdString(std::string_view Salt = {});

// Returns string in format 'Error code 123: Some description.`
std::string FormatSystemErrorMessage(int32 ErrorCode);

bool LooksLikeUrl(std::string_view Str);
bool LooksLikeHash160(const std::string_view Str);
bool LooksLikeHash160(const std::wstring_view Str);

// Json formatting helpers
void FormatJsonKeyValueStr(std::wstring& Output, std::wstring_view K, std::wstring_view V, std::wstring_view Suffix = {});
void FormatJsonKeyValueStr(std::string& Output, std::string_view K, std::string_view V, std::string_view Suffix = {});
void FormatJsonKeyValueUInt(std::wstring& Output, std::wstring_view K, uint64 V, std::wstring_view Suffix = {});
void FormatJsonKeyValueUInt(std::string& Output, std::string_view K, uint64 V, std::string_view Suffix = {});
void FormatJsonKeyValueBool(std::wstring& Output, std::wstring_view K, bool V, std::wstring_view Suffix = {});
void FormatJsonKeyValueBool(std::string& Output, std::string_view K, bool V, std::string_view Suffix = {});

}  // namespace unsync
