// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnsyncCommon.h"
#include "UnsyncUtil.h"

namespace unsync {

inline uint32
MeasureVarUint(uint64 X)
{
	return uint32(std::min(int32(FloorLog264(X)) / 7 + 1, 9));
}

inline uint32
MeasureVarUint(const void* X)
{
	return CountLeadingZeros32(uint8(~*static_cast<const uint8*>(X))) - 23;
}

inline uint32
WriteVarUint(uint64 X, void* Output)
{
	const uint32 Size		 = MeasureVarUint(X);
	uint8*		 OutputBytes = static_cast<uint8*>(Output) + Size - 1;
	switch (Size - 1)
	{
		case 8:
			*OutputBytes-- = uint8(X);
			X >>= 8;
		case 7:
			*OutputBytes-- = uint8(X);
			X >>= 8;
		case 6:
			*OutputBytes-- = uint8(X);
			X >>= 8;
		case 5:
			*OutputBytes-- = uint8(X);
			X >>= 8;
		case 4:
			*OutputBytes-- = uint8(X);
			X >>= 8;
		case 3:
			*OutputBytes-- = uint8(X);
			X >>= 8;
		case 2:
			*OutputBytes-- = uint8(X);
			X >>= 8;
		case 1:
			*OutputBytes-- = uint8(X);
			X >>= 8;
		default:
			break;
	}
	*OutputBytes = uint8(0xff << (9 - Size)) | uint8(X);
	return Size;
}

inline uint64
ReadVarUint(const void* Data, uint32& OutSize)
{
	const uint32 Size = MeasureVarUint(Data);
	OutSize			  = Size;

	const uint8* DataBytes = static_cast<const uint8*>(Data);
	uint64		 Value	   = *DataBytes++ & uint8(0xff >> Size);
	switch (Size - 1)
	{
		case 8:
			Value <<= 8;
			Value |= *DataBytes++;
		case 7:
			Value <<= 8;
			Value |= *DataBytes++;
		case 6:
			Value <<= 8;
			Value |= *DataBytes++;
		case 5:
			Value <<= 8;
			Value |= *DataBytes++;
		case 4:
			Value <<= 8;
			Value |= *DataBytes++;
		case 3:
			Value <<= 8;
			Value |= *DataBytes++;
		case 2:
			Value <<= 8;
			Value |= *DataBytes++;
		case 1:
			Value <<= 8;
			Value |= *DataBytes++;
		default:
			return Value;
	}
}

}  // namespace unsync
