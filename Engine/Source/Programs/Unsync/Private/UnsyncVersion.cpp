// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnsyncVersion.h"

#define UNSYNC_VERSION_STR "1.1.5"

namespace unsync {

const std::string&
GetVersionString()
{
	static std::string Result = []()
	{
		// TODO: generate a version string based on git state
		const char* GitRev	  = "";
		const char* GitBranch = "";
		// const char* GIT_TAG = nullptr;

		static char Str[256];

		if (strlen(GitBranch) && strlen(GitRev))
		{
			snprintf(Str, sizeof(Str), UNSYNC_VERSION_STR " [%s:%s]", GitBranch, GitRev);
		}
		else if (strlen(GitRev))
		{
			snprintf(Str, sizeof(Str), UNSYNC_VERSION_STR " [%s]", GitRev);
		}
		else
		{
			snprintf(Str, sizeof(Str), UNSYNC_VERSION_STR);
		}

		return std::string(Str);
	}();

	return Result;
}

}  // namespace unsync
