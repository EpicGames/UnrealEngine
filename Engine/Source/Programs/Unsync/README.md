# UNSYNC

This repository hosts the `unsync` client implementation, an incremental binary
download download and patching tool. The tool takes inspiration from `zsync`,
`rsync` and `casync`.

## Goals

* Transfer minimum amount of data over network
    * Compute binary difference from previous downloaded build
    * Download only new data chunks
* High speed regardless of geographic location
    * Latency-tolerant protocol and compression
    * Geographically distributed cache servers
* Enable satellite studios and work-from-home developers

## Implementation

The tool has three major components: command line client (this repository), GUI
(UnsyncUI) and an optional server (separate repository, not currently public).
The core algorithms used by the tool are nothing new and have been used by
similar industry standard tools for many years. 

Incremental build download requires a manifest to be generated for the source
data. This manifest contains a list of files with their sizes and timestamps. It
also contains a list of data blocks that make up each file. The blocks consist
of a 128/160/256 bit "strong" hash, 32 bit "weak" hash, size of the block and
offset of the block within the source file. The strong hash defines the identity
of the block, while weak hash is used for computing the binary difference. The
strong hash can be any general purpose hash, but the weak hash must be a rolling
hash. Current defaults are Blake3 (truncated to 160 bits) and Buzhash for strong
and weak hashes respectively.

Blocks can be generated in one of two ways: fixed or varying size. Fixed size
mode produces a more efficient / smaller patch, however varying mode allows
better block reuse between multiple files and builds. Additionally, the varying
mode can produce blocks for different builds entirely independently, without any
knowledge of which blocks may have been produced previously. Varying mode is
therefore used by default. 

The fixed block mode algorithm is well described in the rsync thesis and the
varying mode is most similar to casync.

The manifest files generated by the tool are stored next to the raw source
files. There is no central database or data storage as such. The manifest and
its associated source directory are self-contained and can be located anywhere.
Additionally, the source data remains compatible with other workflows, such as
copying files using robocopy, accessing individual files inside a build, etc.
When a particular build is no longer needed, it can be simply deleted from the
storage. No extra metadata garbage collection / dangling reference cleanup is
needed.

Having said that, some infrastructure can be added to significantly improve
download performance via chunk caching proxy servers. This is entirely optional
and still does not add any central database (raw source build data is always
self-contained).

## Usage

Run `unsync --help` to see all possible options. Some of the common functions
are described below.

### Generate a data set manifest

```
unsync hash -v <DIRECTORY>
```

This will recursively traverse the given directory, compute block hashes for all
encountered files and will write the output to
`<DIRECTORY>/.unsync/manifest.bin`. The `-v` argument enables logging, which is
otherwise entirely disabled by default unless an error occurs.

Typically the full data set is stored on a network drive which is mounted
locally. Storing data in Horde Storage is intended to be supported in the
future.

### Download a data set

```
unsync sync -v <SOURCE> <TARGET>
```

This will first attempt to copy the manifest file from
`<SOURCE>/.unsync/manifest.bin` to `<TARGET>/.unsync/temp/<hash>` (using hash of
the source path). The manifest is then loaded and compared against the current
contents of the target directory. File timestamps and sizes are checked first
and matching entries are skipped from further steps. Files that were identified
as "dirty" are then hashed, to find which source data blocks must be fetched and
which local base data blocks can be copied. The copy process then starts, which
consists of source and base data reading, which are done asynchronously.
Intermediate patched data is written to a temporary file, which is then verified
and renamed to final on success. Source and base data reading is done using
batched asynchronous IO operations, which aims to read data in chunks of up to
8MB by merging adjacent blocks when possible. Multiple blocks are read
simultaneously, while trying to overlap a few large downloads with some small
ones at any one point to hide the small read latency. Multiple files can be
processed in parallel, though currently only small files will be downloaded in
the background while large files are processed serially.

Several additional options can be passed to the sync command:

`--dry-run`

Download remote data and perform the patching in memory, without writing files
to disk (except caching the remote manifest file). --manifest FILENAME Specifies
an explicit manifest file path which should be used instead of implicit
<SOURCE>/.unsync/manifest.bin location. Can be used if manifests are stored
out-of-line.

`--threads N`

Allows limiting the concurrency of the tool to reduce memory usage and general
impact on the machine during patching. By default, all logical CPU cores will be
used if necessary, though typically the process is limited by IO and won't reach
high CPU utilization unless extremely fast SSDs are used. Example: --threads 1
will run everything in single-threaded mode.

`--buffered-files`

By default, unsync will use non-buffered file IO for best performance on SSDs.
However, on some machines it may be best to use buffered mode. In particular,
Horde worker machines perform much better with buffered files.

`--exclude foo,bar`

A basic mechanism for excluding some files from the download, using a
comma-separated list of words. Files with paths that contain any substring in
the excluded word list will be ignored. Currently, wildcard or glob syntax is
not supported. Example: --exclude .pdb,.exe,.map will reduce the Win64 build
download size if a developer intends to run a locally-compiled binary against
cooked data.

`--dfs NAME`

If remote build data is stored on a network file share which uses Distributed
File System (DFS), then Windows will automatically select the "best" server to
use from the current machine. Unfortunately, DFS data replication may take some
time and the latest build files might not show up in a chosen DFS mirror for
hours. To work around this problem, it is possible to explicitly specify the DFS
server name which is known to contain the latest data. Example: --dfs rdu will
choose a DFS mirror with "rdu" in the name, which is typically the best choice
if an unsync proxy server is used.

`--proxy server:port`

Uses a dedicated unsync proxy server as a primary data source. If connection to
proxy cannot be established, then the original source path will be used. Note
that the manifest file is still always downloaded from the original source
location, rather than from proxy. The client user must therefore have the
necessary access to the original network share.

`--no-cleanup`

By default, any extra files in the target directory will be deleted after
successful sync operation (similar to robocopy's mirror mode). This option can
be added to skip the deletion.

`--quick-source-validation`

Skip checking if all source files are present before starting a sync Any errors
due to missing source data will only be reported later during sync instead of at
startup Can save startup time significantly when sync source is a slow network
share

`--quick-difference`

Allow computing file difference based on previous sync manifest and file
timestamps Typically this is safe, as long as local file contents is not
modified without updating the timestamp If local file was somehow corrupt, the
error will be detected later during validation Can save significant time during
incremental syncs by avoiding redundant local file reads

`--quick`

Enables all `--quick-****` options

## How to build

It is possible to build Unsync as a standalone software using vcpkg and cmake or
as part of Unreal Engine (using Unreal Build Tool).

The codebase is currently designed to compile and work without dependencies on
the Unreal Engine core libraries, however this may change in the future.

Windows is the primary target platform, with Linux and Mac support being a work
in progress.

### Unreal Build Tool

```
Engine/Build/BatchFiles/RunUBT Unsync Win64 development
```

### Standalone build

#### Requirements

* _Windows:_ Visual Studio 2019 Version 16.10 or newer
* _Linux and Mac (WIP / experimental):_ GCC 11 or newer (Clang not supported)
* [CMake](https://cmake.org/download/) 3.16 or newer
* [Vcpkg package manger for C++](https://github.com/microsoft/vcpkg)
* `VCPKG_ROOT` environment variable containing `vcpkg` installation directory

#### Extra system dependencies on Ubuntu

```shell
> sudo add-apt-repository -y ppa:ubuntu-toolchain-r/test
> sudo apt install -y build-essential cmake pkg-config gcc-11
```

Generate Visual Studio solution in `build` sub-directory, compile `vcpkg`
dependencies and build optimized binary with debug symbols:

```cmd
> cmake -B build -S .
> cmake --build build --config RelWithDebInfo
```
