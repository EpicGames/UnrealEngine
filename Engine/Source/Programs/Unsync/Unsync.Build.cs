// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class Unsync : ModuleRules
{
	public Unsync(ReadOnlyTargetRules Target) : base(Target)
	{
		CppStandard = CppStandardVersion.Cpp20;
		bUseUnity = false;
		bEnableExceptions = true;
		bUseRTTI = true; // Needed by CLI11 library

		PrivateIncludePaths.Add(Path.Combine(ModuleDirectory, "ThirdParty"));

		PrivateDefinitions.Add("UNSYNC_USE_DEBUG_HEAP=1");

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDefinitions.Add("UNSYNC_PLATFORM_WINDOWS=1");
			PrivateDefinitions.Add("UNSYNC_PLATFORM_LINUX=0");
			PrivateDefinitions.Add("UNSYNC_PLATFORM_UNIX=0");
			PrivateDefinitions.Add("NOMINMAX=1");
			PrivateDefinitions.Add("WIN32_LEAN_AND_MEAN=1");
			PrivateDefinitions.Add("_SILENCE_CXX17_OLD_ALLOCATOR_MEMBERS_DEPRECATION_WARNING=1");
			PrivateDefinitions.Add("_CRT_SECURE_NO_WARNINGS=1");
			PrivateDefinitions.Add("_WINSOCK_DEPRECATED_NO_WARNINGS=1");
			PrivateDefinitions.Add("_SILENCE_CXX17_CODECVT_HEADER_DEPRECATION_WARNING=1");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PrivateDefinitions.Add("UNSYNC_PLATFORM_LINUX=0");
			PrivateDefinitions.Add("UNSYNC_PLATFORM_UNIX=1");
			PrivateDefinitions.Add("UNSYNC_PLATFORM_WINDOWS=0");
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PrivateDefinitions.Add("UNSYNC_PLATFORM_LINUX=1");
			PrivateDefinitions.Add("UNSYNC_PLATFORM_UNIX=1");
			PrivateDefinitions.Add("UNSYNC_PLATFORM_WINDOWS=0");
			PublicSystemLibraries.Add("dl");
		}

		PrivateDependencyModuleNames.Add("BLAKE3");
		PrivateDependencyModuleNames.Add("CLI11");
		PrivateDependencyModuleNames.Add("fmt");
		PrivateDependencyModuleNames.Add("http_parser");
		PrivateDependencyModuleNames.Add("LibreSSL");
		PrivateDependencyModuleNames.Add("zstd");
	}
}
