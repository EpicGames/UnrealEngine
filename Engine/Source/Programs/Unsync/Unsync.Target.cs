// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class UnsyncTarget : TargetRules
{
	public UnsyncTarget(TargetInfo Target) : base(Target)
	{
		Name = "unsync";
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "Unsync";		
		bBuildDeveloperTools = false;
		bCompileICU = false;
		bBuildWithEditorOnlyData = false;
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;
		bEnforceIWYU = false;
		GlobalDefinitions.Add("_CRT_SECURE_NO_WARNINGS");
		WindowsPlatform.TargetWindowsVersion = 0x0603; // Windows 8.1
		bUseStaticCRT = true;
		bIsBuildingConsoleApplication = true;
	}
}
