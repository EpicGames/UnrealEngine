{
  "name": "unsync",
  "version-string": "1",
  "dependencies": [
    "blake3",
    "cli11",
    "fmt",
    "http-parser",
    "libressl",
    "zstd"
  ]
}
